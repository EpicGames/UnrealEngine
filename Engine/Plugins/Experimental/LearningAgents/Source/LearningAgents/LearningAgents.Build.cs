// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LearningAgents : ModuleRules
{
	public LearningAgents(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Learning",
				"NNERuntimeBasicCpu",
			});
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"NNE",
				"Json",
				"JsonUtilities",
			});
	}
}
