// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsActions.h"

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsDebug.h"

#include "LearningArray.h"
#include "LearningLog.h"

#include "Containers/StaticArray.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsActions)

bool operator==(const FLearningAgentsActionSchemaElement& Lhs, const FLearningAgentsActionSchemaElement& Rhs)
{
	return Lhs.SchemaElement.Index == Rhs.SchemaElement.Index;
}

bool operator==(const FLearningAgentsActionObjectElement& Lhs, const FLearningAgentsActionObjectElement& Rhs)
{
	return Lhs.ObjectElement.Index == Rhs.ObjectElement.Index;
}

bool operator==(const FLearningAgentsActionModifierElement& Lhs, const FLearningAgentsActionModifierElement& Rhs)
{
	return Lhs.ModifierElement.Index == Rhs.ModifierElement.Index;
}

uint32 GetTypeHash(const FLearningAgentsActionSchemaElement& Element)
{
	return (uint32)Element.SchemaElement.Index;
}

uint32 GetTypeHash(const FLearningAgentsActionObjectElement& Element)
{
	return (uint32)Element.ObjectElement.Index;
}

uint32 GetTypeHash(const FLearningAgentsActionModifierElement& Element)
{
	return (uint32)Element.ModifierElement.Index;
}

namespace UE::Learning::Agents::Action::Private
{
	// We declare this here instead of in LearningArray.h because FName is part of the UObject system and right now the 
	// Learning core module does not have a dependency on the UObject part of UE which is a nice thing to preserve.	
	static inline FString FormatNames(const TLearningArrayView<1, const FName> Array, const int32 MaxItemNum = 16)
	{
		return Array::Format<FName>(Array, [](const FName& Value) { return FString::Printf(TEXT("%s"), *Value.ToString()); }, MaxItemNum);
	}

	static inline bool ContainsDuplicates(const TArrayView<const int32> Indices)
	{
		TSet<int32, DefaultKeyFuncs<int32>, TInlineSetAllocator<32>> IndicesSet;
		IndicesSet.Append(Indices);
		return Indices.Num() != IndicesSet.Num();
	}

	static inline bool ContainsDuplicates(const TArrayView<const FName> ElementNames)
	{
		TSet<FName, DefaultKeyFuncs<FName>, TInlineSetAllocator<32>> ElementNameSet;
		ElementNameSet.Append(ElementNames);
		return ElementNames.Num() != ElementNameSet.Num();
	}

	static inline const TCHAR* GetActionTypeString(const Learning::Action::EType ActionType)
	{
		switch (ActionType)
		{
		case Learning::Action::EType::Null: return TEXT("Null");
		case Learning::Action::EType::Continuous: return TEXT("Continuous");
		case Learning::Action::EType::DiscreteExclusive: return TEXT("DiscreteExclusive");
		case Learning::Action::EType::DiscreteInclusive: return TEXT("DiscreteInclusive");
		case Learning::Action::EType::NamedDiscreteExclusive: return TEXT("NamedDiscreteExclusive");
		case Learning::Action::EType::NamedDiscreteInclusive: return TEXT("NamedDiscreteInclusive");
		case Learning::Action::EType::And: return TEXT("Struct");
		case Learning::Action::EType::OrExclusive: return TEXT("ExclusiveUnion");
		case Learning::Action::EType::OrInclusive: return TEXT("InclusiveUnion");
		case Learning::Action::EType::Array: return TEXT("Array");
		case Learning::Action::EType::Encoding: return TEXT("Encoding");
		default:
			checkNoEntry();
			return TEXT("Unimplemented");
		}
	}

	static bool ValidateActionObjectMatchesSchema(
		const Learning::Action::FSchema& Schema,
		const Learning::Action::FSchemaElement SchemaElement,
		const Learning::Action::FObject& Object,
		const Learning::Action::FObjectElement ObjectElement)
	{
		// Check Elements are Valid

		if (!Schema.IsValid(SchemaElement))
		{
			UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Invalid Action Schema Element.");
			return false;
		}

		if (!Object.IsValid(ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Invalid Action Object Element.");
			return false;
		}

		// Check Names Match

		const FName ActionSchemaElementTag = Schema.GetTag(SchemaElement);
		const FName ActionObjectElementTag = Object.GetTag(ObjectElement);

		if (ActionSchemaElementTag != ActionObjectElementTag)
		{
			UE_LOGF(LogLearning, Warning, "ValidateActionObjectMatchesSchema: Action tag does not match Schema. Expected '%ls', got '%ls'.",
				*ActionSchemaElementTag.ToString(), *ActionObjectElementTag.ToString());
		}

		// Check Types Match

		const Learning::Action::EType ActionSchemaElementType = Schema.GetType(SchemaElement);
		const Learning::Action::EType ActionObjectElementType = Object.GetType(ObjectElement);

		if (ActionSchemaElementType != ActionObjectElementType)
		{
			UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' type does not match Schema. Expected type '%ls', got type '%ls'.",
				*ActionSchemaElementTag.ToString(),
				GetActionTypeString(ActionSchemaElementType),
				GetActionTypeString(ActionObjectElementType));
			return false;
		}

		// Type Specific Checks

		switch (ActionSchemaElementType)
		{
		case Learning::Action::EType::Null: return true;

		case Learning::Action::EType::Continuous:
		{
			const int32 SchemaElementSize = Schema.GetContinuous(SchemaElement).Num;
			const int32 ObjectElementSize = Object.GetContinuous(ObjectElement).Values.Num();

			if (SchemaElementSize != ObjectElementSize)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' size does not match Schema. Expected '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaElementSize,
					ObjectElementSize);
				return false;
			}

			return true;
		}

		case Learning::Action::EType::DiscreteExclusive:
		{
			const int32 SchemaElementSize = Schema.GetDiscreteExclusive(SchemaElement).Num;
			const int32 ObjectElementIndex = Object.GetDiscreteExclusive(ObjectElement).DiscreteIndex;

			if (ObjectElementIndex < 0 || ObjectElementIndex >= SchemaElementSize)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' index out of range for Schema. Expected '<%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaElementSize,
					ObjectElementIndex);
				return false;
			}

			return true;
		}

		case Learning::Action::EType::DiscreteInclusive:
		{
			const int32 SchemaElementSize = Schema.GetDiscreteInclusive(SchemaElement).Num;
			const TArrayView<const int32> ObjectElementIndices = Object.GetDiscreteInclusive(ObjectElement).DiscreteIndices;

			if (ObjectElementIndices.Num() > SchemaElementSize)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' too many indices provided. Expected at most '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaElementSize,
					ObjectElementIndices.Num());
				return false;
			}

			for (int32 SubElementIdx = 0; SubElementIdx < ObjectElementIndices.Num(); SubElementIdx++)
			{
				if (ObjectElementIndices[SubElementIdx] < 0 || ObjectElementIndices[SubElementIdx] >= SchemaElementSize)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' index out of range for Schema. Expected '<%i', got '%i'.",
						*ActionSchemaElementTag.ToString(),
						SchemaElementSize,
						ObjectElementIndices[SubElementIdx]);
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::NamedDiscreteExclusive:
		{
			const Learning::Action::FSchemaNamedDiscreteExclusiveParameters SchemaParameters = Schema.GetNamedDiscreteExclusive(SchemaElement);
			const Learning::Action::FObjectNamedDiscreteExclusiveParameters ObjectParameters = Object.GetNamedDiscreteExclusive(ObjectElement);
			check(SchemaParameters.ElementNames.Num() == SchemaParameters.ElementNames.Num());

			const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementName);

			if (SchemaSubElementIdx == INDEX_NONE)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' Schema does not include '%ls' action.",
					*ActionSchemaElementTag.ToString(),
					*ObjectParameters.ElementName.ToString());
				return false;
			}

			return true;
		}

		case Learning::Action::EType::NamedDiscreteInclusive:
		{
			const Learning::Action::FSchemaNamedDiscreteInclusiveParameters SchemaParameters = Schema.GetNamedDiscreteInclusive(SchemaElement);
			const Learning::Action::FObjectNamedDiscreteInclusiveParameters ObjectParameters = Object.GetNamedDiscreteInclusive(ObjectElement);

			if (ObjectParameters.ElementNames.Num() > SchemaParameters.ElementNames.Num())
			{
				UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' too many sub-actions provided. Expected at most '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaParameters.ElementNames.Num(),
					ObjectParameters.ElementNames.Num());
				return false;
			}

			for (int32 ObjectSubElementIdx = 0; ObjectSubElementIdx < ObjectParameters.ElementNames.Num(); ObjectSubElementIdx++)
			{
				const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementNames[ObjectSubElementIdx]);

				if (SchemaSubElementIdx == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' Schema does not include '%ls' Action.",
						*ActionSchemaElementTag.ToString(),
						*ObjectParameters.ElementNames[ObjectSubElementIdx].ToString());
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::And:
		{
			const Learning::Action::FSchemaAndParameters SchemaParameters = Schema.GetAnd(SchemaElement);
			const Learning::Action::FObjectAndParameters ObjectParameters = Object.GetAnd(ObjectElement);
			check(SchemaParameters.Elements.Num() == SchemaParameters.ElementNames.Num());
			check(ObjectParameters.Elements.Num() == ObjectParameters.ElementNames.Num());

			if (SchemaParameters.Elements.Num() != ObjectParameters.Elements.Num())
			{
				UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' number of sub-elements does not match Schema. Expected '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaParameters.Elements.Num(),
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaParameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 ObjectElementIdx = ObjectParameters.ElementNames.Find(SchemaParameters.ElementNames[SchemaElementIdx]);

				if (ObjectElementIdx == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' does not include '%ls' action required by Schema.",
						*ActionSchemaElementTag.ToString(),
						*SchemaParameters.ElementNames[SchemaElementIdx].ToString());
					return false;
				}

				if (!ValidateActionObjectMatchesSchema(
					Schema,
					SchemaParameters.Elements[SchemaElementIdx],
					Object,
					ObjectParameters.Elements[ObjectElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::OrExclusive:
		{
			const Learning::Action::FSchemaOrExclusiveParameters SchemaParameters = Schema.GetOrExclusive(SchemaElement);
			const Learning::Action::FObjectOrExclusiveParameters ObjectParameters = Object.GetOrExclusive(ObjectElement);
			check(SchemaParameters.Elements.Num() == SchemaParameters.ElementNames.Num());

			const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementName);

			if (SchemaSubElementIdx == INDEX_NONE)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' Schema does not include '%ls' action.",
					*ActionSchemaElementTag.ToString(),
					*ObjectParameters.ElementName.ToString());
				return false;
			}

			return ValidateActionObjectMatchesSchema(
				Schema,
				SchemaParameters.Elements[SchemaSubElementIdx],
				Object,
				ObjectParameters.Element);
		}

		case Learning::Action::EType::OrInclusive:
		{
			const Learning::Action::FSchemaOrInclusiveParameters SchemaParameters = Schema.GetOrInclusive(SchemaElement);
			const Learning::Action::FObjectOrInclusiveParameters ObjectParameters = Object.GetOrInclusive(ObjectElement);

			if (ObjectParameters.Elements.Num() > SchemaParameters.Elements.Num())
			{
				UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' too many sub-actions provided. Expected at most '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaParameters.Elements.Num(),
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 ObjectSubElementIdx = 0; ObjectSubElementIdx < ObjectParameters.Elements.Num(); ObjectSubElementIdx++)
			{
				const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementNames[ObjectSubElementIdx]);

				if (SchemaSubElementIdx == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' Schema does not include '%ls' action.",
						*ActionSchemaElementTag.ToString(),
						*ObjectParameters.ElementNames[ObjectSubElementIdx].ToString());
					return false;
				}

				if (!ValidateActionObjectMatchesSchema(
					Schema,
					SchemaParameters.Elements[SchemaSubElementIdx],
					Object,
					ObjectParameters.Elements[ObjectSubElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::Array:
		{
			const Learning::Action::FSchemaArrayParameters SchemaParameters = Schema.GetArray(SchemaElement);
			const Learning::Action::FObjectArrayParameters ObjectParameters = Object.GetArray(ObjectElement);

			if (ObjectParameters.Elements.Num() != SchemaParameters.Num)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Action '%ls' array incorrect size. Expected '%i' elements, got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaParameters.Num,
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 ElementIdx = 0; ElementIdx < ObjectParameters.Elements.Num(); ElementIdx++)
			{
				if (!ValidateActionObjectMatchesSchema(
					Schema,
					SchemaParameters.Element,
					Object,
					ObjectParameters.Elements[ElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::Encoding:
		{
			const Learning::Action::FSchemaEncodingParameters SchemaParameters = Schema.GetEncoding(SchemaElement);
			const Learning::Action::FObjectEncodingParameters ObjectParameters = Object.GetEncoding(ObjectElement);

			return ValidateActionObjectMatchesSchema(
				Schema,
				SchemaParameters.Element,
				Object,
				ObjectParameters.Element);
		}

		default:
		{
			checkNoEntry();
			return true;
		}
		}
	}


	static bool ValidateActionModifierMatchesSchema(
		const Learning::Action::FSchema& Schema,
		const Learning::Action::FSchemaElement SchemaElement,
		const Learning::Action::FModifier& Modifier,
		const Learning::Action::FModifierElement ModifierElement)
	{
		// Check Elements are Valid

		if (!Schema.IsValid(SchemaElement))
		{
			UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Invalid Action Schema Element.");
			return false;
		}

		if (!Modifier.IsValid(ModifierElement))
		{
			UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Invalid Action Modifier Element.");
			return false;
		}

		// Null Modifiers Match Everything

		const Learning::Action::EType ActionSchemaElementType = Schema.GetType(SchemaElement);
		const Learning::Action::EType ActionModifierElementType = Modifier.GetType(ModifierElement);

		if (ActionModifierElementType == Learning::Action::EType::Null)
		{
			return true;
		}

		// Check Names Match

		const FName ActionSchemaElementTag = Schema.GetTag(SchemaElement);
		const FName ActionModifierElementTag = Modifier.GetTag(ModifierElement);

		if (ActionSchemaElementTag != ActionModifierElementTag)
		{
			UE_LOGF(LogLearning, Warning, "ValidateActionModifierMatchesSchema: Action Modifier tag does not match Schema. Expected '%ls', got '%ls'.",
				*ActionSchemaElementTag.ToString(), *ActionModifierElementTag.ToString());
		}

		// Check Types Match

		if (ActionSchemaElementType != ActionModifierElementType)
		{
			UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' type does not match Schema. Expected type '%ls', got type '%ls'.",
				*ActionSchemaElementTag.ToString(),
				GetActionTypeString(ActionSchemaElementType),
				GetActionTypeString(ActionModifierElementType));
			return false;
		}

		// Type Specific Checks

		switch (ActionSchemaElementType)
		{
		case Learning::Action::EType::Null: return true;

		case Learning::Action::EType::Continuous:
		{
			const int32 SchemaElementSize = Schema.GetContinuous(SchemaElement).Num;
			const int32 ModifierMaskedElementSize = Modifier.GetContinuous(ModifierElement).Masked.Num();
			const int32 ModifierValueElementSize = Modifier.GetContinuous(ModifierElement).MaskedValues.Num();

			if (SchemaElementSize != ModifierMaskedElementSize)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' size does not match Schema. Expected '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaElementSize,
					ModifierMaskedElementSize);
				return false;
			}

			if (SchemaElementSize != ModifierValueElementSize)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' size does not match Schema. Expected '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaElementSize,
					ModifierValueElementSize);
				return false;
			}

			return true;
		}

		case Learning::Action::EType::DiscreteExclusive:
		{
			const int32 SchemaElementSize = Schema.GetDiscreteExclusive(SchemaElement).Num;
			const TArrayView<const int32> ModifierMaskedIndices = Modifier.GetDiscreteExclusive(ModifierElement).MaskedIndices;

			// For exclusive action we must have one less than the schema size or it means all are masked.
			if (ModifierMaskedIndices.Num() > SchemaElementSize - 1)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' too many indices provided. Expected at most '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaElementSize - 1,
					ModifierMaskedIndices.Num());
				return false;
			}

			for (int32 MaskIdx = 0; MaskIdx < ModifierMaskedIndices.Num(); MaskIdx++)
			{
				if (ModifierMaskedIndices[MaskIdx] < 0 || ModifierMaskedIndices[MaskIdx] >= SchemaElementSize)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' masked index out of range for Schema. Expected '<%i', got '%i'.",
						*ActionSchemaElementTag.ToString(),
						SchemaElementSize,
						ModifierMaskedIndices[MaskIdx]);
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::DiscreteInclusive:
		{
			const int32 SchemaElementSize = Schema.GetDiscreteInclusive(SchemaElement).Num;
			const TArrayView<const int32> ModifierMaskedIndices = Modifier.GetDiscreteInclusive(ModifierElement).MaskedIndices;

			if (ModifierMaskedIndices.Num() > SchemaElementSize)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' too many indices provided. Expected at most '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaElementSize,
					ModifierMaskedIndices.Num());
				return false;
			}

			for (int32 MaskIdx = 0; MaskIdx < ModifierMaskedIndices.Num(); MaskIdx++)
			{
				if (ModifierMaskedIndices[MaskIdx] < 0 || ModifierMaskedIndices[MaskIdx] >= SchemaElementSize)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' masked index out of range for Schema. Expected '<%i', got '%i'.",
						*ActionSchemaElementTag.ToString(),
						SchemaElementSize,
						ModifierMaskedIndices[MaskIdx]);
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::NamedDiscreteExclusive:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames;
			const TArrayView<const FName> ModifierMaskedNames = Modifier.GetNamedDiscreteExclusive(ModifierElement).MaskedElementNames;

			// For exclusive action we must have one less than the schema size or it means all are masked.
			if (ModifierMaskedNames.Num() > SchemaNames.Num() - 1)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' too many indices provided. Expected at most '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaNames.Num() - 1,
					ModifierMaskedNames.Num());
				return false;
			}

			for (int32 MaskIdx = 0; MaskIdx < ModifierMaskedNames.Num(); MaskIdx++)
			{
				if (!SchemaNames.Contains(ModifierMaskedNames[MaskIdx]))
				{
					UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' name '%ls' not found in Schema.",
						*ActionSchemaElementTag.ToString(),
						*ModifierMaskedNames[MaskIdx].ToString());
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::NamedDiscreteInclusive:
		{
			const TArrayView<const FName> SchemaNames = Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames;
			const TArrayView<const FName> ModifierMaskedNames = Modifier.GetNamedDiscreteInclusive(ModifierElement).MaskedElementNames;

			for (int32 MaskIdx = 0; MaskIdx < ModifierMaskedNames.Num(); MaskIdx++)
			{
				if (!SchemaNames.Contains(ModifierMaskedNames[MaskIdx]))
				{
					UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' name '%ls' not found in Schema.",
						*ActionSchemaElementTag.ToString(),
						*ModifierMaskedNames[MaskIdx].ToString());
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::And:
		{
			const Learning::Action::FSchemaAndParameters SchemaParameters = Schema.GetAnd(SchemaElement);
			const Learning::Action::FModifierAndParameters ModifierParameters = Modifier.GetAnd(ModifierElement);
			check(SchemaParameters.Elements.Num() == SchemaParameters.ElementNames.Num());
			check(ModifierParameters.Elements.Num() == ModifierParameters.ElementNames.Num());

			if (SchemaParameters.Elements.Num() < ModifierParameters.Elements.Num())
			{
				UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' too many sub-elements. Expected '<=%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaParameters.Elements.Num(),
					ModifierParameters.Elements.Num());
				return false;
			}

			for (int32 ModifierElementIdx = 0; ModifierElementIdx < ModifierParameters.Elements.Num(); ModifierElementIdx++)
			{
				const int32 SchemaElementIdx = SchemaParameters.ElementNames.Find(ModifierParameters.ElementNames[ModifierElementIdx]);

				if (SchemaElementIdx == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action '%ls' does not include '%ls' action given by Modifier.",
						*ActionSchemaElementTag.ToString(),
						*ModifierParameters.ElementNames[ModifierElementIdx].ToString());
					return false;
				}

				if (!ValidateActionModifierMatchesSchema(
					Schema,
					SchemaParameters.Elements[SchemaElementIdx],
					Modifier,
					ModifierParameters.Elements[ModifierElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::OrExclusive:
		{
			const Learning::Action::FSchemaOrExclusiveParameters SchemaParameters = Schema.GetOrExclusive(SchemaElement);
			const Learning::Action::FModifierOrExclusiveParameters ModifierParameters = Modifier.GetOrExclusive(ModifierElement);
			check(SchemaParameters.Elements.Num() == SchemaParameters.ElementNames.Num());

			// For exclusive action we must have one less than the schema size or it means all are masked.
			if (ModifierParameters.MaskedElements.Num() > SchemaParameters.Elements.Num() - 1)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' too many masked elements. Expected at most '%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaParameters.Elements.Num() - 1,
					ModifierParameters.MaskedElements.Num());
				return false;
			}

			for (int32 ModifierMaskIdx = 0; ModifierMaskIdx < ModifierParameters.MaskedElements.Num(); ModifierMaskIdx++)
			{
				const FName MaskedElement = ModifierParameters.MaskedElements[ModifierMaskIdx];

				if (SchemaParameters.ElementNames.Find(MaskedElement) == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action '%ls' does not include '%ls' action given by Modifier.",
						*ActionSchemaElementTag.ToString(),
						*MaskedElement.ToString());
					return false;
				}
			}

			// TODO: Check Mask does not contain all elements

			if (SchemaParameters.Elements.Num() < ModifierParameters.Elements.Num())
			{
				UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' too many sub-elements. Expected '<=%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaParameters.Elements.Num(),
					ModifierParameters.Elements.Num());
				return false;
			}

			for (int32 ModifierElementIdx = 0; ModifierElementIdx < ModifierParameters.Elements.Num(); ModifierElementIdx++)
			{
				const int32 SchemaElementIdx = SchemaParameters.ElementNames.Find(ModifierParameters.ElementNames[ModifierElementIdx]);

				if (SchemaElementIdx == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action '%ls' does not include '%ls' action given by Modifier.",
						*ActionSchemaElementTag.ToString(),
						*ModifierParameters.ElementNames[ModifierElementIdx].ToString());
					return false;
				}

				if (!ValidateActionModifierMatchesSchema(
					Schema,
					SchemaParameters.Elements[SchemaElementIdx],
					Modifier,
					ModifierParameters.Elements[ModifierElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::OrInclusive:
		{
			const Learning::Action::FSchemaOrInclusiveParameters SchemaParameters = Schema.GetOrInclusive(SchemaElement);
			const Learning::Action::FModifierOrInclusiveParameters ModifierParameters = Modifier.GetOrInclusive(ModifierElement);

			for (int32 ModifierMaskIdx = 0; ModifierMaskIdx < ModifierParameters.MaskedElements.Num(); ModifierMaskIdx++)
			{
				const FName MaskedElement = ModifierParameters.MaskedElements[ModifierMaskIdx];

				if (SchemaParameters.ElementNames.Find(MaskedElement) == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action '%ls' does not include '%ls' action given by Modifier.",
						*ActionSchemaElementTag.ToString(),
						*MaskedElement.ToString());
					return false;
				}
			}

			if (SchemaParameters.Elements.Num() < ModifierParameters.Elements.Num())
			{
				UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action Modifier '%ls' too many sub-elements. Expected '<=%i', got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaParameters.Elements.Num(),
					ModifierParameters.Elements.Num());
				return false;
			}

			for (int32 ModifierElementIdx = 0; ModifierElementIdx < ModifierParameters.Elements.Num(); ModifierElementIdx++)
			{
				const int32 SchemaElementIdx = SchemaParameters.ElementNames.Find(ModifierParameters.ElementNames[ModifierElementIdx]);

				if (SchemaElementIdx == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action '%ls' does not include '%ls' action given by Modifier.",
						*ActionSchemaElementTag.ToString(),
						*ModifierParameters.ElementNames[ModifierElementIdx].ToString());
					return false;
				}

				if (!ValidateActionModifierMatchesSchema(
					Schema,
					SchemaParameters.Elements[SchemaElementIdx],
					Modifier,
					ModifierParameters.Elements[ModifierElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::Array:
		{
			const Learning::Action::FSchemaArrayParameters SchemaParameters = Schema.GetArray(SchemaElement);
			const Learning::Action::FModifierArrayParameters ModifierParameters = Modifier.GetArray(ModifierElement);

			if (ModifierParameters.Elements.Num() != SchemaParameters.Num)
			{
				UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Action '%ls' array incorrect size. Expected '%i' elements, got '%i'.",
					*ActionSchemaElementTag.ToString(),
					SchemaParameters.Num,
					ModifierParameters.Elements.Num());
				return false;
			}

			for (int32 ElementIdx = 0; ElementIdx < ModifierParameters.Elements.Num(); ElementIdx++)
			{
				if (!ValidateActionModifierMatchesSchema(
					Schema,
					SchemaParameters.Element,
					Modifier,
					ModifierParameters.Elements[ElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Action::EType::Encoding:
		{
			const Learning::Action::FSchemaEncodingParameters SchemaParameters = Schema.GetEncoding(SchemaElement);
			const Learning::Action::FModifierEncodingParameters ModifierParameters = Modifier.GetEncoding(ModifierElement);

			return ValidateActionModifierMatchesSchema(
				Schema,
				SchemaParameters.Element,
				Modifier,
				ModifierParameters.Element);
		}

		default:
		{
			checkNoEntry();
			return false;
		}
		}
	}

	static void LogAction(
		const UE::Learning::Action::FObject& Object,
		const UE::Learning::Action::FObjectElement ObjectElement,
		const FString& Indentation,
		const FString& Prefix)
	{
		if (!Object.IsValid(ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "LogAction: Invalid Action Object Element.");
			return;
		}

		const UE::Learning::Action::EType Type = Object.GetType(ObjectElement);
		const FName Tag = Object.GetTag(ObjectElement);

		switch (Type)
		{
		case UE::Learning::Action::EType::Null:
		{
			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type));
			return;
		}

		case UE::Learning::Action::EType::Continuous:
		{
			const UE::Learning::Action::FObjectContinuousParameters Parameters = Object.GetContinuous(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls) %ls", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type), *UE::Learning::Array::FormatFloat(Parameters.Values));
			return;
		}

		case UE::Learning::Action::EType::DiscreteExclusive:
		{
			const UE::Learning::Action::FObjectDiscreteExclusiveParameters Parameters = Object.GetDiscreteExclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls) %i", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type), Parameters.DiscreteIndex);
			return;
		}

		case UE::Learning::Action::EType::DiscreteInclusive:
		{
			const UE::Learning::Action::FObjectDiscreteInclusiveParameters Parameters = Object.GetDiscreteInclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls) %ls", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type), *UE::Learning::Array::FormatInt32(Parameters.DiscreteIndices));
			return;
		}

		case UE::Learning::Action::EType::NamedDiscreteExclusive:
		{
			const UE::Learning::Action::FObjectNamedDiscreteExclusiveParameters Parameters = Object.GetNamedDiscreteExclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls) %ls", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type), *Parameters.ElementName.ToString());
			return;
		}

		case UE::Learning::Action::EType::NamedDiscreteInclusive:
		{
			const UE::Learning::Action::FObjectNamedDiscreteInclusiveParameters Parameters = Object.GetNamedDiscreteInclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls) %ls", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type), *UE::Learning::Agents::Action::Private::FormatNames(Parameters.ElementNames));
			return;
		}

		case UE::Learning::Action::EType::And:
		{
			const UE::Learning::Action::FObjectAndParameters Parameters = Object.GetAnd(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				LogAction(Object, Parameters.Elements[SubElementIdx], *(Indentation + TEXT("    ")), FString::Printf(TEXT("| \"%s\": "), *Parameters.ElementNames[SubElementIdx].ToString()));
			}

			return;
		}

		case UE::Learning::Action::EType::OrExclusive:
		{
			const UE::Learning::Action::FObjectOrExclusiveParameters Parameters = Object.GetOrExclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type));
			LogAction(Object, Parameters.Element, *(Indentation + TEXT("    ")), FString::Printf(TEXT("| \"%s\": "), *Parameters.ElementName.ToString()));

			return;
		}

		case UE::Learning::Action::EType::OrInclusive:
		{
			const UE::Learning::Action::FObjectOrInclusiveParameters Parameters = Object.GetOrInclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				LogAction(Object, Parameters.Elements[SubElementIdx], *(Indentation + TEXT("    ")), FString::Printf(TEXT("| \"%s\": "), *Parameters.ElementNames[SubElementIdx].ToString()));
			}

			return;
		}

		case UE::Learning::Action::EType::Array:
		{
			const UE::Learning::Action::FObjectArrayParameters Parameters = Object.GetArray(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				LogAction(Object, Parameters.Elements[SubElementIdx], *(Indentation + TEXT("    ")), FString::Printf(TEXT("| %3i:"), SubElementIdx));
			}

			return;
		}

		case UE::Learning::Action::EType::Encoding:
		{
			const UE::Learning::Action::FObjectEncodingParameters Parameters = Object.GetEncoding(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetActionTypeString(Type));
			LogAction(Object, Parameters.Element, *(Indentation + TEXT("    ")), TEXT("|"));

			return;
		}

		}
	}

	static inline FVector VectorLogSafe(const FVector V, const float Epsilon = UE_SMALL_NUMBER)
	{
		return FVector(
			FMath::Loge(FMath::Max(V.X, Epsilon)),
			FMath::Loge(FMath::Max(V.Y, Epsilon)),
			FMath::Loge(FMath::Max(V.Z, Epsilon)));
	}

	static inline FVector VectorExp(const FVector V)
	{
		return FVector(
			FMath::Exp(V.X),
			FMath::Exp(V.Y),
			FMath::Exp(V.Z));
	}

	static inline void NormalizeProbabilitiesExclusive(TArrayView<float> PriorProbabilities)
	{
		float Total = 0.0f;
		for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
		{
			if (PriorProbabilities[Idx] < 0.0f || PriorProbabilities[Idx] > 1.0f)
			{
				UE_LOGF(LogLearning, Warning, "NormalizeProbabilitiesExclusive: Invalid Prior Probability Given (%f), must be in range 0 to 1.", PriorProbabilities[Idx]);
			}

			PriorProbabilities[Idx] = FMath::Clamp(PriorProbabilities[Idx], 0.0f, 1.0f);
			Total += PriorProbabilities[Idx];
		}

		if (PriorProbabilities.Num() > 0 && FMath::Abs(Total) < UE_SMALL_NUMBER)
		{
			UE_LOGF(LogLearning, Warning, "NormalizeProbabilitiesExclusive: Prior Probabilities are too small. Should sum to 1.");

			for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
			{
				PriorProbabilities[Idx] = 1.0f / PriorProbabilities.Num();
			}
		}
		else
		{
			for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
			{
				PriorProbabilities[Idx] /= Total;
			}
		}
	}

	static inline void NormalizeProbabilitiesInclusive(TArrayView<float> PriorProbabilities)
	{
		for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
		{
			if (PriorProbabilities[Idx] < 0.0f || PriorProbabilities[Idx] > 1.0f)
			{
				UE_LOGF(LogLearning, Warning, "NormalizeProbabilitiesInclusive: Invalid Prior Probability Given (%f), must be in range 0 to 1.", PriorProbabilities[Idx]);
			}

			PriorProbabilities[Idx] = FMath::Clamp(PriorProbabilities[Idx], 0.0f, 1.0f);
		}
	}

	static inline Learning::Action::EEncodingActivationFunction GetEncodingActivationFunction(const ELearningAgentsActivationFunction ActivationFunction)
	{
		switch (ActivationFunction)
		{
		case ELearningAgentsActivationFunction::ReLU: return Learning::Action::EEncodingActivationFunction::ReLU;
		case ELearningAgentsActivationFunction::ELU: return Learning::Action::EEncodingActivationFunction::ELU;
		case ELearningAgentsActivationFunction::TanH: return Learning::Action::EEncodingActivationFunction::TanH;
		case ELearningAgentsActivationFunction::GELU: return Learning::Action::EEncodingActivationFunction::GELU;
		default: checkNoEntry(); return Learning::Action::EEncodingActivationFunction::ReLU;
		}
	}

	static inline Learning::Action::ENormalization GetNormalizationOption(const ELearningAgentsNormalization NormalizationOption)
	{
		switch (NormalizationOption)
		{
		case ELearningAgentsNormalization::Manual: return Learning::Action::ENormalization::Manual;
		case ELearningAgentsNormalization::AutoShared: return Learning::Action::ENormalization::AutoShared;
		case ELearningAgentsNormalization::AutoPerDimension: return Learning::Action::ENormalization::AutoPerDimension;
		default: checkNoEntry(); return Learning::Action::ENormalization::Manual;
		}
	}
}

bool ULearningAgentsActions::ValidateActionObjectMatchesSchema(
	const ULearningAgentsActionSchema* Schema,
	const FLearningAgentsActionSchemaElement SchemaElement,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement ObjectElement)
{
	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Schema is nullptr.");
		return false;
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "ValidateActionObjectMatchesSchema: Object is nullptr.");
		return false;
	}

	return UE::Learning::Agents::Action::Private::ValidateActionObjectMatchesSchema(
		Schema->ActionSchema,
		SchemaElement.SchemaElement,
		Object->ActionObject,
		ObjectElement.ObjectElement);
}

bool ULearningAgentsActions::ValidateActionModifierMatchesSchema(
	const ULearningAgentsActionSchema* Schema,
	const FLearningAgentsActionSchemaElement SchemaElement,
	const ULearningAgentsActionModifier* Modifier,
	const FLearningAgentsActionModifierElement ModifierElement)
{
	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Schema is nullptr.");
		return false;
	}

	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "ValidateActionModifierMatchesSchema: Modifier is nullptr.");
		return false;
	}

	return UE::Learning::Agents::Action::Private::ValidateActionModifierMatchesSchema(
		Schema->ActionSchema,
		SchemaElement.SchemaElement,
		Modifier->ActionModifier,
		ModifierElement.ModifierElement);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyNullAction(ULearningAgentsActionSchema* Schema, const FName Tag)
{
	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyNullAction: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	return { Schema->ActionSchema.CreateNull(Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyContinuousAction(ULearningAgentsActionSchema* Schema, const int32 Size, const FLearningAgentsActionNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyContinuousAction: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Size < 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyContinuousAction: Invalid Continuous Action Size '%i'.", Size);
		return FLearningAgentsActionSchemaElement();
	}

	if (Size == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyContinuousAction: Specifying zero-sized Continuous Action.");
	}

	return { Schema->ActionSchema.CreateContinuous({ Size, NormalizationSettings.Scales, NormalizationSettings.Offsets, UE::Learning::Agents::Action::Private::GetNormalizationOption(NormalizationSettings.NormalizationOption)}, Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyExclusiveDiscreteAction(ULearningAgentsActionSchema* Schema, const int32 Size, const TArray<float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyExclusiveDiscreteActionFromArrayView(Schema, Size, PriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyExclusiveDiscreteActionFromArrayView(ULearningAgentsActionSchema* Schema, const int32 Size, const TArrayView<const float> PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveDiscreteActionFromArrayView: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Size < 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveDiscreteActionFromArrayView: Invalid DiscreteExclusive Action Size '%i'.", Size);
		return FLearningAgentsActionSchemaElement();
	}

	if (Size == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyExclusiveDiscreteActionFromArrayView: Specifying zero-sized Exclusive Discrete Action.");
	}

	if (PriorProbabilities.Num() > Size)
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveDiscreteActionFromArrayView: Action Size is '%i' but got '%i' PriorProbabilities.", Size, PriorProbabilities.Num());
		return FLearningAgentsActionSchemaElement();
	}

	TArray<float, TInlineAllocator<16>> NormalizedPriorProbabilities;
	NormalizedPriorProbabilities.Init(Size == 0 ? 0.0f : 1.0f / Size, Size);
	for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
	{
		NormalizedPriorProbabilities[Idx] = PriorProbabilities[Idx];
	}
	UE::Learning::Agents::Action::Private::NormalizeProbabilitiesExclusive(NormalizedPriorProbabilities);

	return { Schema->ActionSchema.CreateDiscreteExclusive({ Size, MakeArrayView(NormalizedPriorProbabilities) }, Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyNamedExclusiveDiscreteAction(ULearningAgentsActionSchema* Schema, const TArray<FName>& Names, const TMap<FName, float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	const int32 NameNum = Names.Num();

	if (NameNum == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyNamedExclusiveDiscreteAction: Specifying zero-sized Named Exclusive Discrete Action.");
	}

	TArray<int32, TInlineAllocator<16>> NameIndices;
	TArray<float, TInlineAllocator<16>> NamePriorProbabilities;
	NameIndices.Empty(NameNum);
	NamePriorProbabilities.Empty(NameNum);

	int32 Index = 0;
	for (const FName& Name : Names)
	{
		NameIndices.Add(Index);
		const float* PriorProb = PriorProbabilities.Find(Name);
		NamePriorProbabilities.Add(PriorProb ? *PriorProb : NameNum == 0 ? 0.0f : 1.0f / NameNum);
		Index++;
	}

	// Sort Names According to FName

	NameIndices.Sort([Names](const int32 Lhs, const int32 Rhs)
		{
			return Names[Lhs].ToString().ToLower() < Names[Rhs].ToString().ToLower();
		});

	TArray<FName, TInlineAllocator<16>> SortedNames;
	TArray<float, TInlineAllocator<16>> SortedPriorProbabilities;
	SortedNames.SetNumUninitialized(NameNum);
	SortedPriorProbabilities.SetNumUninitialized(NameNum);
	for (int32 Idx = 0; Idx < NameNum; Idx++)
	{
		SortedNames[Idx] = Names[NameIndices[Idx]];
		SortedPriorProbabilities[Idx] = NamePriorProbabilities[NameIndices[Idx]];
	}

	return SpecifyNamedExclusiveDiscreteActionFromArrayViews(Schema, SortedNames, SortedPriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyNamedExclusiveDiscreteActionFromArrayViews(ULearningAgentsActionSchema* Schema, const TArrayView<const FName> Names, const TArrayView<const float> PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedExclusiveDiscreteActionFromArrayViews: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	const int32 NameNum = Names.Num();

	if (NameNum == 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedExclusiveDiscreteActionFromArrayViews: Specifying zero-sized Named Exclusive Discrete Action.");
		return FLearningAgentsActionSchemaElement();
	}

	if (PriorProbabilities.Num() > NameNum)
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedExclusiveDiscreteActionFromArrayViews: Action Size is '%i' but got '%i' PriorProbabilities.", NameNum, PriorProbabilities.Num());
		return FLearningAgentsActionSchemaElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(Names))
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedExclusiveDiscreteActionFromArrayViews: Names contain duplicates.");
		return FLearningAgentsActionSchemaElement();
	}

	TArray<float, TInlineAllocator<16>> NormalizedPriorProbabilities;
	NormalizedPriorProbabilities.Init(NameNum == 0 ? 0.0f : 1.0f / NameNum, NameNum);
	for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
	{
		NormalizedPriorProbabilities[Idx] = PriorProbabilities[Idx];
	}
	UE::Learning::Agents::Action::Private::NormalizeProbabilitiesExclusive(NormalizedPriorProbabilities);

	return { Schema->ActionSchema.CreateNamedDiscreteExclusive({ Names, MakeArrayView(NormalizedPriorProbabilities) }, Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyInclusiveDiscreteAction(ULearningAgentsActionSchema* Schema, const int32 Size, const TArray<float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyInclusiveDiscreteActionFromArrayView(Schema, Size, PriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyInclusiveDiscreteActionFromArrayView(ULearningAgentsActionSchema* Schema, const int32 Size, const TArrayView<const float> PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveDiscreteActionFromArrayView: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Size < 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveDiscreteActionFromArrayView: Invalid DiscreteInclusive Action Size '%i'.", Size);
		return FLearningAgentsActionSchemaElement();
	}

	if (PriorProbabilities.Num() > Size)
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveDiscreteActionFromArrayView: Action Size is '%i' but got '%i' PriorProbabilities.", Size, PriorProbabilities.Num());
		return FLearningAgentsActionSchemaElement();
	}

	if (Size == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyInclusiveDiscreteActionFromArrayView: Specifying zero-sized Inclusive Discrete Action.");
	}

	TArray<float, TInlineAllocator<16>> NormalizedPriorProbabilities;
	NormalizedPriorProbabilities.Init(0.5f, Size);
	for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
	{
		NormalizedPriorProbabilities[Idx] = PriorProbabilities[Idx];
	}
	UE::Learning::Agents::Action::Private::NormalizeProbabilitiesInclusive(NormalizedPriorProbabilities);

	return { Schema->ActionSchema.CreateDiscreteInclusive({ Size, MakeArrayView(NormalizedPriorProbabilities) }, Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyNamedInclusiveDiscreteAction(ULearningAgentsActionSchema* Schema, const TArray<FName> Names, const TMap<FName, float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	const int32 NameNum = Names.Num();

	if (NameNum == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyNamedInclusiveDiscreteAction: Specifying zero-sized Named Inclusive Discrete Action.");
	}

	TArray<int32, TInlineAllocator<16>> NameIndices;
	TArray<float, TInlineAllocator<16>> NamePriorProbabilities;
	NameIndices.Empty(NameNum);
	NamePriorProbabilities.Empty(NameNum);

	int32 Index = 0;
	for (const FName& Name : Names)
	{
		NameIndices.Add(Index);
		const float* PriorProb = PriorProbabilities.Find(Name);
		NamePriorProbabilities.Add(PriorProb ? *PriorProb : NameNum == 0 ? 0.0f : 1.0f / NameNum);
		Index++;
	}

	// Sort Names According to FName

	NameIndices.Sort([Names](const int32 Lhs, const int32 Rhs)
		{
			return Names[Lhs].ToString().ToLower() < Names[Rhs].ToString().ToLower();
		});

	TArray<FName, TInlineAllocator<16>> SortedNames;
	TArray<float, TInlineAllocator<16>> SortedPriorProbabilities;
	SortedNames.SetNumUninitialized(NameNum);
	SortedPriorProbabilities.SetNumUninitialized(NameNum);
	for (int32 Idx = 0; Idx < NameNum; Idx++)
	{
		SortedNames[Idx] = Names[NameIndices[Idx]];
		SortedPriorProbabilities[Idx] = NamePriorProbabilities[NameIndices[Idx]];
	}

	return SpecifyNamedInclusiveDiscreteActionFromArrayViews(Schema, SortedNames, SortedPriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyNamedInclusiveDiscreteActionFromArrayViews(ULearningAgentsActionSchema* Schema, const TArrayView<const FName> Names, const TArrayView<const float> PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedInclusiveDiscreteActionFromArrayViews: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	const int32 NameNum = Names.Num();

	if (PriorProbabilities.Num() > NameNum)
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedInclusiveDiscreteActionFromArrayViews: Action Size is '%i' but got '%i' PriorProbabilities.", NameNum, PriorProbabilities.Num());
		return FLearningAgentsActionSchemaElement();
	}

	if (NameNum == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyNamedInclusiveDiscreteActionFromArrayViews: Specifying zero-sized Named Inclusive Discrete Action.");
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(Names))
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedInclusiveDiscreteActionFromArrayViews: Names contain duplicates.");
		return FLearningAgentsActionSchemaElement();
	}

	TArray<float, TInlineAllocator<16>> NormalizedPriorProbabilities;
	NormalizedPriorProbabilities.Init(NameNum == 0 ? 0.0f : 1.0f / NameNum, NameNum);
	for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
	{
		NormalizedPriorProbabilities[Idx] = PriorProbabilities[Idx];
	}
	UE::Learning::Agents::Action::Private::NormalizeProbabilitiesInclusive(NormalizedPriorProbabilities);

	return { Schema->ActionSchema.CreateNamedDiscreteInclusive({ Names, MakeArrayView(NormalizedPriorProbabilities) }, Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyStructAction(ULearningAgentsActionSchema* Schema, const TMap<FName, FLearningAgentsActionSchemaElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyStructAction: Specifying zero-sized Struct Action.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<int32, TInlineAllocator<16>> SubElementIndices;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsActionSchemaElement, TInlineAllocator<16>> SubElements;
	SubElementIndices.Empty(SubElementNum);
	SubElementNames.Empty(SubElementNum);
	SubElements.Empty(SubElementNum);

	int32 Index = 0;
	for (const TPair<FName, FLearningAgentsActionSchemaElement>& Element : Elements)
	{
		SubElementIndices.Add(Index);
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
		Index++;
	}

	// Sort Elements According to FName

	SubElementIndices.Sort([SubElementNames](const int32 Lhs, const int32 Rhs)
		{
			return SubElementNames[Lhs].ToString().ToLower() < SubElementNames[Rhs].ToString().ToLower();
		});

	TArray<FName, TInlineAllocator<16>> SortedSubElementNames;
	TArray<FLearningAgentsActionSchemaElement, TInlineAllocator<16>> SortedSubElements;
	SortedSubElementNames.SetNumUninitialized(SubElementNum);
	SortedSubElements.SetNumUninitialized(SubElementNum);
	for (int32 Idx = 0; Idx < SubElementNum; Idx++)
	{
		SortedSubElementNames[Idx] = SubElementNames[SubElementIndices[Idx]];
		SortedSubElements[Idx] = SubElements[SubElementIndices[Idx]];
	}

	return SpecifyStructActionFromArrayViews(Schema, SortedSubElementNames, SortedSubElements, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyStructActionFromArrays(ULearningAgentsActionSchema* Schema, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionSchemaElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyStructActionFromArrayViews(Schema, ElementNames, Elements, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyStructActionFromArrayViews(ULearningAgentsActionSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionSchemaElement> Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyStructActionFromArrayViews: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyStructActionFromArrayViews: Specifying zero-sized Struct Action.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "SpecifyStructActionFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsActionSchemaElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "SpecifyStructActionFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsActionSchemaElement();
	}

	TArray<UE::Learning::Action::FSchemaElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsActionSchemaElement& Element : Elements)
	{
		if (!Schema->ActionSchema.IsValid(Element.SchemaElement))
		{
			UE_LOGF(LogLearning, Error, "SpecifyStructActionFromArrayViews: Invalid Action Object.");
			return FLearningAgentsActionSchemaElement();
		}

		SubElements.Add(Element.SchemaElement);
	}

	return { Schema->ActionSchema.CreateAnd({ ElementNames, SubElements }, Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyExclusiveUnionAction(ULearningAgentsActionSchema* Schema, const TMap<FName, FLearningAgentsActionSchemaElement>& Elements, const TMap<FName, float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyExclusiveUnionAction: Specifying zero-sized Exclusive Union Action.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<int32, TInlineAllocator<16>> SubElementIndices;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsActionSchemaElement, TInlineAllocator<16>> SubElements;
	TArray<float, TInlineAllocator<16>> SubElementPriorProbabilities;
	SubElementIndices.Empty(SubElementNum);
	SubElementNames.Empty(SubElementNum);
	SubElements.Empty(SubElementNum);
	SubElementPriorProbabilities.Empty(SubElementNum);

	int32 Index = 0;
	for (const TPair<FName, FLearningAgentsActionSchemaElement>& Element : Elements)
	{
		SubElementIndices.Add(Index);
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
		const float* PriorProb = PriorProbabilities.Find(Element.Key);
		SubElementPriorProbabilities.Add(PriorProb ? *PriorProb : SubElementNum == 0 ? 0.0f : 1.0f / SubElementNum);
		Index++;
	}

	// Sort Elements According to FName

	SubElementIndices.Sort([SubElementNames](const int32 Lhs, const int32 Rhs)
		{
			return SubElementNames[Lhs].ToString().ToLower() < SubElementNames[Rhs].ToString().ToLower();
		});

	TArray<FName, TInlineAllocator<16>> SortedSubElementNames;
	TArray<FLearningAgentsActionSchemaElement, TInlineAllocator<16>> SortedSubElements;
	TArray<float, TInlineAllocator<16>> SortedSubElementPriorProbabilities;
	SortedSubElementNames.SetNumUninitialized(SubElementNum);
	SortedSubElements.SetNumUninitialized(SubElementNum);
	SortedSubElementPriorProbabilities.SetNumUninitialized(SubElementNum);
	for (int32 Idx = 0; Idx < SubElementNum; Idx++)
	{
		SortedSubElementNames[Idx] = SubElementNames[SubElementIndices[Idx]];
		SortedSubElements[Idx] = SubElements[SubElementIndices[Idx]];
		SortedSubElementPriorProbabilities[Idx] = SubElementPriorProbabilities[SubElementIndices[Idx]];
	}

	return SpecifyExclusiveUnionActionFromArrayViews(Schema, SortedSubElementNames, SortedSubElements, SortedSubElementPriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyExclusiveUnionActionFromArrays(ULearningAgentsActionSchema* Schema, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionSchemaElement>& Elements, const TArray<float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyExclusiveUnionActionFromArrayViews(Schema, ElementNames, Elements, PriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyExclusiveUnionActionFromArrayViews(ULearningAgentsActionSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionSchemaElement> Elements, const TArrayView<const float> PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionActionFromArrayViews: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyExclusiveUnionActionFromArrayViews: Specifying zero-sized Exclusive Union Action.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionActionFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsActionSchemaElement();
	}

	if (PriorProbabilities.Num() > Elements.Num())
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionActionFromArrayViews: Action Size is '%i' but got '%i' PriorProbabilities.", Elements.Num(), PriorProbabilities.Num());
		return FLearningAgentsActionSchemaElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionActionFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsActionSchemaElement();
	}

	TArray<UE::Learning::Action::FSchemaElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsActionSchemaElement& Element : Elements)
	{
		if (!Schema->ActionSchema.IsValid(Element.SchemaElement))
		{
			UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionActionFromArrayViews: Invalid Action Object.");
			return FLearningAgentsActionSchemaElement();
		}

		SubElements.Add(Element.SchemaElement);
	}

	TArray<float, TInlineAllocator<16>> NormalizedPriorProbabilities;
	NormalizedPriorProbabilities.Init(Elements.Num() == 0 ? 0.0f : 1.0f / Elements.Num(), Elements.Num());
	for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
	{
		NormalizedPriorProbabilities[Idx] = PriorProbabilities[Idx];
	}
	UE::Learning::Agents::Action::Private::NormalizeProbabilitiesExclusive(NormalizedPriorProbabilities);

	return { Schema->ActionSchema.CreateOrExclusive({ ElementNames, SubElements, MakeArrayView(NormalizedPriorProbabilities) }, Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyInclusiveUnionAction(ULearningAgentsActionSchema* Schema, const TMap<FName, FLearningAgentsActionSchemaElement>& Elements, const TMap<FName, float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyInclusiveUnionAction: Specifying zero-sized Inclusive Union Action.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<int32, TInlineAllocator<16>> SubElementIndices;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsActionSchemaElement, TInlineAllocator<16>> SubElements;
	TArray<float, TInlineAllocator<16>> SubElementPriorProbabilities;
	SubElementIndices.Empty(SubElementNum);
	SubElementNames.Empty(SubElementNum);
	SubElements.Empty(SubElementNum);
	SubElementPriorProbabilities.Empty(SubElementNum);

	int32 Index = 0;
	for (const TPair<FName, FLearningAgentsActionSchemaElement>& Element : Elements)
	{
		SubElementIndices.Add(Index);
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
		const float* PriorProb = PriorProbabilities.Find(Element.Key);
		SubElementPriorProbabilities.Add(PriorProb ? *PriorProb : SubElementNum == 0 ? 0.0f : 1.0f / SubElementNum);
		Index++;
	}

	// Sort Elements According to FName

	SubElementIndices.Sort([SubElementNames](const int32 Lhs, const int32 Rhs)
		{
			return SubElementNames[Lhs].ToString().ToLower() < SubElementNames[Rhs].ToString().ToLower();
		});

	TArray<FName, TInlineAllocator<16>> SortedSubElementNames;
	TArray<FLearningAgentsActionSchemaElement, TInlineAllocator<16>> SortedSubElements;
	TArray<float, TInlineAllocator<16>> SortedSubElementPriorProbabilities;
	SortedSubElementNames.SetNumUninitialized(SubElementNum);
	SortedSubElements.SetNumUninitialized(SubElementNum);
	SortedSubElementPriorProbabilities.SetNumUninitialized(SubElementNum);
	for (int32 Idx = 0; Idx < SubElementNum; Idx++)
	{
		SortedSubElementNames[Idx] = SubElementNames[SubElementIndices[Idx]];
		SortedSubElements[Idx] = SubElements[SubElementIndices[Idx]];
		SortedSubElementPriorProbabilities[Idx] = SubElementPriorProbabilities[SubElementIndices[Idx]];
	}

	return SpecifyInclusiveUnionActionFromArrayViews(Schema, SortedSubElementNames, SortedSubElements, SortedSubElementPriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyInclusiveUnionActionFromArrays(ULearningAgentsActionSchema* Schema, const TArray<FName> ElementNames, const TArray<FLearningAgentsActionSchemaElement>& Elements, const TArray<float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyInclusiveUnionActionFromArrayViews(Schema, ElementNames, Elements, PriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyInclusiveUnionActionFromArrayViews(ULearningAgentsActionSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionSchemaElement> Elements, const TArrayView<const float> PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionActionFromArrayViews: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyInclusiveUnionActionFromArrayViews: Specifying zero-sized Inclusive Union Action.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionActionFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsActionSchemaElement();
	}

	if (PriorProbabilities.Num() > Elements.Num())
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionActionFromArrayViews: Action Size is '%i' but got '%i' PriorProbabilities.", Elements.Num(), PriorProbabilities.Num());
		return FLearningAgentsActionSchemaElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionActionFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsActionSchemaElement();
	}

	TArray<UE::Learning::Action::FSchemaElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsActionSchemaElement& Element : Elements)
	{
		if (!Schema->ActionSchema.IsValid(Element.SchemaElement))
		{
			UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionActionFromArrayViews: Invalid Action Object.");
			return FLearningAgentsActionSchemaElement();
		}

		SubElements.Add(Element.SchemaElement);
	}

	TArray<float, TInlineAllocator<16>> NormalizedPriorProbabilities;
	NormalizedPriorProbabilities.Init(0.5f, Elements.Num());
	for (int32 Idx = 0; Idx < PriorProbabilities.Num(); Idx++)
	{
		NormalizedPriorProbabilities[Idx] = PriorProbabilities[Idx];
	}
	UE::Learning::Agents::Action::Private::NormalizeProbabilitiesInclusive(NormalizedPriorProbabilities);

	return { Schema->ActionSchema.CreateOrInclusive({ ElementNames, SubElements, MakeArrayView(NormalizedPriorProbabilities) }, Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyStaticArrayAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsActionSchemaElement Element, const int32 Num, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyStaticArrayAction: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Num < 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyStaticArrayAction: Invalid Action Static Array Num %i.", Num);
		return FLearningAgentsActionSchemaElement();
	}

	if (Num == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyStaticArrayAction: Specifying zero-sized Static Array Action.");
	}

	if (!Schema->ActionSchema.IsValid(Element.SchemaElement))
	{
		UE_LOGF(LogLearning, Error, "SpecifyStaticArrayAction: Invalid Action Object.");
		return FLearningAgentsActionSchemaElement();
	}

	return { Schema->ActionSchema.CreateArray({ Element.SchemaElement, Num }, Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyPairAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsActionSchemaElement Key, const FLearningAgentsActionSchemaElement Value, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyStructActionFromArrayViews(Schema, { TEXT("Key"), TEXT("Value") }, { Key, Value }, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyEnumAction(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TMap<uint8, float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "SpecifyEnumAction: Enum is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Enum->NumEnums() <= 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyEnumAction: Invalid Enum.");
		return FLearningAgentsActionSchemaElement();
	}

	TArray<float, TInlineAllocator<16>> PriorProbabilitiesArray;
	PriorProbabilitiesArray.Init((Enum->NumEnums() - 1) == 0 ? 0.0f : 1.0f / (Enum->NumEnums() - 1), Enum->NumEnums() - 1);
	for (const TPair<uint8, float> Prior : PriorProbabilities)
	{
		const int32 EnumIndex = Enum->GetIndexByValue(Prior.Key);
		if (EnumIndex != INDEX_NONE)
		{
			PriorProbabilitiesArray[EnumIndex] = Prior.Value;
		}
	}

	return SpecifyEnumActionFromArrayView(Schema, Enum, PriorProbabilitiesArray, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyEnumActionFromArray(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TArray<float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyEnumActionFromArrayView(Schema, Enum, PriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyEnumActionFromArrayView(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TArrayView<const float> PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "SpecifyEnumActionFromArrayView: Enum is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	return SpecifyExclusiveDiscreteActionFromArrayView(Schema, Enum->NumEnums() - 1, PriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyBitmaskAction(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TMap<uint8, float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "SpecifyBitmaskAction: Enum is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Enum->NumEnums() <= 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyBitmaskAction: Invalid Bitmask.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_LOGF(LogLearning, Error, "SpecifyBitmaskAction: Too many values in Enum to use as Bitmask (%i).", Enum->NumEnums() - 1);
		return FLearningAgentsActionSchemaElement();
	}

	TArray<float, TInlineAllocator<16>> PriorProbabilitiesArray;
	PriorProbabilitiesArray.Init(0.5f, Enum->NumEnums() - 1);
	for (const TPair<uint8, float> Prior : PriorProbabilities)
	{
		const int32 EnumIndex = Enum->GetIndexByValue(Prior.Key);
		if (EnumIndex != INDEX_NONE)
		{
			PriorProbabilitiesArray[EnumIndex] = Prior.Value;
		}
	}

	return SpecifyBitmaskActionFromArrayView(Schema, Enum, PriorProbabilitiesArray, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyBitmaskActionFromArray(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TArray<float>& PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyBitmaskActionFromArrayView(Schema, Enum, PriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyBitmaskActionFromArrayView(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TArrayView<const float> PriorProbabilities, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "SpecifyBitmaskActionFromArrayView: Enum is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_LOGF(LogLearning, Error, "SpecifyBitmaskActionFromArrayView: Too many values in Enum to use as Bitmask (%i).", Enum->NumEnums() - 1);
		return FLearningAgentsActionSchemaElement();
	}

	return SpecifyInclusiveDiscreteActionFromArrayView(Schema, Enum->NumEnums() - 1, PriorProbabilities, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyOptionalAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsActionSchemaElement Element, const float PriorProbability, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyExclusiveUnionActionFromArrayViews(Schema, { TEXT("Null"), TEXT("Valid") }, { SpecifyNullAction(Schema), Element }, { 1.0f - PriorProbability, PriorProbability }, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyEitherAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsActionSchemaElement A, const FLearningAgentsActionSchemaElement B, const float PriorProbabilityOfA, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyExclusiveUnionActionFromArrayViews(Schema, { TEXT("A"), TEXT("B") }, { A, B }, { 1.0f - PriorProbabilityOfA, PriorProbabilityOfA }, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyEncodingAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsActionSchemaElement Element, const int32 EncodingSize, const int32 HiddenLayerNum, const ELearningAgentsActivationFunction ActivationFunction, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyEncodingAction: Schema is nullptr.");
		return FLearningAgentsActionSchemaElement();
	}

	if (EncodingSize < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyEncodingAction: Invalid Action EncodingSize '%i' - must be greater than zero.", EncodingSize);
		return FLearningAgentsActionSchemaElement();
	}

	if (HiddenLayerNum < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyEncodingAction: Invalid Action HiddenLayerNum '%i' - must be greater than zero.", HiddenLayerNum);
		return FLearningAgentsActionSchemaElement();
	}

	if (!Schema->ActionSchema.IsValid(Element.SchemaElement))
	{
		UE_LOGF(LogLearning, Error, "SpecifyEncodingAction: Invalid Action Object.");
		return FLearningAgentsActionSchemaElement();
	}

	return { Schema->ActionSchema.CreateEncoding({ Element.SchemaElement, EncodingSize, HiddenLayerNum, UE::Learning::Agents::Action::Private::GetEncodingActivationFunction(ActivationFunction) }, Tag) };
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyBoolAction(ULearningAgentsActionSchema* Schema, const float PriorProbability, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyExclusiveDiscreteActionFromArrayView(Schema, 2, { 1.0f - PriorProbability, PriorProbability }, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyFloatAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsFloatActionNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	FLearningAgentsActionNormalizationSettings Settings;
	Settings.Scales = { NormalizationSettings.Scale };
	Settings.Offsets = { NormalizationSettings.Offset };
	Settings.NormalizationOption = { NormalizationSettings.NormalizationOption };

	return SpecifyContinuousAction(Schema, 1, Settings, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyLocationAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsVectorActionNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	FLearningAgentsActionNormalizationSettings Settings;
	Settings.Scales = { (float)NormalizationSettings.Scales.X, (float)NormalizationSettings.Scales.Y, (float)NormalizationSettings.Scales.Z };
	Settings.Offsets = { (float)NormalizationSettings.Offsets.X, (float)NormalizationSettings.Offsets.Y, (float)NormalizationSettings.Offsets.Z };
	Settings.NormalizationOption = { NormalizationSettings.NormalizationOption };

	return SpecifyContinuousAction(Schema, 3, Settings, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyRotationAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsVectorActionNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	FLearningAgentsActionNormalizationSettings Settings;
	Settings.Scales = { (float)NormalizationSettings.Scales.X, (float)NormalizationSettings.Scales.Y, (float)NormalizationSettings.Scales.Z };
	Settings.Offsets = { (float)NormalizationSettings.Offsets.X, (float)NormalizationSettings.Offsets.Y, (float)NormalizationSettings.Offsets.Z };
	Settings.NormalizationOption = { NormalizationSettings.NormalizationOption };

	return SpecifyContinuousAction(Schema, 3, Settings, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyScaleAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsVectorActionNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	FLearningAgentsActionNormalizationSettings Settings;
	Settings.Scales = { (float)NormalizationSettings.Scales.X, (float)NormalizationSettings.Scales.Y, (float)NormalizationSettings.Scales.Z };
	Settings.Offsets = { (float)NormalizationSettings.Offsets.X, (float)NormalizationSettings.Offsets.Y, (float)NormalizationSettings.Offsets.Z };
	Settings.NormalizationOption = NormalizationSettings.NormalizationOption;

	return SpecifyContinuousAction(Schema, 3, Settings, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyTransformAction(
	ULearningAgentsActionSchema* Schema,
	const FLearningAgentsVectorActionNormalizationSettings& LocationNormalization,
	const FLearningAgentsVectorActionNormalizationSettings& RotationNormalization,
	const FLearningAgentsVectorActionNormalizationSettings& ScaleNormalization,
	const bool bActive,
	const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyStructActionFromArrayViews(Schema,
		{
			TEXT("Location"),
			TEXT("Rotation"),
			TEXT("Scale")
		},
			{
				SpecifyLocationAction(Schema, LocationNormalization),
				SpecifyRotationAction(Schema, RotationNormalization),
				SpecifyScaleAction(Schema, ScaleNormalization)
			},
		bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyAngleAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsFloatActionNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	FLearningAgentsActionNormalizationSettings Settings;
	Settings.Scales = { NormalizationSettings.Scale };
	Settings.Offsets = { NormalizationSettings.Offset };
	Settings.NormalizationOption = NormalizationSettings.NormalizationOption;

	return SpecifyContinuousAction(Schema, 1, Settings, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyVelocityAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsVectorActionNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	FLearningAgentsActionNormalizationSettings Settings;
	Settings.Scales = { (float)NormalizationSettings.Scales.X, (float)NormalizationSettings.Scales.Y, (float)NormalizationSettings.Scales.Z };
	Settings.Offsets = { (float)NormalizationSettings.Offsets.X, (float)NormalizationSettings.Offsets.Y, (float)NormalizationSettings.Offsets.Z };
	Settings.NormalizationOption = NormalizationSettings.NormalizationOption;

	return SpecifyContinuousAction(Schema, 3, Settings, bActive, Tag);
}

FLearningAgentsActionSchemaElement ULearningAgentsActions::SpecifyDirectionAction(ULearningAgentsActionSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullAction(Schema, Tag);
	}

	return SpecifyContinuousAction(Schema, 3, FLearningAgentsActionNormalizationSettings(), bActive, Tag);
}

void ULearningAgentsActions::LogAction(const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element)
{
	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "LogAction: Object is nullptr.");
		return;
	}

	UE::Learning::Agents::Action::Private::LogAction(Object->ActionObject, Element.ObjectElement, TEXT(""), TEXT(""));
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeNullAction(ULearningAgentsActionObject* Object, const FName Tag)
{
	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeNullAction: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	return { Object->ActionObject.CreateNull(Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeContinuousAction(
	ULearningAgentsActionObject* Object,
	const TArray<float>& Values,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeContinuousActionFromArrayView(Object, Values, bActive, Tag, VisualLoggerSettings);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeContinuousActionFromArrayView(
	ULearningAgentsActionObject* Object,
	const TArrayView<const float> Values,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeContinuousActionFromArrayView: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	if (Values.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeContinuousActionFromArrayView: Creating zero-sized Continuous Action.");
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nEncoded: %s\nValues: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*UE::Learning::Array::FormatFloat(Values),
			*UE::Learning::Array::FormatFloat(Values)); // Encoded is identical to provided values
	}
#endif

	return { Object->ActionObject.CreateContinuous({ Values }, Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeExclusiveDiscreteAction(
	ULearningAgentsActionObject* Object,
	const int32 Index,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveDiscreteAction: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	if (Index < 0)
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveDiscreteAction: Invalid Action Index %i.", Index);
		return FLearningAgentsActionObjectElement();
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nIndex: [%i]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Index);
	}
#endif

	return { Object->ActionObject.CreateDiscreteExclusive({ Index }, Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeNamedExclusiveDiscreteAction(
	ULearningAgentsActionObject* Object,
	const FName Name,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeNamedExclusiveDiscreteAction: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nName: [%s]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*Name.ToString());
	}
#endif

	return { Object->ActionObject.CreateNamedDiscreteExclusive({ Name }, Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeInclusiveDiscreteAction(
	ULearningAgentsActionObject* Object,
	const TArray<int32>& Indices,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeInclusiveDiscreteActionFromArrayView(Object, Indices, bActive, Tag, VisualLoggerSettings);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeInclusiveDiscreteActionFromArrayView(
	ULearningAgentsActionObject* Object,
	const TArrayView<const int32> Indices,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveDiscreteActionFromArrayView: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(Indices))
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveDiscreteActionFromArrayView: Indices contain duplicates.");
		return FLearningAgentsActionObjectElement();
	}

	const int32 IndexNum = Indices.Num();

	for (int32 IndexIdx = 0; IndexIdx < IndexNum; IndexIdx++)
	{
		if (Indices[IndexIdx] < 0)
		{
			UE_LOGF(LogLearning, Error, "MakeInclusiveDiscreteActionFromArrayView: Invalid Action Index %i.", Indices[IndexIdx]);
			return FLearningAgentsActionObjectElement();
		}
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nIndices: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*UE::Learning::Array::FormatInt32(Indices, 256));
	}
#endif

	return { Object->ActionObject.CreateDiscreteInclusive({ Indices }, Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeNamedInclusiveDiscreteAction(
	ULearningAgentsActionObject* Object,
	const TArray<FName>& Names,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeNamedInclusiveDiscreteActionFromArrayView(Object, Names, bActive, Tag, VisualLoggerSettings);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeNamedInclusiveDiscreteActionFromArrayView(
	ULearningAgentsActionObject* Object,
	const TArrayView<const FName> Names,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeNamedInclusiveDiscreteActionFromArrayView: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(Names))
	{
		UE_LOGF(LogLearning, Error, "MakeNamedInclusiveDiscreteActionFromArrayView: Names contain duplicates.");
		return FLearningAgentsActionObjectElement();
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nNames: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*UE::Learning::Agents::Action::Private::FormatNames(Names, 256));
	}
#endif

	return { Object->ActionObject.CreateNamedDiscreteInclusive({ Names }, Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeStructAction(ULearningAgentsActionObject* Object, const TMap<FName, FLearningAgentsActionObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeStructAction: Creating zero-sized Struct Action.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<FLearningAgentsActionObjectElement, TInlineAllocator<16>> SubElements;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	SubElements.Empty(SubElementNum);
	SubElementNames.Empty(SubElementNum);

	for (const TPair<FName, FLearningAgentsActionObjectElement>& Element : Elements)
	{
		SubElements.Add(Element.Value);
		SubElementNames.Add(Element.Key);
	}

	return MakeStructActionFromArrayViews(Object, SubElementNames, SubElements, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeStructActionFromArrays(ULearningAgentsActionObject* Object, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeStructActionFromArrayViews(Object, ElementNames, Elements, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeStructActionFromArrayViews(ULearningAgentsActionObject* Object, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionObjectElement> Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeStructActionFromArrayViews: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeStructActionFromArrayViews: Creating zero-sized Struct Action.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "MakeStructActionFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsActionObjectElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "MakeStructActionFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsActionObjectElement();
	}

	TArray<UE::Learning::Action::FObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsActionObjectElement& Element : Elements)
	{
		if (!Object->ActionObject.IsValid(Element.ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeStructActionFromArrayViews: Invalid Action Object.");
			return FLearningAgentsActionObjectElement();
		}

		SubElements.Add(Element.ObjectElement);
	}

	return { Object->ActionObject.CreateAnd({ ElementNames, SubElements }, Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeExclusiveUnionAction(ULearningAgentsActionObject* Object, const FName ElementName, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveUnionAction: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveUnionAction: Invalid Action Object.");
		return FLearningAgentsActionObjectElement();
	}

	return { Object->ActionObject.CreateOrExclusive({ ElementName, Element.ObjectElement }, Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeInclusiveUnionAction(ULearningAgentsActionObject* Object, const TMap<FName, FLearningAgentsActionObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	const int32 SubElementNum = Elements.Num();

	TArray<FLearningAgentsActionObjectElement, TInlineAllocator<16>> SubElements;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	SubElements.Empty(SubElementNum);
	SubElementNames.Empty(SubElementNum);

	for (const TPair<FName, FLearningAgentsActionObjectElement>& Element : Elements)
	{
		SubElements.Add(Element.Value);
		SubElementNames.Add(Element.Key);
	}

	return MakeInclusiveUnionActionFromArrayViews(Object, SubElementNames, SubElements, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeInclusiveUnionActionFromArrays(ULearningAgentsActionObject* Object, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeInclusiveUnionActionFromArrayViews(Object, ElementNames, Elements, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeInclusiveUnionActionFromArrayViews(ULearningAgentsActionObject* Object, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionObjectElement> Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveUnionActionFromArrayViews: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveUnionActionFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsActionObjectElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveUnionActionFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsActionObjectElement();
	}

	TArray<UE::Learning::Action::FObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsActionObjectElement& Element : Elements)
	{
		if (!Object->ActionObject.IsValid(Element.ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeInclusiveUnionActionFromArrayViews: Invalid Action Object.");
			return FLearningAgentsActionObjectElement();
		}

		SubElements.Add(Element.ObjectElement);
	}

	return { Object->ActionObject.CreateOrInclusive({ ElementNames, SubElements }, Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeStaticArrayAction(ULearningAgentsActionObject* Object, const TArray<FLearningAgentsActionObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeStaticArrayActionFromArrayView(Object, Elements, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeStaticArrayActionFromArrayView(ULearningAgentsActionObject* Object, const TArrayView<const FLearningAgentsActionObjectElement> Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeStaticArrayActionFromArrayView: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeStaticArrayActionFromArrayView: Creating zero-sized Static Array Action.");
	}

	TArray<UE::Learning::Action::FObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsActionObjectElement& Element : Elements)
	{
		if (!Object->ActionObject.IsValid(Element.ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeStaticArrayActionFromArrayView: Invalid Action Object.");
			return FLearningAgentsActionObjectElement();
		}

		SubElements.Add(Element.ObjectElement);
	}

	return { Object->ActionObject.CreateArray({ SubElements }, Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakePairAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Key, const FLearningAgentsActionObjectElement Value, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeStructActionFromArrayViews(Object, { TEXT("Key"), TEXT("Value") }, { Key, Value }, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeEnumAction(
	ULearningAgentsActionObject* Object,
	const UEnum* Enum,
	const uint8 EnumValue,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "MakeEnumAction: Enum is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	const int32 EnumValueIndex = Enum->GetIndexByValue(EnumValue);

	if (EnumValueIndex == INDEX_NONE || EnumValueIndex < 0 || EnumValueIndex >= Enum->NumEnums() - 1)
	{
		UE_LOGF(LogLearning, Error, "MakeEnumAction: EnumValue %i not valid for Enum '%ls'.", EnumValue, *Enum->GetName());
		return FLearningAgentsActionObjectElement();
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nEnum: %s\nSize: [%i]\nValue: [%s]\nIndex: [%i]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*Enum->GetName(),
			Enum->NumEnums() - 1,
			*Enum->GetDisplayNameTextByValue(EnumValue).ToString(),
			EnumValueIndex);
	}
#endif

	return MakeExclusiveDiscreteAction(Object, EnumValueIndex, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeBitmaskAction(
	ULearningAgentsActionObject* Object,
	const UEnum* Enum,
	const int32 BitmaskValue,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "MakeBitmaskAction: Enum is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_LOGF(LogLearning, Error, "MakeBitmaskAction: Too many values in Enum to use as Bitmask (%i).", Enum->NumEnums() - 1);
		return FLearningAgentsActionObjectElement();
	}

	TArray<int32, TInlineAllocator<32>> BitmaskIndices;
	BitmaskIndices.Empty(Enum->NumEnums() - 1);

	for (int32 BitmaskIdx = 0; BitmaskIdx < Enum->NumEnums() - 1; BitmaskIdx++)
	{
		if (BitmaskValue & (1 << BitmaskIdx))
		{
			BitmaskIndices.Add(BitmaskIdx);
		}
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		FString ValuesString;
		FString IndicesString;

		for (int32 EnumIdx = 0; EnumIdx < Enum->NumEnums() - 1; EnumIdx++)
		{
			if (BitmaskValue & (1 << EnumIdx))
			{
				ValuesString += Enum->GetDisplayNameTextByIndex(EnumIdx).ToString() + TEXT(" ");
				IndicesString += FString::FromInt(EnumIdx) + TEXT(" ");
			}
		}

		ValuesString = ValuesString.TrimEnd();
		IndicesString = IndicesString.TrimEnd();

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nEnum: %s\nSize: [%i]\nValues: [%s]\nIndices: [%s]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*Enum->GetName(),
			Enum->NumEnums() - 1,
			*ValuesString,
			*IndicesString);
	}
#endif

	return MakeInclusiveDiscreteActionFromArrayView(Object, BitmaskIndices, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeOptionalAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const ELearningAgentsOptionalAction Option, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeExclusiveUnionAction(Object,
		Option == ELearningAgentsOptionalAction::Null ? TEXT("Null") : TEXT("Valid"),
		Option == ELearningAgentsOptionalAction::Null ? MakeNullAction(Object) : Element,
		bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeOptionalNullAction(ULearningAgentsActionObject* Object, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeExclusiveUnionAction(Object, TEXT("Null"), MakeNullAction(Object), bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeOptionalValidAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeExclusiveUnionAction(Object, TEXT("Valid"), Element, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeEitherAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const ELearningAgentsEitherAction Either, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeExclusiveUnionAction(Object, Either == ELearningAgentsEitherAction::A ? TEXT("A") : TEXT("B"), Element, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeEitherAAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement A, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeExclusiveUnionAction(Object, TEXT("A"), A, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeEitherBAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement B, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeExclusiveUnionAction(Object, TEXT("B"), B, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeEncodingAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeEncodingAction: Object is nullptr.");
		return FLearningAgentsActionObjectElement();
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "MakeEncodingAction: Invalid Action Object.");
		return FLearningAgentsActionObjectElement();
	}

	return { Object->ActionObject.CreateEncoding({ Element.ObjectElement }, Tag) };
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeBoolAction(
	ULearningAgentsActionObject* Object,
	const bool bValue,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nValue: [%s]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			bValue ? TEXT("true") : TEXT("false"));
	}
#endif

	return MakeExclusiveDiscreteAction(Object, bValue ? 1 : 0, bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeFloatAction(
	ULearningAgentsActionObject* Object,
	const float Value,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nValue: [% 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Value);
	}
#endif

	return MakeContinuousActionFromArrayView(Object, { Value }, bActive, Tag, FLearningAgentsVisualLoggerSettings());
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeLocationAction(
	ULearningAgentsActionObject* Object,
	const FVector Location,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	const FVector LocalLocation = RelativeTransform.InverseTransformPosition(Location);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			Location,
			10,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			RelativeTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Location: [% 6.1f % 6.1f % 6.1f]\nLocation: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			LocalLocation.X, LocalLocation.Y, LocalLocation.Z,
			Location.X, Location.Y, Location.Z);
	}
#endif

	return MakeContinuousActionFromArrayView(Object, {
		(float)LocalLocation.X,
		(float)LocalLocation.Y,
		(float)LocalLocation.Z }, bActive, Tag, FLearningAgentsVisualLoggerSettings());
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeRotationAction(
	ULearningAgentsActionObject* Object,
	const FRotator Rotation,
	const FRotator RelativeRotation,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeRotationActionFromQuat(Object, FQuat::MakeFromRotator(Rotation), FQuat::MakeFromRotator(RelativeRotation), bActive, Tag,
		VisualLoggerSettings);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeRotationActionFromQuat(
	ULearningAgentsActionObject* Object,
	const FQuat Rotation,
	const FQuat RelativeRotation,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	FQuat LocalRotation = (RelativeRotation.Inverse() * Rotation).GetShortestArcWith(FQuat::Identity);
	const FVector RotationVector = LocalRotation.ToRotationVector();

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			VisualLoggerSettings.DebugLocation,
			LocalRotation.Rotator(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Rotation Vector: [% 6.1f % 6.1f % 6.1f]\nLocal Rotation: [% 6.1f % 6.1f % 6.1f % 6.1f]\nRotation: [% 6.1f % 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			RotationVector.X, RotationVector.Y, RotationVector.Z,
			LocalRotation.X, LocalRotation.Y, LocalRotation.Z, LocalRotation.W,
			Rotation.X, Rotation.Y, Rotation.Z, Rotation.W);
	}
#endif

	return MakeContinuousActionFromArrayView(Object, {
		(float)FMath::RadiansToDegrees(RotationVector.X),
		(float)FMath::RadiansToDegrees(RotationVector.Y),
		(float)FMath::RadiansToDegrees(RotationVector.Z),
		}, bActive, Tag, FLearningAgentsVisualLoggerSettings());
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeScaleAction(
	ULearningAgentsActionObject* Object,
	const FVector Scale,
	const FVector RelativeScale,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	const FVector LocalLogScale =
		UE::Learning::Agents::Action::Private::VectorLogSafe(Scale) -
		UE::Learning::Agents::Action::Private::VectorLogSafe(RelativeScale);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Log Scale: [% 6.1f % 6.1f % 6.1f]\nScale: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			LocalLogScale.X, LocalLogScale.Y, LocalLogScale.Z,
			Scale.X, Scale.Y, Scale.Z);
	}
#endif

	return MakeContinuousActionFromArrayView(Object, {
		(float)LocalLogScale.X,
		(float)LocalLogScale.Y,
		(float)LocalLogScale.Z,
		}, bActive, Tag, FLearningAgentsVisualLoggerSettings());
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeTransformAction(
	ULearningAgentsActionObject* Object,
	const FTransform Transform,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	FVector Location = Transform.GetLocation();
	FQuat Rotation = Transform.GetRotation();
	FVector Scale = Transform.GetScale3D();

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			Location,
			Rotation,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocation: [% 6.1f % 6.1f % 6.1f]\nRotation: [% 6.1f % 6.1f % 6.1f % 6.1f]\nScale: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Location.X, Location.Y, Location.Z,
			Rotation.X, Rotation.Y, Rotation.Z, Rotation.W,
			Scale.X, Scale.Y, Scale.Z);
	}
#endif

	return MakeStructActionFromArrayViews(Object,
		{
			TEXT("Location"),
			TEXT("Rotation"),
			TEXT("Scale")
		},
		{
			MakeLocationAction(Object, Location, RelativeTransform),
			MakeRotationActionFromQuat(Object, Rotation, RelativeTransform.GetRotation()),
			MakeScaleAction(Object, Scale, RelativeTransform.GetScale3D())
		},
		bActive, Tag);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeAngleAction(
	ULearningAgentsActionObject* Object,
	const float Angle,
	const float RelativeAngle,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	float LocalAngle = FMath::FindDeltaAngleDegrees(RelativeAngle, Angle);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ANGLE_DEGREES(VisualLoggerObject, LogLearning, Display,
			Angle,
			0.0f,
			VisualLoggerSettings.DebugLocation,
			10.0f,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Angle: [% 6.1f]\nAngle: [% 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			LocalAngle,
			Angle);
	}
#endif

	return MakeContinuousActionFromArrayView(Object, { LocalAngle }, bActive, Tag, FLearningAgentsVisualLoggerSettings());
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeAngleActionRadians(
	ULearningAgentsActionObject* Object,
	const float Angle,
	const float RelativeAngle,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	return MakeAngleAction(Object, FMath::RadiansToDegrees(Angle), FMath::RadiansToDegrees(RelativeAngle), bActive, Tag,
		VisualLoggerSettings);
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeVelocityAction(
	ULearningAgentsActionObject* Object,
	const FVector Velocity,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	const FVector LocalVelocity = RelativeTransform.InverseTransformVectorNoScale(Velocity);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ARROW(VisualLoggerObject, LogLearning, Display,
			VisualLoggerSettings.DebugLocation,
			VisualLoggerSettings.DebugLocation + Velocity,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			RelativeTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Velocity: [% 6.1f % 6.1f % 6.1f]\nVelocity: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			LocalVelocity.X, LocalVelocity.Y, LocalVelocity.Z,
			Velocity.X, Velocity.Y, Velocity.Z);
	}
#endif

	return MakeContinuousActionFromArrayView(Object, {
		(float)LocalVelocity.X,
		(float)LocalVelocity.Y,
		(float)LocalVelocity.Z,
		}, bActive, Tag, FLearningAgentsVisualLoggerSettings());
}

FLearningAgentsActionObjectElement ULearningAgentsActions::MakeDirectionAction(
	ULearningAgentsActionObject* Object,
	const FVector Direction,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullAction(Object, Tag);
	}

	const FVector LocalDirection = RelativeTransform.InverseTransformVectorNoScale(Direction).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ARROW(VisualLoggerObject, LogLearning, Display,
			VisualLoggerSettings.DebugLocation,
			VisualLoggerSettings.DebugLocation + VisualLoggerSettings.ArrowLength * Direction,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			RelativeTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Direction: [% 6.1f % 6.1f % 6.1f]\nDirection: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			LocalDirection.X, LocalDirection.Y, LocalDirection.Z,
			Direction.X, Direction.Y, Direction.Z);
	}
#endif

	return MakeContinuousActionFromArrayView(Object, {
		(float)LocalDirection.X,
		(float)LocalDirection.Y,
		(float)LocalDirection.Z,
		}, bActive, Tag, FLearningAgentsVisualLoggerSettings());
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeNullActionModifier(ULearningAgentsActionModifier* Modifier, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeNullActionModifier: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	return { Modifier->ActionModifier.CreateNull(Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeContinuousActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<bool>& Masked, const TArray<float>& MaskedValues, const FName Tag)
{
	return MakeContinuousActionModifierFromArrayView(Modifier, Masked, MaskedValues, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeContinuousActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const bool> Masked, const TArrayView<const float> MaskedValues, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeContinuousActionModifierFromArrayView: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (MaskedValues.Num() != Masked.Num())
	{
		UE_LOGF(LogLearning, Error, "MakeContinuousActionModifierFromArrayView: Masked and MaskedValues sizes don't match. Got %i and %i.", Masked.Num(), MaskedValues.Num());
		return FLearningAgentsActionModifierElement();
	}

	return { Modifier->ActionModifier.CreateContinuous({ Masked, MaskedValues }, Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeExclusiveDiscreteActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<int32>& MaskedIndices, const FName Tag)
{
	return MakeExclusiveDiscreteActionModifierFromArrayView(Modifier, MaskedIndices, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeExclusiveDiscreteActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const int32> MaskedIndices, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveDiscreteActionModifierFromArrayView: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(MaskedIndices))
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveDiscreteActionModifierFromArrayView: MaskedIndices contain duplicates.");
		return FLearningAgentsActionModifierElement();
	}

	return { Modifier->ActionModifier.CreateDiscreteExclusive({ MaskedIndices }, Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeNamedExclusiveDiscreteActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<FName>& MaskedNames, const FName Tag)
{
	return MakeNamedExclusiveDiscreteActionModifierFromArrayView(Modifier, MaskedNames, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeNamedExclusiveDiscreteActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const FName> MaskedNames, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeNamedExclusiveDiscreteActionModifierFromArrayView: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(MaskedNames))
	{
		UE_LOGF(LogLearning, Error, "MakeNamedExclusiveDiscreteActionModifierFromArrayView: MaskedNames contain duplicates.");
		return FLearningAgentsActionModifierElement();
	}

	return { Modifier->ActionModifier.CreateNamedDiscreteExclusive({ MaskedNames }, Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeInclusiveDiscreteActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<int32>& MaskedIndices, const FName Tag)
{
	return MakeInclusiveDiscreteActionModifierFromArrayView(Modifier, MaskedIndices, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeInclusiveDiscreteActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const int32> MaskedIndices, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveDiscreteActionModifierFromArrayView: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(MaskedIndices))
	{
		UE_LOGF(LogLearning, Error, "MakeNamedExclusiveDiscreteActionModifierFromArrayView: MaskedIndices contain duplicates.");
		return FLearningAgentsActionModifierElement();
	}

	return { Modifier->ActionModifier.CreateDiscreteInclusive({ MaskedIndices }, Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeNamedInclusiveDiscreteActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<FName>& MaskedNames, const FName Tag)
{
	return MakeNamedInclusiveDiscreteActionModifierFromArrayView(Modifier, MaskedNames, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeNamedInclusiveDiscreteActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const FName> MaskedNames, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeNamedInclusiveDiscreteActionModifierFromArrayView: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(MaskedNames))
	{
		UE_LOGF(LogLearning, Error, "MakeNamedExclusiveDiscreteActionModifierFromArrayView: MaskedNames contain duplicates.");
		return FLearningAgentsActionModifierElement();
	}

	return { Modifier->ActionModifier.CreateNamedDiscreteInclusive({ MaskedNames }, Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeStructActionModifier(ULearningAgentsActionModifier* Modifier, const TMap<FName, FLearningAgentsActionModifierElement>& Elements, const FName Tag)
{
	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeStructActionModifier: Creating zero-sized Struct Action Modifier.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<FLearningAgentsActionModifierElement, TInlineAllocator<16>> SubElements;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	SubElements.Empty(SubElementNum);
	SubElementNames.Empty(SubElementNum);

	for (const TPair<FName, FLearningAgentsActionModifierElement>& Element : Elements)
	{
		SubElements.Add(Element.Value);
		SubElementNames.Add(Element.Key);
	}

	return MakeStructActionModifierFromArrayViews(Modifier, SubElementNames, SubElements, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeStructActionModifierFromArrays(ULearningAgentsActionModifier* Modifier, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionModifierElement>& Elements, const FName Tag)
{
	return MakeStructActionModifierFromArrayViews(Modifier, ElementNames, Elements, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeStructActionModifierFromArrayViews(ULearningAgentsActionModifier* Modifier, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionModifierElement> Elements, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeStructActionModifierFromArrayViews: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeStructActionModifierFromArrayViews: Creating zero-sized Struct Action Modifier.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "MakeStructActionModifierFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsActionModifierElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "MakeStructActionModifierFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsActionModifierElement();
	}

	TArray<UE::Learning::Action::FModifierElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsActionModifierElement& Element : Elements)
	{
		if (!Modifier->ActionModifier.IsValid(Element.ModifierElement))
		{
			UE_LOGF(LogLearning, Error, "MakeStructActionModifierFromArrayViews: Invalid Action Modifier.");
			return FLearningAgentsActionModifierElement();
		}

		SubElements.Add(Element.ModifierElement);
	}

	return { Modifier->ActionModifier.CreateAnd({ ElementNames, SubElements }, Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeExclusiveUnionActionModifier(ULearningAgentsActionModifier* Modifier, const TMap<FName, FLearningAgentsActionModifierElement>& Elements, const TArray<FName>& MaskedElements, const FName Tag)
{
	const int32 SubElementNum = Elements.Num();

	TArray<FLearningAgentsActionModifierElement, TInlineAllocator<16>> SubElements;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	SubElements.Empty(SubElementNum);
	SubElementNames.Empty(SubElementNum);

	for (const TPair<FName, FLearningAgentsActionModifierElement>& Element : Elements)
	{
		SubElements.Add(Element.Value);
		SubElementNames.Add(Element.Key);
	}

	return MakeExclusiveUnionActionModifierFromArrayViews(Modifier, SubElementNames, SubElements, MaskedElements, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeExclusiveUnionActionModifierFromArrays(ULearningAgentsActionModifier* Modifier, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionModifierElement>& Elements, const TArray<FName>& MaskedElements, const FName Tag)
{
	return MakeExclusiveUnionActionModifierFromArrayViews(Modifier, ElementNames, Elements, MaskedElements, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeExclusiveUnionActionModifierFromArrayViews(ULearningAgentsActionModifier* Modifier, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionModifierElement> Elements, const TArrayView<const FName> MaskedElements, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveUnionActionModifierFromArrayViews: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveUnionActionModifierFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsActionModifierElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveUnionActionModifierFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsActionModifierElement();
	}

	TArray<UE::Learning::Action::FModifierElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsActionModifierElement& Element : Elements)
	{
		if (!Modifier->ActionModifier.IsValid(Element.ModifierElement))
		{
			UE_LOGF(LogLearning, Error, "MakeExclusiveUnionActionModifierFromArrayViews: Invalid Action Modifier.");
			return FLearningAgentsActionModifierElement();
		}

		SubElements.Add(Element.ModifierElement);
	}

	return { Modifier->ActionModifier.CreateOrExclusive({ ElementNames, SubElements, MaskedElements }, Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeInclusiveUnionActionModifier(ULearningAgentsActionModifier* Modifier, const TMap<FName, FLearningAgentsActionModifierElement>& Elements, const TArray<FName>& MaskedElements, const FName Tag)
{
	const int32 SubElementNum = Elements.Num();

	TArray<FLearningAgentsActionModifierElement, TInlineAllocator<16>> SubElements;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	SubElements.Empty(SubElementNum);
	SubElementNames.Empty(SubElementNum);

	for (const TPair<FName, FLearningAgentsActionModifierElement>& Element : Elements)
	{
		SubElements.Add(Element.Value);
		SubElementNames.Add(Element.Key);
	}

	return MakeInclusiveUnionActionModifierFromArrayViews(Modifier, SubElementNames, SubElements, MaskedElements, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeInclusiveUnionActionModifierFromArrays(ULearningAgentsActionModifier* Modifier, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionModifierElement>& Elements, const TArray<FName>& MaskedElements, const FName Tag)
{
	return MakeInclusiveUnionActionModifierFromArrayViews(Modifier, ElementNames, Elements, MaskedElements, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeInclusiveUnionActionModifierFromArrayViews(ULearningAgentsActionModifier* Modifier, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionModifierElement> Elements, const TArrayView<const FName> MaskedElements, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveUnionActionModifierFromArrayViews: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveUnionActionModifierFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsActionModifierElement();
	}

	if (UE::Learning::Agents::Action::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveUnionActionModifierFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsActionModifierElement();
	}

	TArray<UE::Learning::Action::FModifierElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsActionModifierElement& Element : Elements)
	{
		if (!Modifier->ActionModifier.IsValid(Element.ModifierElement))
		{
			UE_LOGF(LogLearning, Error, "MakeInclusiveUnionActionModifierFromArrayViews: Invalid Action Modifier.");
			return FLearningAgentsActionModifierElement();
		}

		SubElements.Add(Element.ModifierElement);
	}

	return { Modifier->ActionModifier.CreateOrInclusive({ ElementNames, SubElements, MaskedElements }, Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeStaticArrayActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<FLearningAgentsActionModifierElement>& Elements, const FName Tag)
{
	return MakeStaticArrayActionModifierFromArrayView(Modifier, Elements, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeStaticArrayActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const FLearningAgentsActionModifierElement> Elements, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeStaticArrayActionModifierFromArrayView: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeStaticArrayActionModifierFromArrayView: Creating zero-sized Static Array Action.");
	}

	TArray<UE::Learning::Action::FModifierElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsActionModifierElement& Element : Elements)
	{
		if (!Modifier->ActionModifier.IsValid(Element.ModifierElement))
		{
			UE_LOGF(LogLearning, Error, "MakeStaticArrayActionModifierFromArrayView: Invalid Action Modifier.");
			return FLearningAgentsActionModifierElement();
		}

		SubElements.Add(Element.ModifierElement);
	}

	return { Modifier->ActionModifier.CreateArray({ SubElements }, Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakePairActionModifier(ULearningAgentsActionModifier* Modifier, const FLearningAgentsActionModifierElement Key, const FLearningAgentsActionModifierElement Value, const FName Tag)
{
	return MakeStructActionModifierFromArrayViews(Modifier, { TEXT("Key"), TEXT("Value") }, { Key, Value }, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeEnumActionModifier(ULearningAgentsActionModifier* Modifier, const UEnum* Enum, const TArray<uint8>& EnumMaskedValues, const FName Tag)
{
	return MakeEnumActionModifierFromArrayView(Modifier, Enum, EnumMaskedValues, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeEnumActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const UEnum* Enum, const TArrayView<const uint8> EnumMaskedValues, const FName Tag)
{
	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "MakeEnumActionModifierFromArrayView: Enum is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	TArray<int32, TInlineAllocator<32>> EnumValueIndices;

	for (const uint8 EnumValue : EnumMaskedValues)
	{
		const int32 EnumValueIndex = Enum->GetIndexByValue(EnumValue);

		if (EnumValueIndex == INDEX_NONE || EnumValueIndex < 0 || EnumValueIndex >= Enum->NumEnums() - 1)
		{
			UE_LOGF(LogLearning, Error, "MakeEnumActionModifierFromArrayView: EnumValue %i not valid for Enum '%ls'.", EnumValue, *Enum->GetName());
			return FLearningAgentsActionModifierElement();
		}
	}

	return MakeExclusiveDiscreteActionModifierFromArrayView(Modifier, EnumValueIndices, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeBitmaskActionModifier(ULearningAgentsActionModifier* Modifier, const UEnum* Enum, const int32 MaskedBitmask, const FName Tag)
{
	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "MakeBitmaskActionModifier: Enum is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_LOGF(LogLearning, Error, "MakeBitmaskActionModifier: Too many values in Enum to use as Bitmask (%i).", Enum->NumEnums() - 1);
		return FLearningAgentsActionModifierElement();
	}

	TArray<int32, TInlineAllocator<32>> BitmaskIndices;
	BitmaskIndices.Empty(Enum->NumEnums() - 1);

	for (int32 BitmaskIdx = 0; BitmaskIdx < Enum->NumEnums() - 1; BitmaskIdx++)
	{
		if (MaskedBitmask & (1 << BitmaskIdx))
		{
			BitmaskIndices.Add(BitmaskIdx);
		}
	}

	return MakeInclusiveDiscreteActionModifierFromArrayView(Modifier, BitmaskIndices, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeOptionalActionModifier(ULearningAgentsActionModifier* Modifier, const FLearningAgentsActionModifierElement Element, const bool bAllowOnlyValid, const bool bAllowOnlyNull, const FName Tag)
{
	if (bAllowOnlyValid && bAllowOnlyNull)
	{
		UE_LOGF(LogLearning, Warning, "MakeOptionalActionModifier: Must either set bAllowOnlyValid or bAllowOnlyNull.");
	}

	if (bAllowOnlyValid)
	{
		return MakeExclusiveUnionActionModifierFromArrayViews(Modifier, { TEXT("Valid") }, { Element }, { TEXT("Null") }, Tag);
	}

	if (bAllowOnlyNull)
	{
		return MakeExclusiveUnionActionModifierFromArrayViews(Modifier, { TEXT("Valid") }, { Element }, { TEXT("Valid") }, Tag);
	}

	return MakeExclusiveUnionActionModifierFromArrayViews(Modifier, { TEXT("Valid") }, { Element }, {}, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeEitherActionModifier(ULearningAgentsActionModifier* Modifier, const FLearningAgentsActionModifierElement A, const FLearningAgentsActionModifierElement B, const bool bAllowOnlyA, const bool bAllowOnlyB, const FName Tag)
{
	if (bAllowOnlyA && bAllowOnlyB)
	{
		UE_LOGF(LogLearning, Warning, "MakeEitherActionModifier: Must either set bAllowOnlyA or bAllowOnlyB.");
	}

	if (bAllowOnlyA)
	{
		return MakeExclusiveUnionActionModifierFromArrayViews(Modifier, { TEXT("A"), TEXT("B") }, { A, B }, { TEXT("B") }, Tag);
	}

	if (bAllowOnlyB)
	{
		return MakeExclusiveUnionActionModifierFromArrayViews(Modifier, { TEXT("A"), TEXT("B") }, { A, B }, { TEXT("A") }, Tag);
	}

	return MakeExclusiveUnionActionModifierFromArrayViews(Modifier, { TEXT("A"), TEXT("B") }, { A, B }, {}, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeEncodingActionModifier(ULearningAgentsActionModifier* Modifier, const FLearningAgentsActionModifierElement Element, const FName Tag)
{
	if (!Modifier)
	{
		UE_LOGF(LogLearning, Error, "MakeEncodingActionModifier: Modifier is nullptr.");
		return FLearningAgentsActionModifierElement();
	}

	if (!Modifier->ActionModifier.IsValid(Element.ModifierElement))
	{
		UE_LOGF(LogLearning, Error, "MakeEncodingActionModifier: Invalid Action Modifier.");
		return FLearningAgentsActionModifierElement();
	}

	return { Modifier->ActionModifier.CreateEncoding({ Element.ModifierElement }, Tag) };
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeBoolActionModifier(ULearningAgentsActionModifier* Modifier, const bool bValue, const FName Tag)
{
	return MakeExclusiveDiscreteActionModifierFromArrayView(Modifier, { bValue ? 0 : 1 }, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeFloatActionModifier(ULearningAgentsActionModifier* Modifier, const float MaskedValue, const bool bMasked, const FName Tag)
{
	return MakeContinuousActionModifierFromArrayView(Modifier, { bMasked }, { MaskedValue }, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeLocationActionModifier(ULearningAgentsActionModifier* Modifier, const FVector MaskedLocation, const bool bMaskedX, const bool bMaskedY, const bool bMaskedZ, const FTransform RelativeTransform, const FName Tag)
{
	const FVector LocalLocation = RelativeTransform.InverseTransformPosition(MaskedLocation);

	return MakeContinuousActionModifierFromArrayView(Modifier, {
		bMaskedX,
		bMaskedY,
		bMaskedZ
		}, {
		(float)LocalLocation.X,
		(float)LocalLocation.Y,
		(float)LocalLocation.Z }, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeScaleActionModifier(ULearningAgentsActionModifier* Modifier, const FVector MaskedScale, const bool bMaskedX, const bool bMaskedY, const bool bMaskedZ, const FVector RelativeScale, const FName Tag)
{
	const FVector LocalLogScale =
		UE::Learning::Agents::Action::Private::VectorLogSafe(MaskedScale) -
		UE::Learning::Agents::Action::Private::VectorLogSafe(RelativeScale);

	return MakeContinuousActionModifierFromArrayView(Modifier, {
		bMaskedX,
		bMaskedY,
		bMaskedZ
		}, {
		(float)LocalLogScale.X,
		(float)LocalLogScale.Y,
		(float)LocalLogScale.Z }, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeAngleActionModifier(ULearningAgentsActionModifier* Modifier, const float MaskedAngle, const bool bMask, const float RelativeAngle, const FName Tag)
{
	return MakeContinuousActionModifierFromArrayView(Modifier, { bMask }, { FMath::FindDeltaAngleDegrees(RelativeAngle, MaskedAngle) }, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeAngleActionModifierRadians(ULearningAgentsActionModifier* Modifier, const float MaskedAngle, const bool bMask, const float RelativeAngle, const FName Tag)
{
	return MakeAngleActionModifier(Modifier, FMath::RadiansToDegrees(MaskedAngle), bMask, FMath::RadiansToDegrees(RelativeAngle), Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeVelocityActionModifier(ULearningAgentsActionModifier* Modifier, const FVector MaskedVelocity, const bool bMaskedX, const bool bMaskedY, const bool bMaskedZ, const FTransform RelativeTransform, const FName Tag)
{
	const FVector LocalVelocity = RelativeTransform.InverseTransformVectorNoScale(MaskedVelocity);

	return MakeContinuousActionModifierFromArrayView(Modifier, {
		bMaskedX,
		bMaskedY,
		bMaskedZ }, {
		(float)LocalVelocity.X,
		(float)LocalVelocity.Y,
		(float)LocalVelocity.Z,
		}, Tag);
}

FLearningAgentsActionModifierElement ULearningAgentsActions::MakeDirectionActionModifier(ULearningAgentsActionModifier* Modifier, const FVector MaskedDirection, const bool bMaskedX, const bool bMaskedY, const bool bMaskedZ, const FTransform RelativeTransform, const FName Tag)
{
	const FVector LocalDirection = RelativeTransform.InverseTransformVectorNoScale(MaskedDirection).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);

	return MakeContinuousActionModifierFromArrayView(Modifier, {
		bMaskedX,
		bMaskedY,
		bMaskedZ }, {
		(float)LocalDirection.X,
		(float)LocalDirection.Y,
		(float)LocalDirection.Z,
		}, Tag);
}

bool ULearningAgentsActions::GetNullAction(const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const FName Tag)
{
	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetNullAction: Object is nullptr.");
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetNullAction: Invalid Action Object.");
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetNullAction: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::Null)
	{
		UE_LOGF(LogLearning, Error, "GetNullAction: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::Null));
		return false;
	}

	return true;
}

bool ULearningAgentsActions::GetContinuousActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetContinuousActionNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetContinuousActionNum: Invalid Action Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetContinuousActionNum: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::Continuous)
	{
		UE_LOGF(LogLearning, Error, "GetContinuousActionNum: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::Continuous));
		OutNum = 0;
		return false;
	}

	OutNum = Object->ActionObject.GetContinuous(Element.ObjectElement).Values.Num();
	return true;
}

bool ULearningAgentsActions::GetContinuousAction(
	TArray<float>& OutValues,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	int32 OutValueNum = 0;
	if (!GetContinuousActionNum(OutValueNum, Object, Element, bActive, Tag))
	{
		OutValues.Empty();
		return false;
	}

	OutValues.SetNumUninitialized(OutValueNum);

	if (!GetContinuousActionToArrayView(OutValues, Object, Element, bActive, Tag, VisualLoggerSettings))
	{
		OutValues.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsActions::GetContinuousActionToArrayView(
	TArrayView<float> OutValues,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetContinuousActionToArrayView: Object is nullptr.");
		UE::Learning::Array::Zero<1, float>(OutValues);
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetContinuousActionToArrayView: Invalid Action Object.");
		UE::Learning::Array::Zero<1, float>(OutValues);
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetContinuousActionToArrayView: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::Continuous)
	{
		UE_LOGF(LogLearning, Error, "GetContinuousActionToArrayView: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::Continuous));
		UE::Learning::Array::Zero<1, float>(OutValues);
		return false;
	}

	const TArrayView<const float> Values = Object->ActionObject.GetContinuous(Element.ObjectElement).Values;

	if (Values.Num() != OutValues.Num())
	{
		UE_LOGF(LogLearning, Error, "GetContinuousActionToArrayView: Action '%ls' size does not match. Action is '%i' values but asked for '%i'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			Values.Num(), OutValues.Num());
		UE::Learning::Array::Zero<1, float>(OutValues);
		return false;
	}

	UE::Learning::Array::Copy<1, float>(OutValues, Values);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nEncoded: %s\nValues: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*UE::Learning::Array::FormatFloat(OutValues),
			*UE::Learning::Array::FormatFloat(OutValues)); // Encoded is identical to provided values
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetExclusiveDiscreteAction(
	int32& OutIndex,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetExclusiveDiscreteAction: Object is nullptr.");
		OutIndex = 0;
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetExclusiveDiscreteAction: Invalid Action Object.");
		OutIndex = 0;
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetExclusiveDiscreteAction: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::DiscreteExclusive)
	{
		UE_LOGF(LogLearning, Error, "GetExclusiveDiscreteAction: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::DiscreteExclusive));
		OutIndex = 0;
		return false;
	}

	OutIndex = Object->ActionObject.GetDiscreteExclusive(Element.ObjectElement).DiscreteIndex;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nIndex: [%i]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			OutIndex);
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetNamedExclusiveDiscreteAction(
	FName& OutName,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetNamedExclusiveDiscreteAction: Object is nullptr.");
		OutName = NAME_None;
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetNamedExclusiveDiscreteAction: Invalid Action Object.");
		OutName = NAME_None;
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetNamedExclusiveDiscreteAction: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::NamedDiscreteExclusive)
	{
		UE_LOGF(LogLearning, Error, "GetNamedExclusiveDiscreteAction: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::NamedDiscreteExclusive));
		OutName = NAME_None;
		return false;
	}

	OutName = Object->ActionObject.GetNamedDiscreteExclusive(Element.ObjectElement).ElementName;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nName: [%s]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*OutName.ToString());
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetInclusiveDiscreteActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteActionNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteActionNum: Invalid Action Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetInclusiveDiscreteActionNum: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::DiscreteInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteActionNum: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::DiscreteInclusive));
		OutNum = 0;
		return false;
	}

	OutNum = Object->ActionObject.GetDiscreteInclusive(Element.ObjectElement).DiscreteIndices.Num();
	return true;
}

bool ULearningAgentsActions::GetInclusiveDiscreteAction(
	TArray<int32>& OutIndices,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	int32 OutIndexNum = 0;
	if (!GetInclusiveDiscreteActionNum(OutIndexNum, Object, Element, bActive, Tag))
	{
		OutIndices.Empty();
		return false;
	}

	OutIndices.SetNumUninitialized(OutIndexNum);

	if (!GetInclusiveDiscreteActionToArrayView(OutIndices, Object, Element, bActive, Tag, VisualLoggerSettings))
	{
		OutIndices.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsActions::GetInclusiveDiscreteActionToArrayView(
	TArrayView<int32> OutIndices,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteActionToArrayView: Object is nullptr.");
		UE::Learning::Array::Zero<1, int32>(OutIndices);
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteActionToArrayView: Invalid Action Object.");
		UE::Learning::Array::Zero<1, int32>(OutIndices);
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetInclusiveDiscreteActionToArrayView: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::DiscreteInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteActionToArrayView: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::DiscreteInclusive));
		UE::Learning::Array::Zero<1, int32>(OutIndices);
		return false;
	}

	const TArrayView<const int32> Indices = Object->ActionObject.GetDiscreteInclusive(Element.ObjectElement).DiscreteIndices;

	if (Indices.Num() != OutIndices.Num())
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteActionToArrayView: Action '%ls' size does not match. Action is '%i' elements but asked for '%i'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			Indices.Num(), OutIndices.Num());
		UE::Learning::Array::Zero<1, int32>(OutIndices);
		return false;
	}

	UE::Learning::Array::Copy<1, int32>(OutIndices, Indices);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nIndices: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*UE::Learning::Array::FormatInt32(OutIndices, 256));
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetNamedInclusiveDiscreteActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteActionNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteActionNum: Invalid Action Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetNamedInclusiveDiscreteActionNum: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::NamedDiscreteInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteActionNum: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::NamedDiscreteInclusive));
		OutNum = 0;
		return false;
	}

	OutNum = Object->ActionObject.GetNamedDiscreteInclusive(Element.ObjectElement).ElementNames.Num();
	return true;
}

bool ULearningAgentsActions::GetNamedInclusiveDiscreteAction(
	TArray<FName>& OutNames,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	int32 OutNameNum = 0;
	if (!GetNamedInclusiveDiscreteActionNum(OutNameNum, Object, Element, bActive, Tag))
	{
		OutNames.Empty();
		return false;
	}

	OutNames.SetNumUninitialized(OutNameNum);

	if (!GetNamedInclusiveDiscreteActionToArrayView(OutNames, Object, Element, bActive, Tag, VisualLoggerSettings))
	{
		OutNames.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsActions::GetNamedInclusiveDiscreteActionToArrayView(
	TArrayView<FName> OutNames,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteActionToArrayView: Object is nullptr.");
		UE::Learning::Array::Set<1, FName>(OutNames, NAME_None);
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteActionToArrayView: Invalid Action Object.");
		UE::Learning::Array::Set<1, FName>(OutNames, NAME_None);
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetNamedInclusiveDiscreteActionToArrayView: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::NamedDiscreteInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteActionToArrayView: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::NamedDiscreteInclusive));
		UE::Learning::Array::Set<1, FName>(OutNames, NAME_None);
		return false;
	}

	const TArrayView<const FName> Names = Object->ActionObject.GetNamedDiscreteInclusive(Element.ObjectElement).ElementNames;

	if (Names.Num() != OutNames.Num())
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteActionToArrayView: Action '%ls' size does not match. Action is '%i' elements but asked for '%i'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			Names.Num(), OutNames.Num());
		UE::Learning::Array::Set<1, FName>(OutNames, NAME_None);
		return false;
	}

	UE::Learning::Array::Copy<1, FName>(OutNames, Names);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nNames: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*UE::Learning::Agents::Action::Private::FormatNames(OutNames, 256));
	}
#endif

	return true;
}


bool ULearningAgentsActions::GetStructActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStructActionNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStructActionNum: Invalid Action Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStructActionNum: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::And)
	{
		UE_LOGF(LogLearning, Error, "GetStructActionNum: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::And));
		OutNum = 0;
		return false;
	}

	const UE::Learning::Action::FObjectAndParameters Parameters = Object->ActionObject.GetAnd(Element.ObjectElement);

	OutNum = Parameters.Elements.Num();
	return true;
}

bool ULearningAgentsActions::GetStructAction(TMap<FName, FLearningAgentsActionObjectElement>& OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetStructActionNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsActionObjectElement, TInlineAllocator<16>> SubElements;
	SubElementNames.SetNumUninitialized(OutElementNum);
	SubElements.SetNumUninitialized(OutElementNum);

	if (!GetStructActionToArrayViews(SubElementNames, SubElements, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.Empty(OutElementNum);
	for (int32 ElementIdx = 0; ElementIdx < OutElementNum; ElementIdx++)
	{
		OutElements.Add(SubElementNames[ElementIdx], SubElements[ElementIdx]);
	}

	return true;
}

bool ULearningAgentsActions::GetStructActionElement(FLearningAgentsActionObjectElement& OutElement, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const FName ElementName, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStructActionElement: Object is nullptr.");
		OutElement = FLearningAgentsActionObjectElement();
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStructActionElement: Invalid Action Object.");
		OutElement = FLearningAgentsActionObjectElement();
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStructActionElement: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::And)
	{
		UE_LOGF(LogLearning, Error, "GetStructActionElement: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::And));
		OutElement = FLearningAgentsActionObjectElement();
		return false;
	}

	const UE::Learning::Action::FObjectAndParameters Parameters = Object->ActionObject.GetAnd(Element.ObjectElement);

	const int32 ElementIdx = Parameters.ElementNames.Find(ElementName);

	if (ElementIdx == INDEX_NONE)
	{
		UE_LOGF(LogLearning, Error, "GetStructActionElement: Element '%ls' not found.", *ElementName.ToString());
		OutElement = FLearningAgentsActionObjectElement();
		return false;
	}

	OutElement = { Parameters.Elements[ElementIdx] };
	return true;
}

bool ULearningAgentsActions::GetStructActionToArrays(TArray<FName>& OutElementNames, TArray<FLearningAgentsActionObjectElement>& OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetStructActionNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	OutElementNames.SetNumUninitialized(OutElementNum);
	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetStructActionToArrayViews(OutElementNames, OutElements, Object, Element, bActive, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsActions::GetStructActionToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FLearningAgentsActionObjectElement> OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStructActionToArrayViews: Object is nullptr.");
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStructActionToArrayViews: Invalid Action Object.");
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStructActionToArrayViews: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::And)
	{
		UE_LOGF(LogLearning, Error, "GetStructActionToArrayViews: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::And));
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	const UE::Learning::Action::FObjectAndParameters Parameters = Object->ActionObject.GetAnd(Element.ObjectElement);

	if (Parameters.Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "GetStructActionToArrayViews: Getting zero-sized And Action.");
	}

	if (Parameters.Elements.Num() != OutElements.Num())
	{
		UE_LOGF(LogLearning, Error, "GetStructActionToArrayViews: Action '%ls' size does not match. Action is '%i' elements but asked for '%i'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			Parameters.Elements.Num(), OutElements.Num());
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
	{
		if (!Object->ActionObject.IsValid(Parameters.Elements[ElementIdx]))
		{
			UE_LOGF(LogLearning, Error, "GetStructActionToArrayViews: Invalid Action Object.");
			UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
			UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
			return false;
		}

		OutElementNames[ElementIdx] = Parameters.ElementNames[ElementIdx];
		OutElements[ElementIdx] = { Parameters.Elements[ElementIdx] };
	}

	return true;
}

bool ULearningAgentsActions::GetExclusiveUnionAction(FName& OutElementName, FLearningAgentsActionObjectElement& OutElement, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetExclusiveUnionAction: Object is nullptr.");
		OutElementName = NAME_None;
		OutElement = FLearningAgentsActionObjectElement();
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetExclusiveUnionAction: Invalid Action Object.");
		OutElementName = NAME_None;
		OutElement = FLearningAgentsActionObjectElement();
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetExclusiveUnionAction: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::OrExclusive)
	{
		UE_LOGF(LogLearning, Error, "GetExclusiveUnionAction: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::OrExclusive));
		OutElementName = NAME_None;
		OutElement = FLearningAgentsActionObjectElement();
		return false;
	}

	const UE::Learning::Action::FObjectOrExclusiveParameters Parameters = Object->ActionObject.GetOrExclusive(Element.ObjectElement);

	OutElementName = Parameters.ElementName;
	OutElement = { Parameters.Element };
	return true;
}

bool ULearningAgentsActions::GetInclusiveUnionActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionActionNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionActionNum: Invalid Action Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetInclusiveUnionActionNum: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::OrInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionActionNum: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::OrInclusive));
		OutNum = 0;
		return false;
	}

	const UE::Learning::Action::FObjectOrInclusiveParameters Parameters = Object->ActionObject.GetOrInclusive(Element.ObjectElement);

	OutNum = Parameters.Elements.Num();
	return true;
}

bool ULearningAgentsActions::GetInclusiveUnionAction(TMap<FName, FLearningAgentsActionObjectElement>& OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetInclusiveUnionActionNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsActionObjectElement, TInlineAllocator<16>> SubElements;
	SubElementNames.SetNumUninitialized(OutElementNum);
	SubElements.SetNumUninitialized(OutElementNum);

	if (!GetInclusiveUnionActionToArrayViews(SubElementNames, SubElements, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.Empty(OutElementNum);
	for (int32 ElementIdx = 0; ElementIdx < OutElementNum; ElementIdx++)
	{
		OutElements.Add(SubElementNames[ElementIdx], SubElements[ElementIdx]);
	}

	return true;
}

bool ULearningAgentsActions::GetInclusiveUnionActionToArrays(TArray<FName>& OutElementNames, TArray<FLearningAgentsActionObjectElement>& OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetInclusiveUnionActionNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	OutElementNames.SetNumUninitialized(OutElementNum);
	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetInclusiveUnionActionToArrayViews(OutElementNames, OutElements, Object, Element, bActive, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsActions::GetInclusiveUnionActionToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FLearningAgentsActionObjectElement> OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionActionToArrayViews: Object is nullptr.");
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionActionToArrayViews: Invalid Action Object.");
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetInclusiveUnionActionToArrayViews: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::OrInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionActionToArrayViews: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::OrInclusive));
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	const UE::Learning::Action::FObjectOrInclusiveParameters Parameters = Object->ActionObject.GetOrInclusive(Element.ObjectElement);

	if (Parameters.Elements.Num() != OutElements.Num())
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionActionToArrayViews: Action '%ls' size does not match. Action is '%i' elements but asked for '%i'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			Parameters.Elements.Num(), OutElements.Num());
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
	{
		if (!Object->ActionObject.IsValid(Parameters.Elements[ElementIdx]))
		{
			UE_LOGF(LogLearning, Error, "GetInclusiveUnionActionToArrayViews: Invalid Action Object.");
			UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
			UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
			return false;
		}

		OutElementNames[ElementIdx] = Parameters.ElementNames[ElementIdx];
		OutElements[ElementIdx] = { Parameters.Elements[ElementIdx] };
	}

	return true;
}

bool ULearningAgentsActions::GetStaticArrayActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayActionNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayActionNum: Invalid Action Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStaticArrayActionNum: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::Array)
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayActionNum: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::Array));
		OutNum = 0;
		return false;
	}

	OutNum = Object->ActionObject.GetArray(Element.ObjectElement).Elements.Num();
	return true;
}

bool ULearningAgentsActions::GetStaticArrayAction(TArray<FLearningAgentsActionObjectElement>& OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetStaticArrayActionNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetStaticArrayActionToArrayView(OutElements, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsActions::GetStaticArrayActionToArrayView(TArrayView<FLearningAgentsActionObjectElement> OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayActionToArrayView: Object is nullptr.");
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayActionToArrayView: Invalid Action Object.");
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStaticArrayActionToArrayView: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::Array)
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayActionToArrayView: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::Array));
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	const TArrayView<const UE::Learning::Action::FObjectElement> SubElements = Object->ActionObject.GetArray(Element.ObjectElement).Elements;

	if (SubElements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "GetStaticArrayActionToArrayView: Getting zero-sized Array Action.");
	}

	if (SubElements.Num() != OutElements.Num())
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayActionToArrayView: Action '%ls' size does not match. Action is '%i' elements but asked for '%i'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			SubElements.Num(), OutElements.Num());
		UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
		return false;
	}

	for (int32 ElementIdx = 0; ElementIdx < SubElements.Num(); ElementIdx++)
	{
		if (!Object->ActionObject.IsValid(SubElements[ElementIdx]))
		{
			UE_LOGF(LogLearning, Error, "GetStaticArrayActionToArrayView: Invalid Action Object.");
			UE::Learning::Array::Set<1, FLearningAgentsActionObjectElement>(OutElements, FLearningAgentsActionObjectElement());
			return false;
		}

		OutElements[ElementIdx] = { SubElements[ElementIdx] };
	}

	return true;
}

bool ULearningAgentsActions::GetPairAction(FLearningAgentsActionObjectElement& OutKey, FLearningAgentsActionObjectElement& OutValue, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	TStaticArray<FName, 2> OutElementNames;
	TStaticArray<FLearningAgentsActionObjectElement, 2> OutElements;
	if (!GetStructActionToArrayViews(OutElementNames, OutElements, Object, Element, bActive, Tag))
	{
		OutKey = FLearningAgentsActionObjectElement();
		OutValue = FLearningAgentsActionObjectElement();
		return false;
	}

	OutKey = OutElements[MakeArrayView(OutElementNames).Find(TEXT("Key"))];
	OutValue = OutElements[MakeArrayView(OutElementNames).Find(TEXT("Value"))];
	return true;
}

bool ULearningAgentsActions::GetEnumAction(
	uint8& OutEnumValue,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const UEnum* Enum,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "GetEnumAction: Enum is nullptr.");
		OutEnumValue = 0;
		return false;
	}

	int32 OutIndex = 0;
	if (!GetExclusiveDiscreteAction(OutIndex, Object, Element, bActive, Tag))
	{
		OutEnumValue = 0;
		return false;
	}

	if (OutIndex >= Enum->NumEnums() - 1)
	{
		UE_LOGF(LogLearning, Error, "GetEnumAction: EnumValue out of range for Enum '%ls'. Expected %i or less, got %i.", *Enum->GetName(), Enum->NumEnums() - 1, OutIndex);
		OutEnumValue = 0;
		return false;
	}

	const int32 EnumValue = Enum->GetValueByIndex(OutIndex);

	if (EnumValue == INDEX_NONE)
	{
		UE_LOGF(LogLearning, Error, "GetEnumAction: Enum Value not found for index %i.", OutIndex);
		OutEnumValue = 0;
		return false;
	}

	OutEnumValue = (uint8)EnumValue;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nEnum: %s\nSize: [%i]\nValue: [%s]\nIndex: [%i]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*Enum->GetName(),
			Enum->NumEnums() - 1,
			*Enum->GetDisplayNameTextByValue(OutEnumValue).ToString(),
			OutIndex);
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetBitmaskAction(
	int32& OutBitmaskValue,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const UEnum* Enum,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "GetBitmaskAction: Enum is nullptr.");
		OutBitmaskValue = 0;
		return false;
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_LOGF(LogLearning, Error, "GetBitmaskAction: Too many values in Enum to use as Bitmask (%i).", Enum->NumEnums() - 1);
		OutBitmaskValue = 0;
		return false;
	}

	int32 EnumValueNum;
	if (!GetInclusiveDiscreteActionNum(EnumValueNum, Object, Element, bActive, Tag))
	{
		OutBitmaskValue = 0;
		return false;
	}

	if (EnumValueNum > Enum->NumEnums() - 1)
	{
		UE_LOGF(LogLearning, Error, "GetBitmaskAction: Too many values for Enum '%ls'. Expected %i or less, got %i.", *Enum->GetName(), Enum->NumEnums() - 1, EnumValueNum);
		OutBitmaskValue = 0;
		return false;
	}

	TArray<int32, TInlineAllocator<32>> OutIndices;
	OutIndices.SetNumUninitialized(EnumValueNum);
	if (!GetInclusiveDiscreteActionToArrayView(OutIndices, Object, Element, bActive, Tag))
	{
		OutBitmaskValue = 0;
		return false;
	}

	OutBitmaskValue = 0;
	for (const int32 OutIndex : OutIndices)
	{
		OutBitmaskValue |= (1 << OutIndex);
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		FString ValuesString;
		FString IndicesString;

		for (int32 EnumIdx = 0; EnumIdx < Enum->NumEnums() - 1; EnumIdx++)
		{
			if (OutBitmaskValue & (1 << EnumIdx))
			{
				ValuesString += Enum->GetDisplayNameTextByIndex(EnumIdx).ToString() + TEXT(" ");
				IndicesString += FString::FromInt(EnumIdx) + TEXT(" ");
			}
		}

		ValuesString = ValuesString.TrimEnd();
		IndicesString = IndicesString.TrimEnd();

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nEnum: %s\nSize: [%i]\nValues: [%s]\nIndices: [%s]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*Enum->GetName(),
			Enum->NumEnums() - 1,
			*ValuesString,
			*IndicesString);
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetOptionalAction(ELearningAgentsOptionalAction& OutOption, FLearningAgentsActionObjectElement& OutElement, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	FName OutName = NAME_None;
	if (!GetExclusiveUnionAction(OutName, OutElement, Object, Element, bActive, Tag))
	{
		OutOption = ELearningAgentsOptionalAction::Null;
		return false;
	}

	OutOption = OutName == TEXT("Null") ? ELearningAgentsOptionalAction::Null : ELearningAgentsOptionalAction::Valid;
	return true;
}

bool ULearningAgentsActions::GetEitherAction(ELearningAgentsEitherAction& OutEither, FLearningAgentsActionObjectElement& OutElement, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	FName OutName = NAME_None;
	if (!GetExclusiveUnionAction(OutName, OutElement, Object, Element, bActive, Tag))
	{
		OutEither = ELearningAgentsEitherAction::A;
		return false;
	}

	OutEither = OutName == TEXT("A") ? ELearningAgentsEitherAction::A : ELearningAgentsEitherAction::B;
	return true;
}

bool ULearningAgentsActions::GetEncodingAction(FLearningAgentsActionObjectElement& OutElement, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetEncodingAction: Object is nullptr.");
		OutElement = FLearningAgentsActionObjectElement();
		return false;
	}

	if (!Object->ActionObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetEncodingAction: Invalid Action Object.");
		OutElement = FLearningAgentsActionObjectElement();
		return false;
	}

	if (Object->ActionObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetEncodingAction: Action tag does not match. Action is '%ls' but asked for '%ls'.", *Object->ActionObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ActionObject.GetType(Element.ObjectElement) != UE::Learning::Action::EType::Encoding)
	{
		UE_LOGF(LogLearning, Error, "GetEncodingAction: Action '%ls' type does not match. Action is '%ls' but asked for '%ls'.",
			*Object->ActionObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Action::Private::GetActionTypeString(Object->ActionObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Action::Private::GetActionTypeString(UE::Learning::Action::EType::Encoding));
		OutElement = FLearningAgentsActionObjectElement();
		return false;
	}

	OutElement = { Object->ActionObject.GetEncoding(Element.ObjectElement).Element };

	return true;
}

bool ULearningAgentsActions::GetBoolAction(
	bool& bOutValue,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	int32 OutIndex = 0;
	if (!GetExclusiveDiscreteAction(OutIndex, Object, Element, bActive, Tag))
	{
		bOutValue = false;
		return false;
	}

	bOutValue = OutIndex == 1;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nValue: [%s]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			bOutValue ? TEXT("true") : TEXT("false"));
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetFloatAction(
	float& OutValue,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!GetContinuousActionToArrayView(MakeArrayView(&OutValue, 1), Object, Element, bActive, Tag))
	{
		OutValue = 0.0f;
		return false;
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nValue: [% 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			OutValue);
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetLocationAction(
	FVector& OutLocation,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	TStaticArray<float, 3> OutValues;
	if (!GetContinuousActionToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutLocation = FVector::ZeroVector;
		return false;
	}

	const FVector LocalLocation = FVector(OutValues[0], OutValues[1], OutValues[2]);
	OutLocation = RelativeTransform.TransformPosition(LocalLocation);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			OutLocation,
			10,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			OutLocation,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			RelativeTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Location: [% 6.1f % 6.1f % 6.1f]\nLocation: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			LocalLocation.X, LocalLocation.Y, LocalLocation.Z,
			OutLocation.X, OutLocation.Y, OutLocation.Z);
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetRotationAction(
	FRotator& OutRotation,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const FRotator RelativeRotation,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	FQuat OutRotationQuat;
	if (!GetRotationActionAsQuat(OutRotationQuat, Object, Element, FQuat::MakeFromRotator(RelativeRotation), bActive, Tag, VisualLoggerSettings))
	{
		OutRotation = FRotator::ZeroRotator;
		return false;
	}

	OutRotation = OutRotationQuat.Rotator();
	return true;
}

bool ULearningAgentsActions::GetRotationActionAsQuat(
	FQuat& OutRotation,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const FQuat RelativeRotation,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	TStaticArray<float, 3> OutValues;
	if (!GetContinuousActionToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutRotation = FQuat::Identity;
		return false;
	}

	const FVector LocalRotationVector = FVector(FMath::DegreesToRadians(OutValues[0]), FMath::DegreesToRadians(OutValues[1]), FMath::DegreesToRadians(OutValues[2]));
	const FQuat LocalRotation = FQuat::MakeFromRotationVector(LocalRotationVector);
	OutRotation = RelativeRotation * LocalRotation;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			VisualLoggerSettings.DebugLocation,
			LocalRotation.Rotator(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Rotation Vector: [% 6.1f % 6.1f % 6.1f]\nLocal Rotation: [% 6.1f % 6.1f % 6.1f % 6.1f]\nRotation: [% 6.1f % 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			LocalRotationVector.X, LocalRotationVector.Y, LocalRotationVector.Z,
			LocalRotation.X, LocalRotation.Y, LocalRotation.Z, LocalRotation.W,
			OutRotation.X, OutRotation.Y, OutRotation.Z, OutRotation.W);
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetScaleAction(
	FVector& OutScale,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const FVector RelativeScale,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	TStaticArray<float, 3> OutValues;
	if (!GetContinuousActionToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutScale = FVector::OneVector;
		return false;
	}

	const FVector LocalScaleVector = UE::Learning::Agents::Action::Private::VectorExp(FVector(OutValues[0], OutValues[1], OutValues[2]));
	OutScale = RelativeScale * LocalScaleVector;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Scale: [% 6.1f % 6.1f % 6.1f]\nScale: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			LocalScaleVector.X, LocalScaleVector.Y, LocalScaleVector.Z,
			OutScale.X, OutScale.Y, OutScale.Z);
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetTransformAction(
	FTransform& OutTransform,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	TStaticArray<FName, 3> OutElementNames;
	TStaticArray<FLearningAgentsActionObjectElement, 3> OutElements;
	if (!GetStructActionToArrayViews(OutElementNames, OutElements, Object, Element, bActive, Tag))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	FVector OutLocation;
	if (!GetLocationAction(OutLocation, Object, OutElements[MakeArrayView(OutElementNames).Find(TEXT("Location"))], RelativeTransform))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	FQuat OutRotation;
	if (!GetRotationActionAsQuat(OutRotation, Object, OutElements[MakeArrayView(OutElementNames).Find(TEXT("Rotation"))], RelativeTransform.GetRotation()))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	FVector OutScale;
	if (!GetScaleAction(OutScale, Object, OutElements[MakeArrayView(OutElementNames).Find(TEXT("Scale"))], RelativeTransform.GetScale3D()))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	OutTransform = FTransform(OutRotation, OutLocation, OutScale);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			OutLocation,
			OutRotation,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocation: [% 6.1f % 6.1f % 6.1f]\nRotation: [% 6.1f % 6.1f % 6.1f % 6.1f]\nScale: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			OutLocation.X, OutLocation.Y, OutLocation.Z,
			OutRotation.X, OutRotation.Y, OutRotation.Z, OutRotation.W,
			OutScale.X, OutScale.Y, OutScale.Z);
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetAngleAction(
	float& OutAngle,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const float RelativeAngle,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	float LocalAngle = 0.0f;
	if (!GetContinuousActionToArrayView(MakeArrayView(&LocalAngle, 1), Object, Element, bActive, Tag))
	{
		OutAngle = 0.0f;
		return false;
	}

	OutAngle = RelativeAngle + LocalAngle;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ANGLE_DEGREES(VisualLoggerObject, LogLearning, Display,
			OutAngle,
			0.0f,
			VisualLoggerSettings.DebugLocation,
			10.0f,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Angle: [% 6.1f]\nAngle: [% 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			LocalAngle,
			OutAngle);
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetAngleActionRadians(
	float& OutAngle,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const float RelativeAngle,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	if (!GetAngleAction(OutAngle, Object, Element, FMath::RadiansToDegrees(RelativeAngle), bActive, Tag, VisualLoggerSettings))
	{
		OutAngle = 0.0f;
		return false;
	}

	OutAngle = FMath::DegreesToRadians(OutAngle);
	return true;
}

bool ULearningAgentsActions::GetVelocityAction(
	FVector& OutVelocity,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	TStaticArray<float, 3> OutValues;
	if (!GetContinuousActionToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutVelocity = FVector::OneVector;
		return false;
	}

	const FVector LocalVelocity = FVector(OutValues[0], OutValues[1], OutValues[2]);
	OutVelocity = RelativeTransform.TransformVector(LocalVelocity);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ARROW(VisualLoggerObject, LogLearning, Display,
			VisualLoggerSettings.DebugLocation,
			VisualLoggerSettings.DebugLocation + OutVelocity,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			RelativeTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocal Velocity: [% 6.1f % 6.1f % 6.1f]\nVelocity: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			LocalVelocity.X, LocalVelocity.Y, LocalVelocity.Z,
			OutVelocity.X, OutVelocity.Y, OutVelocity.Z);
	}
#endif

	return true;
}

bool ULearningAgentsActions::GetDirectionAction(
	FVector& OutDirection,
	const ULearningAgentsActionObject* Object,
	const FLearningAgentsActionObjectElement Element,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return GetNullAction(Object, Element, Tag);
	}

	TStaticArray<float, 3> OutValues;
	if (!GetContinuousActionToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutDirection = FVector::ForwardVector;
		return false;
	}

	const FVector LocalDirection = FVector(OutValues[0], OutValues[1], OutValues[2]).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);
	OutDirection = RelativeTransform.TransformVectorNoScale(LocalDirection);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ARROW(VisualLoggerObject, LogLearning, Display,
			VisualLoggerSettings.DebugLocation,
			VisualLoggerSettings.DebugLocation + VisualLoggerSettings.ArrowLength * OutDirection,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			RelativeTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nEncoded: [% 6.2f % 6.2f % 6.2f]\nLocal Direction: [% 6.1f % 6.1f % 6.1f]\nDirection: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			OutValues[0], OutValues[1], OutValues[2],
			LocalDirection.X, LocalDirection.Y, LocalDirection.Z,
			OutDirection.X, OutDirection.Y, OutDirection.Z);
	}
#endif

	return true;
}