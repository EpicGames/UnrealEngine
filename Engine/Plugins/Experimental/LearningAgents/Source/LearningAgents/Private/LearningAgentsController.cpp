// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsController.h"

#include "LearningAgentsManager.h"
#include "LearningAgentsInteractor.h"
#include "LearningAgentsActions.h"
#include "LearningLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsController)

ULearningAgentsController::ULearningAgentsController() : Super(FObjectInitializer::Get()) {}
ULearningAgentsController::ULearningAgentsController(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsController::~ULearningAgentsController() = default;

void ULearningAgentsController::EvaluateAgentController_Implementation(
	FLearningAgentsActionObjectElement& OutActionObjectElement,
	ULearningAgentsActionObject* InActionObject,
	const ULearningAgentsObservationObject* InObservationObject,
	const FLearningAgentsObservationObjectElement& InObservationObjectElement,
	const int32 AgentIds)
{
	UE_LOGF(LogLearning, Error, "%ls: EvaluateAgentController function must be overridden!", *GetName());
	OutActionObjectElement = FLearningAgentsActionObjectElement();
}

void ULearningAgentsController::EvaluateAgentControllers_Implementation(
	TArray<FLearningAgentsActionObjectElement>& OutActionObjectElements,
	ULearningAgentsActionObject* InActionObject,
	const ULearningAgentsObservationObject* InObservationObject,
	const TArray<FLearningAgentsObservationObjectElement>& InObservationObjectElements,
	const TArray<int32>& AgentIds)
{
	const int32 AgentNum = AgentIds.Num();

	if (AgentNum != InObservationObjectElements.Num())
	{
		UE_LOGF(LogLearning, Error, "%ls: Not enough Observation Objects. Expected %i, Got %i.", *GetName(), AgentNum, InObservationObjectElements.Num());
		return;
	}

	OutActionObjectElements.Empty(AgentNum);
	for (int32 AgentIdx = 0; AgentIdx < AgentNum; AgentIdx++)
	{
		FLearningAgentsActionObjectElement OutActionObjectElement;
		EvaluateAgentController(OutActionObjectElement, InActionObject, InObservationObject, InObservationObjectElements[AgentIdx], AgentIds[AgentIdx]);
		OutActionObjectElements.Add(OutActionObjectElement);
	}
}

ULearningAgentsController* ULearningAgentsController::MakeController(
	ULearningAgentsManager*& InManager, 
	ULearningAgentsInteractor*& InInteractor, 
	TSubclassOf<ULearningAgentsController> Class,
	const FName Name)
{
	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "MakeController: InManager is nullptr.");
		return nullptr;
	}

	if (!Class)
	{
		UE_LOGF(LogLearning, Error, "MakeController: Class is nullptr.");
		return nullptr;
	}

	const FName UniqueName = MakeUniqueObjectName(InManager, Class, Name, EUniqueObjectNameOptions::GloballyUnique);

	ULearningAgentsController* Controller = NewObject<ULearningAgentsController>(InManager, Class, UniqueName);
	if (!Controller) { return nullptr; }
	
	Controller->SetupController(InManager, InInteractor);

	return Controller->IsSetup() ? Controller : nullptr;
}

void ULearningAgentsController::SetupController(ULearningAgentsManager*& InManager, ULearningAgentsInteractor*& InInteractor)
{
	if (IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup already run!", *GetName());
		return;
	}

	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "%ls: InManager is nullptr.", *GetName());
		return;
	}

	if (!InInteractor)
	{
		UE_LOGF(LogLearning, Error, "%ls: InInteractor is nullptr.", *GetName());
		return;
	}

	if (!InInteractor->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InInteractor->GetName());
		return;
	}

	Manager = InManager;
	Interactor = InInteractor;
	bIsSetup = true;

	InManager->AddListener(this);
}

void ULearningAgentsController::EvaluateController()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsController::EvaluateController);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	if (Manager->GetAgentNum() != Interactor->GetObservationObjectElements().Num())
	{
		UE_LOGF(LogLearning, Error, "%ls: Not enough Observation Objects added by GatherAgentObservations. Expected %i, Got %i.", *GetName(), Manager->GetAgentNum(), Interactor->GetObservationObjectElements().Num());
		return;
	}

	// Run EvaluateAgentControllers Callback

	Interactor->GetActionObject()->ActionObject.Reset();
	Interactor->GetActionObjectElements().Empty(Manager->GetMaxAgentNum());
	EvaluateAgentControllers(
		Interactor->GetActionObjectElements(),
		Interactor->GetActionObject(),
		Interactor->GetObservationObject(),
		Interactor->GetObservationObjectElements(),
		Manager->GetAllAgentIds());

	if (Manager->GetAgentNum() != Interactor->GetActionObjectElements().Num())
	{
		UE_LOGF(LogLearning, Error, "%ls: Not enough Action Objects added by EvaluateAgentControllers. Expected %i, Got %i.", *GetName(), Manager->GetAgentNum(), Interactor->GetActionObjectElements().Num());
		return;
	}

	// Check Action Objects are Valid and if so convert to action vectors

	for (int32 AgentIdx = 0; AgentIdx < Manager->GetAgentNum(); AgentIdx++)
	{
		if (ULearningAgentsActions::ValidateActionObjectMatchesSchema(
			Interactor->GetActionSchema(),
			Interactor->GetActionSchemaElement(),
			Interactor->GetActionObject(),
			Interactor->GetActionObjectElements()[AgentIdx]))
		{
			UE::Learning::Action::SetVectorFromObject(
				Interactor->GetActionVectorsArrayView()[Manager->GetAllAgentSet()[AgentIdx]],
				Interactor->GetActionSchema()->ActionSchema,
				Interactor->GetActionSchemaElement().SchemaElement,
				Interactor->GetActionObject()->ActionObject,
				Interactor->GetActionObjectElements()[AgentIdx].ObjectElement);

			Interactor->GetActionVectorIterationArrayView()[Manager->GetAllAgentSet()[AgentIdx]]++;
		}
	}
}

ULearningAgentsInteractor* ULearningAgentsController::GetInteractor(const TSubclassOf<ULearningAgentsInteractor> InteractorClass) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return nullptr;
	}

	return Interactor;
}

void ULearningAgentsController::RunController()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsController::RunController);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	Interactor->GatherObservations();
	EvaluateController();
	Interactor->PerformActions();
}
