// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsCritic.h"

#include "LearningAgentsManager.h"
#include "LearningAgentsInteractor.h"
#include "LearningAgentsPolicy.h"
#include "LearningNeuralNetwork.h"
#include "LearningCritic.h"
#include "LearningLog.h"
#include "LearningRandom.h"

#include "UObject/Package.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "GameFramework/Actor.h"

#include "NNERuntimeBasicCpuBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsCritic)

ULearningAgentsCritic::ULearningAgentsCritic() : Super(FObjectInitializer::Get()) {}
ULearningAgentsCritic::ULearningAgentsCritic(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsCritic::~ULearningAgentsCritic() = default;

ULearningAgentsCritic* ULearningAgentsCritic::MakeCritic(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	ULearningAgentsPolicy*& InPolicy,
	TSubclassOf<ULearningAgentsCritic> Class,
	const FName Name,
	ULearningAgentsNeuralNetwork* CriticNeuralNetworkAsset,
	const bool bReinitializeCriticNetwork,
	const FLearningAgentsCriticSettings& CriticSettings,
	const int32 Seed)
{
	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "MakeCritic: InManager is nullptr.");
		return nullptr;
	}

	if (!Class)
	{
		UE_LOGF(LogLearning, Error, "MakeCritic: Class is nullptr.");
		return nullptr;
	}

	const FName UniqueName = MakeUniqueObjectName(InManager, Class, Name, EUniqueObjectNameOptions::GloballyUnique);

	ULearningAgentsCritic* Critic = NewObject<ULearningAgentsCritic>(InManager, Class, UniqueName);
	if (!Critic) { return nullptr; }

	Critic->SetupCritic(
		InManager, 
		InInteractor,
		InPolicy,
		CriticNeuralNetworkAsset,
		bReinitializeCriticNetwork,
		CriticSettings,
		Seed);

	return Critic->IsSetup() ? Critic : nullptr;
}

void ULearningAgentsCritic::SetupCritic(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor, 
	ULearningAgentsPolicy*& InPolicy,
	ULearningAgentsNeuralNetwork* CriticNeuralNetworkAsset,
	const bool bReinitializeCriticNetwork,
	const FLearningAgentsCriticSettings& CriticSettings,
	const int32 Seed)
{
	if (IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup already run!", *GetName());
		return;
	}

	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "%ls: InManager is nullptr.", *GetName());
		return;
	}

	if (!InInteractor)
	{
		UE_LOGF(LogLearning, Error, "%ls: InInteractor is nullptr.", *GetName());
		return;
	}

	if (!InInteractor->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InInteractor->GetName());
		return;
	}

	if (!InPolicy)
	{
		UE_LOGF(LogLearning, Error, "%ls: InPolicy is nullptr.", *GetName());
		return;
	}

	if (!InPolicy->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InPolicy->GetName());
		return;
	}

	Manager = InManager;
	Interactor = InInteractor;
	Policy = InPolicy;

	const int32 ObservationEncodedVectorSize = Interactor->GetObservationEncodedVectorSize();
	const int32 MemoryStateSize = Policy->GetMemoryStateSize();

	const int32 CriticHashData[2] = { MemoryStateSize, ObservationEncodedVectorSize };
	const int32 CriticCompatibilityHash = CityHash32((const char*)CriticHashData, 2 * sizeof(int32));

	// Try to use existing Neural Network Asset

	if (CriticNeuralNetworkAsset)
	{
		CriticNetwork = CriticNeuralNetworkAsset;

		if (CriticNeuralNetworkAsset->NeuralNetworkData && !bReinitializeCriticNetwork)
		{
			if (CriticNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash() != CriticCompatibilityHash)
			{
				UE_LOGF(LogLearning, Error, "%ls: Critic Network Asset provided during Setup is incompatible with Schema. Network hash is %i vs Schema hash %i.", *GetName(),
					CriticNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash(),
					CriticCompatibilityHash);
				return;
			}

			if (CriticNeuralNetworkAsset->NeuralNetworkData->GetInputSize() != ObservationEncodedVectorSize + MemoryStateSize ||
				CriticNeuralNetworkAsset->NeuralNetworkData->GetOutputSize() != 1)
			{
				UE_LOGF(LogLearning, Error, "%ls: Critic Network Asset provided during Setup is incorrect size: Got inputs of size %i, expected %i. Got outputs of size %i, expected %i.", *GetName(),
					CriticNeuralNetworkAsset->NeuralNetworkData->GetInputSize(), ObservationEncodedVectorSize + MemoryStateSize,
					CriticNeuralNetworkAsset->NeuralNetworkData->GetOutputSize(), 1);
				return;
			}
		}
	}

	if (!CriticNetwork)
	{
		const FName UniqueName = MakeUniqueObjectName(this, ULearningAgentsNeuralNetwork::StaticClass(), TEXT("CriticNetwork"), EUniqueObjectNameOptions::GloballyUnique);
		CriticNetwork = NewObject<ULearningAgentsNeuralNetwork>(this, UniqueName);
	}

	if (!CriticNetwork->NeuralNetworkData || bReinitializeCriticNetwork)
	{
		// Create New Neural Network Asset

		if (!CriticNetwork->NeuralNetworkData)
		{
			CriticNetwork->NeuralNetworkData = NewObject<ULearningNeuralNetworkData>(CriticNetwork);
		}

		UE::NNE::RuntimeBasic::FModelBuilder Builder(UE::Learning::Random::Int(Seed ^ 0x2610fc8f));

		TArray<uint8> FileData;
		uint32 CriticInputSize, CriticOutputSize;
		Builder.WriteFileDataAndReset(FileData, CriticInputSize, CriticOutputSize,
			Builder.MakeMLP(
				ObservationEncodedVectorSize + MemoryStateSize,
				1,
				CriticSettings.HiddenLayerSize,
				CriticSettings.HiddenLayerNum + 2, // Add 2 to account for input and output layers
				UE::Learning::Agents::NeuralNetwork::GetBuilderActivationFunction(CriticSettings.ActivationFunction)));

		check(CriticInputSize == ObservationEncodedVectorSize + MemoryStateSize);
		check(CriticOutputSize == 1);

		CriticNetwork->NeuralNetworkData->Init(CriticInputSize, CriticOutputSize, CriticCompatibilityHash, FileData);
		CriticNetwork->ForceMarkDirty();
	}

	// Create Critic Object

	CriticObject = MakeShared<UE::Learning::FNeuralNetworkCritic>(
		Manager->GetMaxAgentNum(),
		ObservationEncodedVectorSize,
		MemoryStateSize,
		CriticNetwork->NeuralNetworkData->GetNetwork());

	Returns.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	UE::Learning::Array::Set<1, float>(Returns, FLT_MAX);

	ReturnsIteration.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	UE::Learning::Array::Set<1, uint64>(ReturnsIteration, INDEX_NONE);

	bIsSetup = true;

	Manager->AddListener(this);
}

void ULearningAgentsCritic::OnAgentsAdded_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Array::Set<1, uint64>(ReturnsIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, float>(Returns, 0.0f, AgentIds);
}

void ULearningAgentsCritic::OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Array::Set<1, uint64>(ReturnsIteration, INDEX_NONE, AgentIds);
	UE::Learning::Array::Set<1, float> (Returns, FLT_MAX, AgentIds);
}

void ULearningAgentsCritic::OnAgentsReset_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Array::Set<1, uint64>(ReturnsIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, float>(Returns, 0.0f, AgentIds);
}

UE::Learning::FNeuralNetworkCritic& ULearningAgentsCritic::GetCriticObject()
{
	return *CriticObject;
}

ULearningAgentsNeuralNetwork* ULearningAgentsCritic::GetCriticNetworkAsset()
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return nullptr;
	}

	return CriticNetwork;
}

void ULearningAgentsCritic::EvaluateCritic()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsCritic::EvaluateCritic);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	// Check Agents actually have encoded observations.
	// All added agents should already have some memory state even if it is zero.

	ValidAgentIds.Empty(Manager->GetMaxAgentNum());

	for (const int32 AgentId : Manager->GetAllAgentSet())
	{
		if (!Policy->HasEncodedObservationsForAgent(AgentId))
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i has not made observations so critic will not be evaluated for it. Was EncodeObservations run without error?", *GetName(), AgentId);
			continue;
		}

		ValidAgentIds.Add(AgentId);
	}

	ValidAgentSet = ValidAgentIds;
	ValidAgentSet.TryMakeSlice();

	// Evaluate Critic

	if (CriticObject->GetNeuralNetwork()->GetInputSize() != Policy->GetObservationVectorsEncoded().Num<1>() + Policy->GetMemoryState().Num<1>())
	{
		UE_LOGF(LogLearning, Error, "%ls: Critic Network Input size doesn't match. Network input size is %i but Critic expects %lli.", *GetName(),
			CriticObject->GetNeuralNetwork()->GetInputSize(), Policy->GetObservationVectorsEncoded().Num<1>() + Policy->GetMemoryState().Num<1>());
		return;
	}

	if (CriticObject->GetNeuralNetwork()->GetOutputSize() != 1)
	{
		UE_LOGF(LogLearning, Error, "%ls: Critic Network Output size don't match. Network output size is %i but Critic expects %i.", *GetName(),
			CriticObject->GetNeuralNetwork()->GetOutputSize(), 1);
		return;
	}

	CriticObject->Evaluate(
		Returns,
		Policy->GetObservationVectorsEncoded(),
		Policy->GetMemoryState(),
		ValidAgentSet);

	// Increment Discounted Return Iteration

	for (const int32 AgentId : ValidAgentSet)
	{
		ReturnsIteration[AgentId]++;
	}
}

float ULearningAgentsCritic::GetEstimatedDiscountedReturn(const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0.0f;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return 0.0f;
	}

	if (ReturnsIteration[AgentId] == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Agent with id %d has not yet computed the estimated discounted return. Did you run EvaluateCritic?", *GetName(), AgentId);
		return 0.0f;
	}

	return Returns[AgentId];
}

