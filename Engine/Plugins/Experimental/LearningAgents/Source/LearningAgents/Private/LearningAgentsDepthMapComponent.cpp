// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsDepthMapComponent.h"
#include "LearningLog.h"
#include "GameFramework/Actor.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "DrawDebugHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsDepthMapComponent)

ULearningAgentsDepthMapComponent::ULearningAgentsDepthMapComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
}

const TArray<float>& ULearningAgentsDepthMapComponent::GetDepthMapFlatArray() const
{
	return DepthMapFlatArray;
}

void ULearningAgentsDepthMapComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	BatchUpdateDepthMap();

	if (bDrawDepthFrustum)
	{
		DebugDrawCornerRays();
	}
}

void ULearningAgentsDepthMapComponent::BeginPlay()
{
	Super::BeginPlay();

	const int32 Size = DepthMapConfig.Height * DepthMapConfig.Width;
	DepthMapFlatArray.SetNumZeroed(Size);
	DepthMapDirections.SetNumZeroed(Size);

	CollisionObjectQueryParams.AddObjectTypesToQuery(CollisionChannel);
	CollisionQueryParams.AddIgnoredActor(GetOwner());

	GenerateDepthMapDirections();
}

void ULearningAgentsDepthMapComponent::GenerateDepthMapDirections()
{
	const float HalfHFOV = FMath::DegreesToRadians(DepthMapConfig.HorizontalFOV / 2);
	const float HalfVFOV = HalfHFOV / DepthMapConfig.AspectRatio;

	for (int32 H = 0; H < DepthMapConfig.Height; H++)
	{
		for (int32 W = 0; W < DepthMapConfig.Width; W++)
		{
			const float U = -1 * (((W + 0.5f) / DepthMapConfig.Width) * 2.0f - 1.0f);
			const float V = -1 * (((H + 0.5f) / DepthMapConfig.Height) * 2.0f - 1.0f);

			FVector LocalDirection(U * FMath::Tan(HalfHFOV), 1.0f, V * FMath::Tan(HalfVFOV));
			LocalDirection.Normalize();

			DepthMapDirections[H * DepthMapConfig.Width + W] = LocalDirection;
		}
	}
}

float ULearningAgentsDepthMapComponent::CastDepthRay(const FVector& RayStartWorld, const FVector& RayEndWorld) const
{
	TObjectPtr<AActor> OwnerActor = GetOwner();
	TObjectPtr<UWorld> World = GetWorld();

	if (!OwnerActor)
	{
		UE_LOGF(LogLearning, Error, "Cannot Cast Depth Ray due to invalid owner actor.");

		return EncodeDepthValue(0.0f);
	}

	if (!World)
	{
		UE_LOGF(LogLearning, Error, "Cannot Cast Depth Ray due to invalid world object.");

		return EncodeDepthValue(0.0f);
	}

	FHitResult TraceHit;
	const bool bHit = World->LineTraceSingleByObjectType(TraceHit, RayStartWorld, RayEndWorld, CollisionObjectQueryParams, CollisionQueryParams);

	return EncodeDepthValue(bHit ? 1.0f - TraceHit.Distance / DepthMapConfig.MaxDistance : 0.0f);
}

void ULearningAgentsDepthMapComponent::BatchUpdateDepthMap()
{ 
	TObjectPtr<AActor> OwnerActor = GetOwner();
	TObjectPtr<UWorld> World = GetWorld();

	if (!OwnerActor)
	{
		UE_LOGF(LogLearning, Error, "Cannot Cast Depth Ray due to invalid owner actor.");
		return;
	}
	if (!World)
	{
		UE_LOGF(LogLearning, Error, "Cannot Cast Depth Ray due to invalid world object.");
		return;
	}

	const int32 Size = DepthMapDirections.Num();

	int32 Start = BatchCastingCounter;
	int32 End = (DepthRaysBatchSize > 0) ? FMath::Min(Start + DepthRaysBatchSize, Size): Size;
	BatchCastingCounter = (End == Size) ? 0 : End;

	const FTransform RayTransform = OwnerActor->GetActorTransform();
	const FVector RayStartWorld = RayTransform.TransformPosition(FVector::ZeroVector) + DepthMapConfig.FrustumOffset;

	for (int32 i = Start; i < End; i++)
	{
		const FVector RayEndWorld = RayTransform.TransformPosition(DepthMapDirections[i] * DepthMapConfig.MaxDistance) + DepthMapConfig.FrustumOffset;
		DepthMapFlatArray[i] = CastDepthRay(RayStartWorld, RayEndWorld);
	}
}

float ULearningAgentsDepthMapComponent::EncodeDepthValue(const float DepthValue) const
{
	return DepthMapConfig.bInvertEncoding ? 1.0f - DepthValue : DepthValue;
}

void ULearningAgentsDepthMapComponent::DebugDrawCornerRays()
{
	AActor* Owner = GetOwner();
	UWorld* World = GetWorld();
	if (!Owner || !World) return;

	const int32 W = DepthMapConfig.Width;
	const int32 H = DepthMapConfig.Height;
	const FVector Origin = Owner->GetActorLocation() + DepthMapConfig.FrustumOffset;

	// Four corners of the depth map
	const int32 TopLeft = 0;
	const int32 TopRight = W - 1;
	const int32 BottomLeft = (H - 1) * W;
	const int32 BottomRight = H * W - 1;
	const int32 Center = H / 2 * W + W / 2;

	const TArray<int32> Indices = { TopLeft, TopRight, BottomLeft, BottomRight, Center };

	const FTransform RayTransform = Owner->GetActorTransform();
	const FVector RayStartWorld = RayTransform.TransformPosition(FVector::ZeroVector) + DepthMapConfig.FrustumOffset;

	for (int32 Idx : Indices)
	{
		const FVector RayEndWorld = RayTransform.TransformPosition(DepthMapDirections[Idx] * DepthMapConfig.MaxDistance) + DepthMapConfig.FrustumOffset;

		DrawDebugLine(
			World,
			RayStartWorld,
			RayEndWorld,
			FColor::Yellow,
			false,
			PrimaryComponentTick.TickInterval,
			0, 
			1.0f
		);
	}
}
