// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsFlowMatching.h"

#include "LearningAgentsManager.h"
#include "LearningAgentsInteractor.h"
#include "LearningNeuralNetwork.h"
#include "LearningLog.h"
#include "LearningRandom.h"

#include "UObject/Package.h"
#include "GameFramework/Actor.h"
#include "Dom/JsonObject.h"
#include "Logging/StructuredLog.h"

#include "NNERuntimeBasicCpuBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsFlowMatching)

ULearningAgentsFlowMatching::ULearningAgentsFlowMatching() : Super(FObjectInitializer::Get()) {}
ULearningAgentsFlowMatching::ULearningAgentsFlowMatching(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsFlowMatching::~ULearningAgentsFlowMatching() = default;

ULearningAgentsFlowMatching* ULearningAgentsFlowMatching::MakeFlowMatchingModel(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	TSubclassOf<ULearningAgentsFlowMatching> Class,
	const FName Name,
	ULearningAgentsNeuralNetwork* EncoderNeuralNetworkAsset,
	ULearningAgentsNeuralNetwork* DenoiserNeuralNetworkAsset,
	ULearningAgentsNeuralNetwork* DecoderNeuralNetworkAsset,
	const bool bReinitializeEncoderNetwork,
	const bool bReinitializeDenoiserNetwork,
	const bool bReinitializeDecoderNetwork,
	const bool bSkipCompatibilityChecks,
	const FLearningAgentsFlowMatchingSettings& FlowMatchingSettings,
	const int32 Seed)
{
	if (!InManager)
	{
		UE_LOGFMT(LogLearning, Error, "MakeFlowMatchingModel: InManager is nullptr.");
		return nullptr;
	}

	if (!Class)
	{
		UE_LOGFMT(LogLearning, Error, "MakeFlowMatchingModel: Class is nullptr.");
		return nullptr;
	}

	const FName UniqueName = MakeUniqueObjectName(InManager, Class, Name, EUniqueObjectNameOptions::GloballyUnique);

	ULearningAgentsFlowMatching* FMModel = NewObject<ULearningAgentsFlowMatching>(InManager, Class, UniqueName);
	if (!FMModel) { return nullptr; }

	FMModel->SetupFlowMatchingModel(
		InManager,
		InInteractor,
		EncoderNeuralNetworkAsset,
		DenoiserNeuralNetworkAsset,
		DecoderNeuralNetworkAsset,
		bReinitializeEncoderNetwork,
		bReinitializeDenoiserNetwork,
		bReinitializeDecoderNetwork,
		bSkipCompatibilityChecks,
		FlowMatchingSettings,
		Seed);

	return FMModel->IsSetup() ? FMModel : nullptr;
}

void ULearningAgentsFlowMatching::SetupFlowMatchingModel(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	ULearningAgentsNeuralNetwork* EncoderNeuralNetworkAsset,
	ULearningAgentsNeuralNetwork* DenoiserNeuralNetworkAsset,
	ULearningAgentsNeuralNetwork* DecoderNeuralNetworkAsset,
	const bool bReinitializeEncoderNetwork,
	const bool bReinitializeDenoiserNetwork,
	const bool bReinitializeDecoderNetwork,
	const bool bSkipCompatibilityChecks,
	const FLearningAgentsFlowMatchingSettings& InFlowMatchingSettings,
	const int32 Seed)
{
	if (IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup already run!", ("Name", GetName()));
		return;
	}

	if (!InManager)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: InManager is nullptr.", ("Name", GetName()));
		return;
	}

	if (!InInteractor)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: InInteractor is nullptr.", ("Name", GetName()));
		return;
	}

	if (!InInteractor->IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: {Interactor}'s Setup must be run before it can be used.", ("Name", GetName()), ("Interactor", InInteractor->GetName()));
		return;
	}

	Manager = InManager;
	Interactor = InInteractor;
	FlowMatchingSettings = InFlowMatchingSettings;

	const int32 ObservationVectorSize = Interactor->GetObservationVectorSize();
	const int32 ObservationEncodedVectorSize = Interactor->GetObservationEncodedVectorSize();

	// This is a hacky way of predicting action chunks
	// Inference: must set the actions to the chunk length
	// Recording: 
	// Training: 
	// During recording, 1 action group is recorded at each timestep
	// During training, enable the ability to predict action chunks
	// This requires the use of two different schemas: one for recording,  the other for training/inference
	// during inference AND recording ActionChunkSize must be set to 1 to allow proper action vector writing
	ActionDimension = Interactor->GetActionVectorSize() * FlowMatchingSettings.ActionChunkSize;

	const int32 ObservationCompatibilityHash = UE::Learning::Observation::GetSchemaObjectsCompatibilityHash(
		Interactor->GetObservationSchema()->ObservationSchema,
		Interactor->GetObservationSchemaElement().SchemaElement);
	const int32 ActionCompatibilityHash = UE::Learning::Action::GetSchemaObjectsCompatibilityHash(
		Interactor->GetActionSchema()->ActionSchema,
		Interactor->GetActionSchemaElement().SchemaElement);

	// Flow matching Denoiser input: [encoded_obs, xt, t]
	const int32 DenoiserHashData[3] = { ObservationEncodedVectorSize, ActionDimension, 1 };
	const int32 DenoiserCompatibilityHash = CityHash32((const char*)DenoiserHashData, 3 * sizeof(int32));

	if (EncoderNeuralNetworkAsset)
	{
		EncoderNetwork = EncoderNeuralNetworkAsset;

		if (EncoderNeuralNetworkAsset->NeuralNetworkData && !bReinitializeEncoderNetwork && !bSkipCompatibilityChecks)
		{
			if (EncoderNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash() != ObservationCompatibilityHash)
			{
				UE_LOGFMT(LogLearning, Error, "{Name}: Encoder Network Asset is incompatible with Schema.", ("Name", GetName()));
				return;
			}
		}
	}

	if (!EncoderNetwork)
	{
		const FName UniqueName = MakeUniqueObjectName(this, ULearningAgentsNeuralNetwork::StaticClass(), TEXT("EncoderNetwork"), EUniqueObjectNameOptions::GloballyUnique);
		EncoderNetwork = NewObject<ULearningAgentsNeuralNetwork>(this, UniqueName);
	}

	if (!EncoderNetwork->NeuralNetworkData || bReinitializeEncoderNetwork)
	{
		if (!EncoderNetwork->NeuralNetworkData)
		{
			EncoderNetwork->NeuralNetworkData = NewObject<ULearningNeuralNetworkData>(EncoderNetwork);
		}

		TArray<uint8> FileData;
		uint32 EncoderInputSize, EncoderOutputSize;
		UE::Learning::Observation::GenerateEncoderNetworkFileDataFromSchema(
			FileData,
			EncoderInputSize,
			EncoderOutputSize,
			Interactor->GetObservationSchema()->ObservationSchema,
			Interactor->GetObservationSchemaElement().SchemaElement,
			UE::Learning::Observation::FNetworkSettings(),
			UE::Learning::Random::Int(Seed ^ 0x658868dd));

		EncoderNetwork->NeuralNetworkData->Init(EncoderInputSize, EncoderOutputSize, ObservationCompatibilityHash, FileData);
		EncoderNetwork->ForceMarkDirty();
	}

	if (DenoiserNeuralNetworkAsset)
	{
		DenoiserNetwork = DenoiserNeuralNetworkAsset;

		if (DenoiserNeuralNetworkAsset->NeuralNetworkData && !bReinitializeDenoiserNetwork && !bSkipCompatibilityChecks)
		{
			if (DenoiserNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash() != DenoiserCompatibilityHash)
			{
				UE_LOGFMT(LogLearning, Error, "{Name}: Denoiser Network Asset is incompatible.", ("Name", GetName()));
				return;
			}
		}
	}

	if (!DenoiserNetwork)
	{
		const FName UniqueName = MakeUniqueObjectName(this, ULearningAgentsNeuralNetwork::StaticClass(), TEXT("DenoiserNetwork"), EUniqueObjectNameOptions::GloballyUnique);
		DenoiserNetwork = NewObject<ULearningAgentsNeuralNetwork>(this, UniqueName);
	}

	if (!DenoiserNetwork->NeuralNetworkData || bReinitializeDenoiserNetwork)
	{
		if (!DenoiserNetwork->NeuralNetworkData)
		{
			DenoiserNetwork->NeuralNetworkData = NewObject<ULearningNeuralNetworkData>(DenoiserNetwork);
		}

		UE::NNE::RuntimeBasic::FModelBuilder Builder(UE::Learning::Random::Int(Seed ^ 0x69315bf9));

		const int32 DenoiserHiddenLayerSize = FlowMatchingSettings.HiddenLayerSize;
		const int32 DenoiserHiddenLayerNum = FlowMatchingSettings.HiddenLayerNum;
		const ELearningAgentsActivationFunction DenoiserActivationFunction = FlowMatchingSettings.ActivationFunction;
		const int32 FlowMatchingDenoiserInputSize = ObservationEncodedVectorSize + ActionDimension + 1; // Conditioning size + ODE state + time [encoded_obs, xt, t]

		TArray<uint8> FileData;
		uint32 DenoiserInputSize = 0, DenoiserOutputSize = 0;

		Builder.WriteFileDataAndReset(FileData, DenoiserInputSize, DenoiserOutputSize,
			Builder.MakeSequence({
				Builder.MakeMLP(
					FlowMatchingDenoiserInputSize,
					ActionDimension,
					DenoiserHiddenLayerSize,
					DenoiserHiddenLayerNum + 2,
					UE::Learning::Agents::NeuralNetwork::GetBuilderActivationFunction(DenoiserActivationFunction)),
				Builder.MakeDenormalize(
					ActionDimension,
					Builder.MakeValuesZero(ActionDimension),
					Builder.MakeValuesOne(ActionDimension)) }));

		DenoiserNetwork->NeuralNetworkData->Init(DenoiserInputSize, DenoiserOutputSize, DenoiserCompatibilityHash, FileData);
		DenoiserNetwork->ForceMarkDirty();
	}

	if (DecoderNeuralNetworkAsset)
	{
		DecoderNetwork = DecoderNeuralNetworkAsset;

		if (DecoderNeuralNetworkAsset->NeuralNetworkData && !bReinitializeDecoderNetwork && !bSkipCompatibilityChecks)
		{
			if (DecoderNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash() != ActionCompatibilityHash)
			{
				UE_LOGFMT(LogLearning, Error, "{Name}: Decoder Network Asset is incompatible with Schema.", ("Name", GetName()));
				return;
			}
		}
	}

	if (!DecoderNetwork)
	{
		const FName UniqueName = MakeUniqueObjectName(this, ULearningAgentsNeuralNetwork::StaticClass(), TEXT("DecoderNetwork"), EUniqueObjectNameOptions::GloballyUnique);
		DecoderNetwork = NewObject<ULearningAgentsNeuralNetwork>(this, UniqueName);
	}

	if (!DecoderNetwork->NeuralNetworkData || bReinitializeDecoderNetwork)
	{
		if (!DecoderNetwork->NeuralNetworkData)
		{
			DecoderNetwork->NeuralNetworkData = NewObject<ULearningNeuralNetworkData>(DecoderNetwork);
		}

		TArray<uint8> FileData;
		uint32 DecoderInputSize = 0, DecoderOutputSize = 0;

		GenerateDenormalizerNetworkFileDataFromSchema(
			FileData,
			DecoderInputSize,
			DecoderOutputSize,
			Interactor->GetActionSchema()->ActionSchema,
			Interactor->GetActionSchemaElement().SchemaElement,
			UE::Learning::Action::FNetworkSettings(),
			UE::Learning::Random::Int(Seed ^ 0x22312bf9));

		check(DecoderInputSize == ActionDimension);
		check(DecoderOutputSize == ActionDimension);

		DecoderNetwork->NeuralNetworkData->Init(DecoderInputSize, DecoderOutputSize, ActionCompatibilityHash, FileData);
		DecoderNetwork->ForceMarkDirty();
	}

	UE::Learning::FNeuralNetworkInferenceSettings InferenceSettings;

	EncoderObject = MakeShared<UE::Learning::FNeuralNetworkFunction>(
		Manager->GetMaxAgentNum(),
		EncoderNetwork->NeuralNetworkData->GetNetwork(),
		InferenceSettings);

	DenoiserObject = MakeShared<UE::Learning::FNeuralNetworkFunction>(
		Manager->GetMaxAgentNum(),
		DenoiserNetwork->NeuralNetworkData->GetNetwork(),
		InferenceSettings);

	DecoderObject = MakeShared<UE::Learning::FNeuralNetworkFunction>(
		Manager->GetMaxAgentNum(),
		DecoderNetwork->NeuralNetworkData->GetNetwork(),
		InferenceSettings);

	// State Variables
	GlobalSeed = Seed;
	Seeds.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	UE::Learning::Array::Set<1, uint32>(Seeds, INDEX_NONE);

	const int32 DenoiserInputBufferSize = ObservationEncodedVectorSize + ActionDimension + 1;

	ObservationVectorsEncoded.SetNumUninitialized({ Manager->GetMaxAgentNum(), ObservationEncodedVectorSize });
	ODEState.SetNumUninitialized({ Manager->GetMaxAgentNum(), ActionDimension });
	VelocityBuffer.SetNumUninitialized({ Manager->GetMaxAgentNum(), ActionDimension });
	DenoiserInputBuffer.SetNumUninitialized({ Manager->GetMaxAgentNum(), DenoiserInputBufferSize });

	UE::Learning::Array::Set(ObservationVectorsEncoded, 0.0f);
	UE::Learning::Array::Set(ODEState, 0.0f);
	UE::Learning::Array::Set(VelocityBuffer, 0.0f);
	UE::Learning::Array::Set(DenoiserInputBuffer, 0.0f);

	bIsSetup = true;

	Manager->AddListener(this);
}

void ULearningAgentsFlowMatching::OnAgentsAdded_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup not complete.", ("Name", GetName()));
		return;
	}

	UE::Learning::Random::SampleIntArray(Seeds, GlobalSeed, AgentIds);
	UE::Learning::Array::Set<2, float>(ObservationVectorsEncoded, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(ODEState, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(VelocityBuffer, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(DenoiserInputBuffer, 0.0f, AgentIds);
}

void ULearningAgentsFlowMatching::OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup not complete.", ("Name", GetName()));
		return;
	}

	UE::Learning::Array::Set<1, uint32>(Seeds, INDEX_NONE, AgentIds);
	UE::Learning::Array::Set<2, float>(ObservationVectorsEncoded, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(ODEState, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(VelocityBuffer, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(DenoiserInputBuffer, 0.0f, AgentIds);
}

void ULearningAgentsFlowMatching::OnAgentsReset_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup not complete.", ("Name", GetName()));
		return;
	}

	UE::Learning::Array::Set<2, float>(ObservationVectorsEncoded, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(ODEState, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(VelocityBuffer, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(DenoiserInputBuffer, 0.0f, AgentIds);
}

TSharedRef<FJsonObject> ULearningAgentsFlowMatching::AsJsonConfig() const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	ConfigObject->SetNumberField(TEXT("ActionChunkSize"), FlowMatchingSettings.ActionChunkSize);
	ConfigObject->SetNumberField(TEXT("HiddenLayerNum"), FlowMatchingSettings.HiddenLayerNum);
	ConfigObject->SetNumberField(TEXT("HiddenLayerSize"), FlowMatchingSettings.HiddenLayerSize);
	ConfigObject->SetStringField(TEXT("ActivationFunction"), StaticEnum<ELearningAgentsActivationFunction>()->GetNameStringByValue(static_cast<int64>(FlowMatchingSettings.ActivationFunction)));

	return ConfigObject;
}

void ULearningAgentsFlowMatching::RunInference()
{
	RunInference(FlowMatchingSettings.ODEStepsNum);
}

void ULearningAgentsFlowMatching::RunInference(const int32 ODEStepsNum)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsFlowMatching::RunInference);

	if (!IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup not complete.", ("Name", GetName()));
		return;
	}

	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGFMT(LogLearning, Warning, "{Name}: No agents added to Manager.", ("Name", GetName()));
		return;
	}

	if (ODEStepsNum < 1)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: ODEStepsNum must be at least 1, got {Steps}.", ("Name", GetName()), ("Steps", ODEStepsNum));
		return;
	}

	Interactor->GatherObservations();

	ValidAgentIds.Empty(Manager->GetMaxAgentNum());
	for (const int32 AgentId : Manager->GetAllAgentSet())
	{
		ValidAgentIds.Add(AgentId);
	}
	ValidAgentSet = ValidAgentIds;
	ValidAgentSet.TryMakeSlice();

	EncoderObject->Evaluate(ObservationVectorsEncoded, Interactor->GetObservationVectorsArrayView(), ValidAgentSet);

	InitializeNoiseState(ODEState);

	// ODE integration loop
	const float DeltaTime = 1.0f / static_cast<float>(ODEStepsNum);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsFlowMatching::ODEIntegration);

		for (int32 Step = 0; Step < ODEStepsNum; Step++)
		{
			const float Time = static_cast<float>(Step) * DeltaTime;
			PerformODEStep(ODEState, Time, DeltaTime);
		}
	}

	// Denormalize final ODE state
	DecoderObject->Evaluate(Interactor->GetActionVectorsArrayView(), ODEState, ValidAgentSet);

	// Increment action vector iteration
	for (const int32 AgentId : ValidAgentSet)
	{
		Interactor->GetActionVectorIterationArrayView()[AgentId]++;
	}

	Interactor->PerformActions();
}

void ULearningAgentsFlowMatching::PerformODEStep(
	TLearningArray<2, float>& CurrentState,
	const float Time,
	const float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsFlowMatching::PerformODEStep);

	EvaluateVelocity(VelocityBuffer, CurrentState, Time);

	// Euler update: x += dt * v
	for (const int32 AgentId : ValidAgentSet)
	{
		for (int32 Dim = 0; Dim < ActionDimension; Dim++)
		{
			CurrentState[AgentId][Dim] += DeltaTime * VelocityBuffer[AgentId][Dim];
		}
	}
}

void ULearningAgentsFlowMatching::EvaluateVelocity(
	TLearningArray<2, float>& OutVelocity,
	const TLearningArray<2, float>& CurrentState,
	const float Time)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsFlowMatching::EvaluateVelocity);

	// Make denoiser input: [encoded_obs, xt, t]
	const int32 EncodedObsSize = ObservationVectorsEncoded.Num<1>();
	for (const int32 AgentId : ValidAgentSet)
	{
		int32 Offset = 0;

		// conditioning
		for (int32 i = 0; i < EncodedObsSize; i++)
		{
			DenoiserInputBuffer[AgentId][Offset++] = ObservationVectorsEncoded[AgentId][i];
		}

		// ODE state (xt)
		for (int32 i = 0; i < ActionDimension; i++)
		{
			DenoiserInputBuffer[AgentId][Offset++] = CurrentState[AgentId][i];
		}

		// t
		DenoiserInputBuffer[AgentId][Offset] = Time;

		check(Offset == EncodedObsSize + ActionDimension);
	}

	DenoiserObject->Evaluate(OutVelocity, DenoiserInputBuffer, ValidAgentSet);
}

void ULearningAgentsFlowMatching::InitializeNoiseState(TLearningArray<2, float>& OutInitialState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsFlowMatching::InitializeNoiseState);

	for (const int32 AgentId : ValidAgentSet)
	{
		// Sample Gaussian noise per dimension
		UE::Learning::Random::SampleGaussianArray(
			OutInitialState[AgentId],
			Seeds[AgentId],
			0.0f,
			1.0f);
	}
}

ULearningAgentsNeuralNetwork* ULearningAgentsFlowMatching::GetEncoderNetworkAsset()
{
	return EncoderNetwork;
}

ULearningAgentsNeuralNetwork* ULearningAgentsFlowMatching::GetDenoiserNetworkAsset()
{
	return DenoiserNetwork;
}

ULearningAgentsNeuralNetwork* ULearningAgentsFlowMatching::GetDecoderNetworkAsset()
{
	return DecoderNetwork;
}

UE::Learning::FNeuralNetworkFunction& ULearningAgentsFlowMatching::GetEncoderObject()
{
	return *EncoderObject;
}

UE::Learning::FNeuralNetworkFunction& ULearningAgentsFlowMatching::GetDenoiserObject()
{
	return *DenoiserObject;
}


UE::Learning::FNeuralNetworkFunction& ULearningAgentsFlowMatching::GetDecoderObject()
{
	return *DecoderObject;
}

void ULearningAgentsFlowMatching::GenerateDenormalizerNetworkFileDataFromSchema(
	TArray<uint8>& OutFileData,
	uint32& OutInputSize,
	uint32& OutOutputSize,
	const UE::Learning::Action::FSchema& Schema,
	const UE::Learning::Action::FSchemaElement SchemaElement,
	const UE::Learning::Action::FNetworkSettings& NetworkSettings,
	const uint32 Seed)
{
	check(Schema.IsValid(SchemaElement));

	UE::NNE::RuntimeBasic::FModelBuilder Builder(Seed);
	UE::NNE::RuntimeBasic::FModelBuilderElement Element;
	MakeDenormalizerNetworkModelBuilderElementFromSchema(Element, Builder, Schema, SchemaElement, NetworkSettings);
	Builder.WriteFileDataAndReset(OutFileData, OutInputSize, OutOutputSize, Element);
}

void ULearningAgentsFlowMatching::MakeDenormalizerNetworkModelBuilderElementFromSchema(
	UE::NNE::RuntimeBasic::FModelBuilderElement& OutElement,
	UE::NNE::RuntimeBasic::FModelBuilder& Builder,
	const UE::Learning::Action::FSchema& Schema,
	const UE::Learning::Action::FSchemaElement SchemaElement,
	const UE::Learning::Action::FNetworkSettings& NetworkSettings)
{
	const UE::Learning::Action::EType SchemaElementType = Schema.GetType(SchemaElement);

	switch (SchemaElementType)
	{
	case UE::Learning::Action::EType::Null:
	{
		OutElement = Builder.MakeCopy(0);
		break;
	}
	case UE::Learning::Action::EType::DiscreteExclusive:
	{
		const int32 ValueNum = Schema.GetDiscreteExclusive(SchemaElement).Num * FlowMatchingSettings.ActionChunkSize;

		OutElement = Builder.MakeDenormalize(
			ValueNum,
			Builder.MakeValuesZero(ValueNum),
			Builder.MakeValuesOne(ValueNum));
		break;
	}
	case UE::Learning::Action::EType::DiscreteInclusive:
	{
		const int32 ValueNum = Schema.GetDiscreteInclusive(SchemaElement).Num * FlowMatchingSettings.ActionChunkSize;

		OutElement = Builder.MakeDenormalize(
			ValueNum,
			Builder.MakeValuesZero(ValueNum),
			Builder.MakeValuesOne(ValueNum));
		break;
	}
	case UE::Learning::Action::EType::NamedDiscreteExclusive:
	{
		const int32 ValueNum = Schema.GetNamedDiscreteExclusive(SchemaElement).ElementNames.Num() * FlowMatchingSettings.ActionChunkSize;

		OutElement = Builder.MakeDenormalize(
			ValueNum,
			Builder.MakeValuesZero(ValueNum),
			Builder.MakeValuesOne(ValueNum));
		break;
	}
	case UE::Learning::Action::EType::NamedDiscreteInclusive:
	{
		const int32 ValueNum = Schema.GetNamedDiscreteInclusive(SchemaElement).ElementNames.Num() * FlowMatchingSettings.ActionChunkSize;

		OutElement = Builder.MakeDenormalize(
			ValueNum,
			Builder.MakeValuesZero(ValueNum),
			Builder.MakeValuesOne(ValueNum));
		break;
	}

	case UE::Learning::Action::EType::Continuous:
	{
		const int32 ValueNum = Schema.GetContinuous(SchemaElement).Num * FlowMatchingSettings.ActionChunkSize;

		OutElement = Builder.MakeDenormalize(
			ValueNum,
			Builder.MakeValuesZero(ValueNum),
			Builder.MakeValuesOne(ValueNum));
		break;
	}

	case UE::Learning::Action::EType::And:
	{
		const UE::Learning::Action::FSchemaAndParameters Parameters = Schema.GetAnd(SchemaElement);

		TArray<UE::NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderLayers;
		BuilderLayers.Reserve(Parameters.Elements.Num());
		for (const UE::Learning::Action::FSchemaElement SubElement : Parameters.Elements)
		{
			UE::NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
			MakeDenormalizerNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, SubElement, NetworkSettings);
			BuilderLayers.Emplace(BuilderSubElement);
		}

		OutElement = Builder.MakeConcat(BuilderLayers);
		break;
	}

	case UE::Learning::Action::EType::OrExclusive:
	{
		const UE::Learning::Action::FSchemaOrExclusiveParameters Parameters = Schema.GetOrExclusive(SchemaElement);

		TArray<UE::NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderLayers;
		BuilderLayers.Reserve(Parameters.Elements.Num());
		for (const UE::Learning::Action::FSchemaElement SubElement : Parameters.Elements)
		{
			UE::NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
			MakeDenormalizerNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, SubElement, NetworkSettings);
			BuilderLayers.Emplace(BuilderSubElement);
		}

		OutElement = Builder.MakeConcat(BuilderLayers);
		break;
	}

	case UE::Learning::Action::EType::OrInclusive:
	{
		const UE::Learning::Action::FSchemaOrInclusiveParameters Parameters = Schema.GetOrInclusive(SchemaElement);

		TArray<UE::NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<8>> BuilderLayers;
		BuilderLayers.Reserve(Parameters.Elements.Num());
		for (const UE::Learning::Action::FSchemaElement SubElement : Parameters.Elements)
		{
			UE::NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
			MakeDenormalizerNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, SubElement, NetworkSettings);
			BuilderLayers.Emplace(BuilderSubElement);
		}
		OutElement = Builder.MakeConcat(BuilderLayers);
		break;
	}

	case UE::Learning::Action::EType::Array:
	{
		UE::NNE::RuntimeBasic::FModelBuilderElement BuilderSubElement;
		MakeDenormalizerNetworkModelBuilderElementFromSchema(BuilderSubElement, Builder, Schema, Schema.GetArray(SchemaElement).Element, NetworkSettings);
		OutElement = Builder.MakeArray(Schema.GetArray(SchemaElement).Num, BuilderSubElement);
		break;
	}

	case UE::Learning::Action::EType::Encoding:
	{
		MakeDenormalizerNetworkModelBuilderElementFromSchema(OutElement, Builder, Schema, Schema.GetEncoding(SchemaElement).Element, NetworkSettings);
		break;
	}

	default:
	{
		checkNoEntry();
	}
	}
	checkf(OutElement.GetInputSize() == Schema.GetActionVectorSize(SchemaElement) * FlowMatchingSettings.ActionChunkSize,
		TEXT("Decoder Network Input unexpected size. Got %i, expected %i according to Schema."),
		OutElement.GetInputSize(), Schema.GetActionVectorSize(SchemaElement) * FlowMatchingSettings.ActionChunkSize);

	check(OutElement.GetOutputSize() == OutElement.GetInputSize());
}
