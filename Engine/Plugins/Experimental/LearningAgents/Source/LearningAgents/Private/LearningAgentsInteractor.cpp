// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsInteractor.h"

#include "LearningAgentsManager.h"
#include "LearningAgentsObservations.h"
#include "LearningAgentsActions.h"
#include "LearningAgentsNeuralNetwork.h"

#include "LearningNeuralNetwork.h"
#include "LearningArray.h"
#include "LearningLog.h"

#include "EngineDefines.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsInteractor)

ULearningAgentsInteractor::ULearningAgentsInteractor() : Super(FObjectInitializer::Get()) {}
ULearningAgentsInteractor::ULearningAgentsInteractor(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsInteractor::~ULearningAgentsInteractor() = default;

ULearningAgentsInteractor* ULearningAgentsInteractor::MakeInteractor(
	ULearningAgentsManager*& InManager,
	TSubclassOf<ULearningAgentsInteractor> Class,
	const FName Name)
{
	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "MakeInteractor: InManager is nullptr.");
		return nullptr;
	}

	if (!Class)
	{
		UE_LOGF(LogLearning, Error, "MakeInteractor: Class is nullptr.");
		return nullptr;
	}

	const FName UniqueName = MakeUniqueObjectName(InManager, Class, Name, EUniqueObjectNameOptions::GloballyUnique);

	ULearningAgentsInteractor* Interactor = NewObject<ULearningAgentsInteractor>(InManager, Class, UniqueName);
	if (!Interactor) { return nullptr; }

	Interactor->SetupInteractor(InManager);

	return Interactor->IsSetup() ? Interactor : nullptr;
}

void ULearningAgentsInteractor::SetupInteractor(ULearningAgentsManager*& InManager)
{
	if (IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup already run!", *GetName());
		return;
	}

	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "%ls: InManager is nullptr.", *GetName());
		return;
	}

	Manager = InManager;

	// Specify Observations

	const FName ObservationSchemaUniqueName = MakeUniqueObjectName(this, ULearningAgentsObservationSchema::StaticClass(), TEXT("ObservationSchema"), EUniqueObjectNameOptions::GloballyUnique);

	ObservationSchema = NewObject<ULearningAgentsObservationSchema>(this, ObservationSchemaUniqueName);
	SpecifyAgentObservation(ObservationSchemaElement, ObservationSchema);

	if (!ObservationSchema->ObservationSchema.IsValid(ObservationSchemaElement.SchemaElement))
	{
		UE_LOGF(LogLearning, Error, "%ls: Invalid observation provided to Interactor during SpecifyObservations.", *GetName());
		return;
	}

	const int32 ObservationVectorSize = ObservationSchema->ObservationSchema.GetObservationVectorSize(ObservationSchemaElement.SchemaElement);

	if (ObservationVectorSize == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Observation vector is zero-sized - specified observations have no size.", *GetName());
		return;
	}

	const int32 ObservationEncodedVectorSize = ObservationSchema->ObservationSchema.GetEncodedVectorSize(ObservationSchemaElement.SchemaElement);

	if (ObservationEncodedVectorSize == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Observation encoded vector is zero-sized - observations map to empty encoding.", *GetName());
		return;
	}

	ObservationVectors.SetNumUninitialized({ Manager->GetMaxAgentNum(), ObservationVectorSize  });
	ObservationCompatibilityHash = UE::Learning::Observation::GetSchemaObjectsCompatibilityHash(ObservationSchema->ObservationSchema, ObservationSchemaElement.SchemaElement);

	const FName ObservationObjectUniqueName = MakeUniqueObjectName(this, ULearningAgentsObservationObject::StaticClass(), TEXT("ObservationObject"), EUniqueObjectNameOptions::GloballyUnique);

	ObservationObject = NewObject<ULearningAgentsObservationObject>(this, ObservationObjectUniqueName);
	ObservationObjectElements.Empty(Manager->GetMaxAgentNum());

	// Specify Actions

	const FName ActionSchemaUniqueName = MakeUniqueObjectName(this, ULearningAgentsActionSchema::StaticClass(), TEXT("ActionSchema"), EUniqueObjectNameOptions::GloballyUnique);

	ActionSchema = NewObject<ULearningAgentsActionSchema>(this, ActionSchemaUniqueName);

	SpecifyAgentAction(ActionSchemaElement, ActionSchema);

	if (!ActionSchema->ActionSchema.IsValid(ActionSchemaElement.SchemaElement))
	{
		UE_LOGF(LogLearning, Error, "%ls: Invalid action provided to Interactor during SpecifyActions.", *GetName());
		return;
	}

	const int32 ActionVectorSize = ActionSchema->ActionSchema.GetActionVectorSize(ActionSchemaElement.SchemaElement);

	if (ActionVectorSize == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Action vector is zero-sized - specified actions have no size.", *GetName());
		return;
	}

	const int32 ActionEncodedVectorSize = ActionSchema->ActionSchema.GetEncodedVectorSize(ActionSchemaElement.SchemaElement);

	if (ActionEncodedVectorSize == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Action encoded vector is zero-sized - actions map to empty encoding.", *GetName());
		return;
	}

	const int32 ActionDistributionVectorSize = ActionSchema->ActionSchema.GetActionDistributionVectorSize(ActionSchemaElement.SchemaElement);
	const int32 ActionModifierVectorSize = ActionSchema->ActionSchema.GetActionModifierVectorSize(ActionSchemaElement.SchemaElement);

	ActionVectors.SetNumUninitialized({ Manager->GetMaxAgentNum(), ActionVectorSize });
	ActionCompatibilityHash = UE::Learning::Action::GetSchemaObjectsCompatibilityHash(ActionSchema->ActionSchema, ActionSchemaElement.SchemaElement);

	const FName ActionObjectUniqueName = MakeUniqueObjectName(this, ULearningAgentsActionObject::StaticClass(), TEXT("ActionObject"), EUniqueObjectNameOptions::GloballyUnique);

	ActionObject = NewObject<ULearningAgentsActionObject>(this, ActionObjectUniqueName);
	ActionObjectElements.Empty(Manager->GetMaxAgentNum());

	ActionModifierVectors.SetNumUninitialized({ Manager->GetMaxAgentNum(), ActionModifierVectorSize });
	
	const FName ActionModifierUniqueName = MakeUniqueObjectName(this, ULearningAgentsActionModifier::StaticClass(), TEXT("ActionModifier"), EUniqueObjectNameOptions::GloballyUnique);

	ActionModifier = NewObject<ULearningAgentsActionModifier>(this, ActionModifierUniqueName);
	ActionModifierElements.Empty(Manager->GetMaxAgentNum());

	// Reset Agent iteration

	ObservationVectorIteration.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	ActionModifierVectorIteration.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	ActionVectorIteration.SetNumUninitialized({ Manager->GetMaxAgentNum() });

	UE::Learning::Array::Set<1, uint64>(ObservationVectorIteration, INDEX_NONE);
	UE::Learning::Array::Set<1, uint64>(ActionModifierVectorIteration, INDEX_NONE);
	UE::Learning::Array::Set<1, uint64>(ActionVectorIteration, INDEX_NONE);

	bIsSetup = true;

	Manager->AddListener(this);
}

void ULearningAgentsInteractor::OnAgentsAdded_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Array::Set<1, uint64>(ObservationVectorIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ActionModifierVectorIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ActionVectorIteration, 0, AgentIds);
}

void ULearningAgentsInteractor::OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Array::Set<1, uint64>(ObservationVectorIteration, INDEX_NONE, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ActionModifierVectorIteration, INDEX_NONE, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ActionVectorIteration, INDEX_NONE, AgentIds);
}

void ULearningAgentsInteractor::OnAgentsReset_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Array::Set<1, uint64>(ObservationVectorIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ActionModifierVectorIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ActionVectorIteration, 0, AgentIds);
}

void ULearningAgentsInteractor::SpecifyAgentObservation_Implementation(FLearningAgentsObservationSchemaElement& OutObservationSchemaElement, ULearningAgentsObservationSchema* InObservationSchema)
{
	UE_LOGF(LogLearning, Error, "%ls: SpecifyAgentObservation function must be overridden!", *GetName());
	OutObservationSchemaElement = FLearningAgentsObservationSchemaElement();
}

void ULearningAgentsInteractor::GatherAgentObservation_Implementation(FLearningAgentsObservationObjectElement& OutObservationObjectElement, ULearningAgentsObservationObject* InObservationObject, const int32 AgentId)
{
	UE_LOGF(LogLearning, Error, "%ls: GatherAgentObservation function must be overridden!", *GetName());
	OutObservationObjectElement = FLearningAgentsObservationObjectElement();
}

void ULearningAgentsInteractor::GatherAgentObservations_Implementation(TArray<FLearningAgentsObservationObjectElement>& OutObservationObjectElements, ULearningAgentsObservationObject* InObservationObject, const TArray<int32>& AgentIds)
{
	OutObservationObjectElements.Empty(AgentIds.Num());
	for (const int32 AgentId : AgentIds)
	{
		FLearningAgentsObservationObjectElement OutElement;
		GatherAgentObservation(OutElement, InObservationObject, AgentId);
		OutObservationObjectElements.Add(OutElement);
	}
}

void ULearningAgentsInteractor::SpecifyAgentAction_Implementation(FLearningAgentsActionSchemaElement& OutActionSchemaElement, ULearningAgentsActionSchema* InActionSchema)
{
	UE_LOGF(LogLearning, Error, "%ls: SpecifyAgentAction function must be overridden!", *GetName());
	OutActionSchemaElement = FLearningAgentsActionSchemaElement();
}

void ULearningAgentsInteractor::PerformAgentAction_Implementation(const ULearningAgentsActionObject* InActionObject, const FLearningAgentsActionObjectElement& InActionObjectElement, const int32 AgentId)
{
	UE_LOGF(LogLearning, Error, "%ls: PerformAgentAction function must be overridden!", *GetName());
}

void ULearningAgentsInteractor::PerformAgentActions_Implementation(const ULearningAgentsActionObject* InActionObject, const TArray<FLearningAgentsActionObjectElement>& InActionObjectElements, const TArray<int32>& AgentIds)
{
	const int32 AgentNum = AgentIds.Num();

	if (AgentNum != InActionObjectElements.Num())
	{
		UE_LOGF(LogLearning, Error, "%ls: PerformAgentActions: Not enough Action Objects. Expected %i, Got %i.", *GetName(), AgentNum, InActionObjectElements.Num());
		return;
	}

	for (int32 AgentIdx = 0; AgentIdx < AgentNum; AgentIdx++)
	{
		PerformAgentAction(InActionObject, InActionObjectElements[AgentIdx], AgentIds[AgentIdx]);
	}
}

void ULearningAgentsInteractor::MakeAgentActionModifier_Implementation(
	FLearningAgentsActionModifierElement& OutActionModifierElement, 
	ULearningAgentsActionModifier* InActionModifier, 
	const ULearningAgentsObservationObject* InObservationObject,
	const FLearningAgentsObservationObjectElement& InObservationObjectElement,
	const int32 AgentId)
{
	OutActionModifierElement = ULearningAgentsActions::MakeNullActionModifier(InActionModifier);
}

void ULearningAgentsInteractor::MakeAgentActionModifiers_Implementation(
	TArray<FLearningAgentsActionModifierElement>& OutActionModifierElements, 
	ULearningAgentsActionModifier* InActionModifier, 
	const ULearningAgentsObservationObject* InObservationObject,
	const TArray<FLearningAgentsObservationObjectElement>& InObservationObjectElements,
	const TArray<int32>& AgentIds)
{
	const int32 AgentNum = AgentIds.Num();

	if (AgentNum != InObservationObjectElements.Num())
	{
		UE_LOGF(LogLearning, Error, "%ls: MakeAgentActionModifiers: Not enough Observation Objects. Expected %i, Got %i.", *GetName(), AgentNum, InObservationObjectElements.Num());
		return;
	}

	OutActionModifierElements.Empty(AgentNum);
	for (int32 AgentIdx = 0; AgentIdx < AgentNum; AgentIdx++)
	{
		FLearningAgentsActionModifierElement OutElement;
		MakeAgentActionModifier(OutElement, InActionModifier, InObservationObject, InObservationObjectElements[AgentIdx], AgentIds[AgentIdx]);
		OutActionModifierElements.Add(OutElement);
	}
}

void ULearningAgentsInteractor::GatherObservations(const UE::Learning::FIndexSet AgentSet, bool bIncrementIteration)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsInteractor::GatherObservations);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	// Run GatherAgentObservations Callback

	ValidAgentIds.Empty(Manager->GetMaxAgentNum());
	for (const int32 AgentId : AgentSet)
	{
		ValidAgentIds.Add(AgentId);
	}

	ObservationObject->ObservationObject.Reset();
	ObservationObjectElements.Empty(Manager->GetMaxAgentNum());
	GatherAgentObservations(ObservationObjectElements, ObservationObject, ValidAgentIds);

	if (ValidAgentIds.Num() != ObservationObjectElements.Num())
	{
		UE_LOGF(LogLearning, Error, "%ls: Not enough Observation Objects added by GatherAgentObservations. Expected %i, Got %i.", *GetName(), ValidAgentIds.Num(), ObservationObjectElements.Num());
		return;
	}

	// Check Observation Objects are Valid and if so convert to observation vectors

	for (int32 AgentIdx = 0; AgentIdx < AgentSet.Num(); AgentIdx++)
	{
		if (ULearningAgentsObservations::ValidateObservationObjectMatchesSchema(
			ObservationSchema, 
			ObservationSchemaElement, 
			ObservationObject, 
			ObservationObjectElements[AgentIdx]))
		{
			UE::Learning::Observation::SetVectorFromObject(
				ObservationVectors[AgentSet[AgentIdx]],
				ObservationSchema->ObservationSchema,
				ObservationSchemaElement.SchemaElement,
				ObservationObject->ObservationObject,
				ObservationObjectElements[AgentIdx].ObjectElement);

			if (bIncrementIteration)
			{
				ObservationVectorIteration[AgentSet[AgentIdx]]++;
			}
		}
	}
}

void ULearningAgentsInteractor::MakeActionModifiers(const UE::Learning::FIndexSet AgentSet, bool bIncrementIteration)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsInteractor::MakeActionModifiers);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	// Check which agents have had their observation vector set

	ValidAgentIds.Empty(Manager->GetMaxAgentNum());

	for (const int32 AgentId : AgentSet)
	{
		if (ObservationVectorIteration[AgentId] == 0)
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i does not have an observation vector so action modifiers will not be created for it. Was GatherObservations run without error?", *GetName(), AgentId);
			continue;
		}

		ValidAgentIds.Add(AgentId);
	}

	ValidAgentSet = ValidAgentIds;
	ValidAgentSet.TryMakeSlice();

	// Run MakeAgentActionModifiers Callback

	ValidAgentIds.Empty(Manager->GetMaxAgentNum());
	for (const int32 AgentId : AgentSet)
	{
		ValidAgentIds.Add(AgentId);
	}

	ActionModifier->ActionModifier.Reset();
	ActionModifierElements.Empty(Manager->GetMaxAgentNum());
	MakeAgentActionModifiers(
		ActionModifierElements,
		ActionModifier,
		ObservationObject,
		ObservationObjectElements, 
		ValidAgentIds);

	if (ValidAgentIds.Num() != ActionModifierElements.Num())
	{
		UE_LOGF(LogLearning, Error, "%ls: Not enough Action Modifiers added by MakeAgentActionModifiers. Expected %i, Got %i.", *GetName(), ValidAgentIds.Num(), ActionModifierElements.Num());
		return;
	}

	// Check Action Modifiers are Valid and if so convert to action modifier vectors

	for (int32 AgentIdx = 0; AgentIdx < AgentSet.Num(); AgentIdx++)
	{
		if (ULearningAgentsActions::ValidateActionModifierMatchesSchema(
			ActionSchema,
			ActionSchemaElement,
			ActionModifier,
			ActionModifierElements[AgentIdx]))
		{
			UE::Learning::Action::SetVectorFromModifier(
				ActionModifierVectors[AgentSet[AgentIdx]],
				ActionSchema->ActionSchema,
				ActionSchemaElement.SchemaElement,
				ActionModifier->ActionModifier,
				ActionModifierElements[AgentIdx].ModifierElement);

			if (bIncrementIteration)
			{
				ActionModifierVectorIteration[AgentSet[AgentIdx]]++;
			}
		}
	}
}

void ULearningAgentsInteractor::PerformActions(const UE::Learning::FIndexSet AgentSet)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsInteractor::PerformActions);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	// Check which agents have had their action vector set

	ValidAgentIds.Empty(Manager->GetMaxAgentNum());

	for (const int32 AgentId : AgentSet)
	{
		if (ActionVectorIteration[AgentId] == 0)
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i does not have an action vector so actions will not be scattered for it. Was DecodeAndSampleActions run without error?", *GetName(), AgentId);
			continue;
		}

		ValidAgentIds.Add(AgentId);
	}

	ValidAgentSet = ValidAgentIds;
	ValidAgentSet.TryMakeSlice();

	// Generate Action Objects

	ActionObject->ActionObject.Reset();
	ActionObjectElements.Empty(Manager->GetMaxAgentNum());

	for (int32 AgentIdx = 0; AgentIdx < ValidAgentSet.Num(); AgentIdx++)
	{
		FLearningAgentsActionObjectElement ActionObjectElement;

		UE::Learning::Action::GetObjectFromVector(
			ActionObject->ActionObject,
			ActionObjectElement.ObjectElement,
			ActionSchema->ActionSchema,
			ActionSchemaElement.SchemaElement,
			ActionVectors[ValidAgentSet[AgentIdx]]);

		ActionObjectElements.Add(ActionObjectElement);
	}

	// Perform Action Objects

	PerformAgentActions(ActionObject, ActionObjectElements, ValidAgentIds);
}

void ULearningAgentsInteractor::GatherObservations()
{
	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	GatherObservations(Manager->GetAllAgentSet());
}

void ULearningAgentsInteractor::MakeActionModifiers()
{
	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	MakeActionModifiers(Manager->GetAllAgentSet());
}

void ULearningAgentsInteractor::PerformActions()
{
	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	PerformActions(Manager->GetAllAgentSet());
}


void ULearningAgentsInteractor::GetObservationVector(TArray<float>& OutObservationVector, int32& OutObservationCompatibilityHash, const int32 AgentId)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		OutObservationVector.Empty();
		OutObservationCompatibilityHash = 0;
		return;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		OutObservationVector.Empty();
		OutObservationCompatibilityHash = 0;
		return;
	}

	if (ObservationVectorIteration[AgentId] == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Observation vector not set for agent %i.", *GetName(), AgentId);
		OutObservationVector.Empty();
		OutObservationCompatibilityHash = 0;
		return;
	}

	OutObservationVector.SetNumUninitialized(GetObservationVectorSize());
	UE::Learning::Array::Copy<1, float>(OutObservationVector, ObservationVectors[AgentId]);
	OutObservationCompatibilityHash = ObservationCompatibilityHash;
}

void ULearningAgentsInteractor::GetActionModifierVector(TArray<float>& OutActionModifierVector, int32& OutActionCompatibilityHash, const int32 AgentId)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		OutActionModifierVector.Empty();
		OutActionCompatibilityHash = 0;
		return;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		OutActionModifierVector.Empty();
		OutActionCompatibilityHash = 0;
		return;
	}

	if (ActionModifierVectorIteration[AgentId] == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Action Modifier vector not set for agent %i.", *GetName(), AgentId);
		OutActionModifierVector.Empty();
		OutActionCompatibilityHash = 0;
		return;
	}

	OutActionModifierVector.SetNumUninitialized(GetActionModifierVectorSize());
	UE::Learning::Array::Copy<1, float>(OutActionModifierVector, ActionModifierVectors[AgentId]);
	OutActionCompatibilityHash = ActionCompatibilityHash;
}

void ULearningAgentsInteractor::GetActionVector(TArray<float>& OutActionVector, int32& OutActionCompatibilityHash, const int32 AgentId)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		OutActionVector.Empty();
		OutActionCompatibilityHash = 0;
		return;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		OutActionVector.Empty();
		OutActionCompatibilityHash = 0;
		return;
	}

	if (ActionVectorIteration[AgentId] == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Action vector not set for agent %i.", *GetName(), AgentId);
		OutActionVector.Empty();
		OutActionCompatibilityHash = 0;
		return;
	}

	OutActionVector.SetNumUninitialized(GetActionVectorSize());
	UE::Learning::Array::Copy<1, float>(OutActionVector, ActionVectors[AgentId]);
	OutActionCompatibilityHash = ActionCompatibilityHash;
}

void ULearningAgentsInteractor::SetObservationVector(const TArray<float>& ObservationVector, const int32 InObservationCompatibilityHash, const int32 AgentId, bool bIncrementIteration)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return;
	}

	if (InObservationCompatibilityHash != ObservationCompatibilityHash)
	{
		UE_LOGF(LogLearning, Error, "%ls: Observation Compatibility hash incompatible. Got %i, expected %i.", *GetName(), InObservationCompatibilityHash, ObservationCompatibilityHash);
		return;
	}

	if (ObservationVector.Num() != GetObservationVectorSize())
	{
		UE_LOGF(LogLearning, Error, "%ls: Observation Vector size incompatible. Got %i, expected %i.", *GetName(), ObservationVector.Num(), GetObservationVectorSize());
		return;
	}

	UE::Learning::Array::Copy<1, float>(ObservationVectors[AgentId], ObservationVector);
	if (bIncrementIteration) { ObservationVectorIteration[AgentId]++; }
}

void ULearningAgentsInteractor::SetActionModifierVector(const TArray<float>& ActionModifierVector, const int32 InActionCompatibilityHash, const int32 AgentId, bool bIncrementIteration)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return;
	}

	if (InActionCompatibilityHash != ActionCompatibilityHash)
	{
		UE_LOGF(LogLearning, Error, "%ls: Action Compatibility hash incompatible. Got %i, expected %i.", *GetName(), InActionCompatibilityHash, ActionCompatibilityHash);
		return;
	}

	if (ActionModifierVector.Num() != GetActionModifierVectorSize())
	{
		UE_LOGF(LogLearning, Error, "%ls: Action Modifier Vector size incompatible. Got %i, expected %i.", *GetName(), ActionModifierVector.Num(), GetActionModifierVectorSize());
		return;
	}

	UE::Learning::Array::Copy<1, float>(ActionModifierVectors[AgentId], ActionModifierVector);
	if (bIncrementIteration) { ActionModifierVectorIteration[AgentId]++; }
}

void ULearningAgentsInteractor::SetActionVector(const TArray<float>& ActionVector, const int32 InActionCompatibilityHash, const int32 AgentId, bool bIncrementIteration)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return;
	}

	if (InActionCompatibilityHash != ActionCompatibilityHash)
	{
		UE_LOGF(LogLearning, Error, "%ls: Action Compatibility hash incompatible. Got %i, expected %i.", *GetName(), InActionCompatibilityHash, ActionCompatibilityHash);
		return;
	}

	if (ActionVector.Num() != GetActionVectorSize())
	{
		UE_LOGF(LogLearning, Error, "%ls: Action Vector size incompatible. Got %i, expected %i.", *GetName(), ActionVector.Num(), GetActionVectorSize());
		return;
	}

	UE::Learning::Array::Copy<1, float>(ActionVectors[AgentId], ActionVector);
	if (bIncrementIteration) { ActionVectorIteration[AgentId]++; }
}

bool ULearningAgentsInteractor::HasObservationVector(const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return false;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return false;
	}

	return ObservationVectorIteration[AgentId] > 0;
}

bool ULearningAgentsInteractor::HasActionModifierVector(const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return false;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return false;
	}

	return ActionModifierVectorIteration[AgentId] > 0;
}

bool ULearningAgentsInteractor::HasActionVector(const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return false;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return false;
	}

	return ActionVectorIteration[AgentId] > 0;
}

int32 ULearningAgentsInteractor::GetObservationVectorSize() const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0;
	}

	return ObservationSchema->ObservationSchema.GetObservationVectorSize(ObservationSchemaElement.SchemaElement);
}

int32 ULearningAgentsInteractor::GetObservationEncodedVectorSize() const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0;
	}

	return ObservationSchema->ObservationSchema.GetEncodedVectorSize(ObservationSchemaElement.SchemaElement);
}

int32 ULearningAgentsInteractor::GetActionVectorSize() const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0;
	}

	return ActionSchema->ActionSchema.GetActionVectorSize(ActionSchemaElement.SchemaElement);
}

int32 ULearningAgentsInteractor::GetActionDistributionVectorSize() const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0;
	}

	return ActionSchema->ActionSchema.GetActionDistributionVectorSize(ActionSchemaElement.SchemaElement);
}

int32 ULearningAgentsInteractor::GetActionModifierVectorSize() const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0;
	}

	return ActionSchema->ActionSchema.GetActionModifierVectorSize(ActionSchemaElement.SchemaElement);
}

int32 ULearningAgentsInteractor::GetActionEncodedVectorSize() const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0;
	}

	return ActionSchema->ActionSchema.GetEncodedVectorSize(ActionSchemaElement.SchemaElement);
}

const ULearningAgentsObservationSchema* ULearningAgentsInteractor::GetObservationSchema() const
{
	return ObservationSchema;
}

const FLearningAgentsObservationSchemaElement ULearningAgentsInteractor::GetObservationSchemaElement() const
{
	return ObservationSchemaElement;
}

const ULearningAgentsActionSchema* ULearningAgentsInteractor::GetActionSchema() const
{
	return ActionSchema;
}

const FLearningAgentsActionSchemaElement ULearningAgentsInteractor::GetActionSchemaElement() const
{
	return ActionSchemaElement;
}

TLearningArrayView<2, const float> ULearningAgentsInteractor::GetObservationVectorsArrayView() const
{
	return ObservationVectors;
}

uint64 ULearningAgentsInteractor::GetObservationIteration(const int32 AgentId) const
{
	return ObservationVectorIteration[AgentId];
}

TLearningArrayView<2, const float> ULearningAgentsInteractor::GetActionModifierVectorsArrayView() const
{
	return ActionModifierVectors;
}

uint64 ULearningAgentsInteractor::GetActionModifierIteration(const int32 AgentId) const
{
	return ActionModifierVectorIteration[AgentId];
}

TLearningArrayView<2, const float> ULearningAgentsInteractor::GetActionVectorsArrayView() const
{
	return ActionVectors;
}

uint64 ULearningAgentsInteractor::GetActionIteration(const int32 AgentId) const
{
	return ActionVectorIteration[AgentId];
}

const ULearningAgentsObservationObject* ULearningAgentsInteractor::GetObservationObject() const
{
	return ObservationObject;
}

const TArray<FLearningAgentsObservationObjectElement>& ULearningAgentsInteractor::GetObservationObjectElements() const
{
	return ObservationObjectElements;
}

const ULearningAgentsActionModifier* ULearningAgentsInteractor::GetActionModifier() const
{
	return ActionModifier;
}

const TArray<FLearningAgentsActionModifierElement>& ULearningAgentsInteractor::GetActionModifierElements() const
{
	return ActionModifierElements;
}

ULearningAgentsActionObject* ULearningAgentsInteractor::GetActionObject()
{
	return ActionObject;
}

TArray<FLearningAgentsActionObjectElement>& ULearningAgentsInteractor::GetActionObjectElements()
{
	return ActionObjectElements;
}

TLearningArrayView<2, float> ULearningAgentsInteractor::GetActionVectorsArrayView()
{
	return ActionVectors;
}

TLearningArrayView<1, uint64> ULearningAgentsInteractor::GetActionVectorIterationArrayView()
{
	return ActionVectorIteration;
}

