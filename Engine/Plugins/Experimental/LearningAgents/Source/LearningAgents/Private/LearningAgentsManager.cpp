// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsManager.h"
#include "LearningAgentsManagerListener.h"

#include "LearningArray.h"
#include "LearningLog.h"

#include "UObject/ScriptInterface.h"
#include "EngineDefines.h"
#include "Async/Async.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsManager)

ULearningAgentsManager::ULearningAgentsManager() : Super(FObjectInitializer::Get()) { PrimaryComponentTick.bCanEverTick = true; }
ULearningAgentsManager::ULearningAgentsManager(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsManager::~ULearningAgentsManager() = default;

void ULearningAgentsManager::OnRegister()
{
	Super::OnRegister();

	// Pre-populate the vacant ids
	OnEventAgentIds.Reserve(MaxAgentNum);
	OccupiedAgentIds.Reserve(MaxAgentNum);
	VacantAgentIds.Reserve(MaxAgentNum);
	Agents.Reserve(MaxAgentNum);
	for (int32 AgentId = MaxAgentNum - 1; AgentId >= 0; AgentId--)
	{
		VacantAgentIds.Push(AgentId);
		Agents.Add(nullptr);
	}

	UpdateAgentSets();
}

void ULearningAgentsManager::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// We only want to give agent tick events when the actual level is ticking - not when paused or just the viewport is ticking.
	if (TickType == ELevelTick::LEVELTICK_All)
	{
		for (ULearningAgentsManagerListener* Listener : Listeners)
		{
			Listener->OnAgentsManagerTick(OccupiedAgentIds, DeltaTime);
		}
	}
}

int32 ULearningAgentsManager::GetMaxAgentNum() const
{
	return MaxAgentNum;
}

void ULearningAgentsManager::SetMaxAgentNum(const int32 InMaxAgentNum)
{
	if (InMaxAgentNum <= 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Attempted to SetMaxAgentNum but the input is less than or equal to 0.", *GetName());
		return;
	}
	
	if (GetAgentNum() > 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Cannot SetMaxAgentNum because agents have already been added.", *GetName());
		return;
	}

	int32 PreviousMaxAgentNum = MaxAgentNum;
	MaxAgentNum = InMaxAgentNum;

	OnEventAgentIds.Reserve(MaxAgentNum);
	OccupiedAgentIds.Reserve(MaxAgentNum);
	VacantAgentIds.Reserve(MaxAgentNum);
	Agents.Reserve(MaxAgentNum);

	for (int32 AgentId = MaxAgentNum - 1; AgentId >= PreviousMaxAgentNum; AgentId--)
	{
		VacantAgentIds.Push(AgentId);
		Agents.Add(nullptr);
	}

	UpdateAgentSets();
}

int32 ULearningAgentsManager::AddAgent(UObject* Agent)
{
	if (Agent == nullptr)
	{
		UE_LOGF(LogLearning, Error, "%ls: Attempted to add an agent but agent is nullptr.", *GetName());
		return INDEX_NONE;
	}

	if (VacantAgentIds.IsEmpty())
	{
		UE_LOGF(LogLearning, Error, "%ls: Attempting to add an agent but we have no more vacant ids. Increase MaxAgentNum (%d) or remove unused agents.", *GetName(), MaxAgentNum);
		return INDEX_NONE;
	}

	if (Agents.Contains(Agent))
	{
		UE_LOGF(LogLearning, Error, "%ls: Agent %ls already added to this manager.", *GetName(), *Agent->GetName());
		return INDEX_NONE;
	}

	// Add Agent

	const int32 AgentId = VacantAgentIds.Pop();
	Agents[AgentId] = Agent;
	OccupiedAgentIds.Add(AgentId);
	UpdateAgentSets();

	OnEventAgentIds.Reset();
	OnEventAgentIds.Add(AgentId);

	// Call OnAgentsAdded Event

	UE_LOGF(LogLearning, Display, "%ls: Adding Agent %ls with id %i.", *GetName(), *Agent->GetName(), AgentId);

	for (ULearningAgentsManagerListener* Listener : Listeners)
	{
		Listener->OnAgentsAdded(OnEventAgentIds);
	}

	// Return new id

	return AgentId;
}

void ULearningAgentsManager::AddAgents(TArray<int32>& OutAgentIds, const TArray<UObject*>& InAgents)
{
	OnEventAgentIds.Reset();
	OutAgentIds.SetNumUninitialized(InAgents.Num());

	for (int32 AgentIdx = 0; AgentIdx < InAgents.Num(); AgentIdx++)
	{
		if (InAgents[AgentIdx] == nullptr)
		{
			UE_LOGF(LogLearning, Error, "%ls: Attempted to add an agent but agent is nullptr.", *GetName());
			OutAgentIds[AgentIdx] = INDEX_NONE;
			continue;
		}

		if (VacantAgentIds.IsEmpty())
		{
			UE_LOGF(LogLearning, Error, "%ls: Attempting to add an agent but we have no more vacant ids. Increase MaxAgentNum (%d) or remove unused agents.", *GetName(), MaxAgentNum);
			OutAgentIds[AgentIdx] = INDEX_NONE;
			continue;
		}

		if (Agents.Contains(InAgents[AgentIdx]))
		{
			UE_LOGF(LogLearning, Error, "%ls: Agent %ls already added to this manager.", *GetName(), *InAgents[AgentIdx]->GetName());
			continue;
		}

		// Add Agent

		OutAgentIds[AgentIdx] = VacantAgentIds.Pop();
		Agents[OutAgentIds[AgentIdx]] = InAgents[AgentIdx];
		OccupiedAgentIds.Add(OutAgentIds[AgentIdx]);
		OnEventAgentIds.Add(OutAgentIds[AgentIdx]);
	}

	UpdateAgentSets();

	// Call OnAgentsAdded Event

	UE_LOGF(LogLearning, Display, "%ls: Adding Agents %ls.", *GetName(), *UE::Learning::Array::FormatInt32(OnEventAgentIds));

	for (ULearningAgentsManagerListener* Listener : Listeners)
	{
		Listener->OnAgentsAdded(OnEventAgentIds);
	}
}

void ULearningAgentsManager::RequestAddAgent(UObject* Agent)
{
	if (!IsInGameThread())
	{
		AsyncTask(ENamedThreads::GameThread, [this, Agent]()
			{
				AddAgent(Agent);
			});
		return;
	}
	AddAgent(Agent);
}

void ULearningAgentsManager::RemoveAgent(const int32 AgentId)
{
	if (AgentId == INDEX_NONE)
	{
		UE_LOGF(LogLearning, Error, "%ls: Attempting to remove an agent with id of -1.", *GetName());
		return;
	}

	const int32 RemovedCount = OccupiedAgentIds.RemoveSingleSwap(AgentId, EAllowShrinking::No);

	if (RemovedCount == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Trying to remove an agent with id of %i but it was not found.", *GetName(), AgentId);
		return;
	}

	checkf(RemovedCount == 1, TEXT("Somehow agent id was included multiple times in set."));

	// Remove Agent

	VacantAgentIds.Push(AgentId);
	Agents[AgentId] = nullptr;
	UpdateAgentSets();

	OnEventAgentIds.Reset();
	OnEventAgentIds.Add(AgentId);

	// Call OnAgentsRemoved Event

	UE_LOGF(LogLearning, Display, "%ls: Removing Agent %i.", *GetName(), AgentId);

	for (ULearningAgentsManagerListener* Listener : Listeners)
	{
		Listener->OnAgentsRemoved(OnEventAgentIds);
	}
}

void ULearningAgentsManager::RemoveAgents(const TArray<int32>& AgentIds)
{
	OnEventAgentIds.Reset();

	for (int32 AgentIdx = 0; AgentIdx < AgentIds.Num(); AgentIdx++)
	{
		if (AgentIds[AgentIdx] == INDEX_NONE)
		{
			UE_LOGF(LogLearning, Error, "%ls: Attempting to remove an agent with id of -1.", *GetName());
			continue;
		}

		const int32 RemovedCount = OccupiedAgentIds.RemoveSingleSwap(AgentIds[AgentIdx], EAllowShrinking::No);

		if (RemovedCount == 0)
		{
			UE_LOGF(LogLearning, Error, "%ls: Trying to remove an agent with id of %i but it was not found.", *GetName(), AgentIds[AgentIdx]);
			continue;
		}

		checkf(RemovedCount == 1, TEXT("Somehow agent id was included multiple times in set."));

		// Remove Agent

		VacantAgentIds.Push(AgentIds[AgentIdx]);
		Agents[AgentIds[AgentIdx]] = nullptr;
		OnEventAgentIds.Add(AgentIds[AgentIdx]);
	}

	UpdateAgentSets();

	// Call OnAgentsRemoved Event

	UE_LOGF(LogLearning, Display, "%ls: Removing Agents %ls.", *GetName(), *UE::Learning::Array::FormatInt32(OnEventAgentIds));

	for (ULearningAgentsManagerListener* Listener : Listeners)
	{
		Listener->OnAgentsRemoved(OnEventAgentIds);
	}
}

void ULearningAgentsManager::RemoveAllAgents()
{
	// We need to make a copy as OccupiedAgentIds will be modified during removal.
	const TArray<int32> RemoveAgentIds = OccupiedAgentIds;
	RemoveAgents(RemoveAgentIds);
}

void ULearningAgentsManager::RequestRemoveAgent(const int32 AgentId)
{
	if (!IsInGameThread())
	{
		AsyncTask(ENamedThreads::GameThread, [this, AgentId]()
			{
				RemoveAgent(AgentId);
			});
		return;
	}
	RemoveAgent(AgentId);
}

void ULearningAgentsManager::ResetAgent(const int32 AgentId)
{
	if (AgentId == INDEX_NONE)
	{
		UE_LOGF(LogLearning, Error, "%ls: Attempting to reset an agent with id of -1.", *GetName());
		return;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: Trying to reset an agent with id of %i but it was not found.", *GetName(), AgentId);
		return;
	}

	OnEventAgentIds.Reset();
	OnEventAgentIds.Add(AgentId);

	// Call OnAgentsReset Event

	UE_LOGF(LogLearning, Display, "%ls: Resetting Agent %i.", *GetName(), AgentId);

	for (ULearningAgentsManagerListener* Listener : Listeners)
	{
		Listener->OnAgentsReset(OnEventAgentIds);
	}
}

void ULearningAgentsManager::ResetAgents(const TArray<int32>& AgentIds)
{
	OnEventAgentIds.Reset();

	for (int32 AgentIdx = 0; AgentIdx < AgentIds.Num(); AgentIdx++)
	{
		if (AgentIds[AgentIdx] == INDEX_NONE)
		{
			UE_LOGF(LogLearning, Error, "%ls: Attempting to reset an agent with id of -1.", *GetName());
			continue;
		}

		if (!HasAgent(AgentIds[AgentIdx]))
		{
			UE_LOGF(LogLearning, Error, "%ls: Trying to reset an agent with id of %i but it was not found.", *GetName(), AgentIds[AgentIdx]);
			continue;
		}

		OnEventAgentIds.Add(AgentIds[AgentIdx]);
	}

	// Call OnAgentsReset Event

	UE_LOGF(LogLearning, Display, "%ls: Resetting Agents %ls.", *GetName(), *UE::Learning::Array::FormatInt32(OnEventAgentIds));

	for (ULearningAgentsManagerListener* Listener : Listeners)
	{
		Listener->OnAgentsReset(OnEventAgentIds);
	}
}

void ULearningAgentsManager::ResetAllAgents()
{
	ResetAgents(OccupiedAgentIds);
}

UObject* ULearningAgentsManager::GetAgent(const int32 AgentId, const TSubclassOf<UObject> AgentClass) const
{
	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found. Be sure to only use AgentIds returned by AddAgent and check that the agent has not been removed.", *GetName(), AgentId);
		return nullptr;
	}
	
	if (Agents[AgentId] == nullptr)
	{
		UE_LOGF(LogLearning, Warning, "%ls: AgentId %d object is nullptr.", *GetName(), AgentId);
		return nullptr;
	}
	
	if (!Agents[AgentId].IsA(AgentClass))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d cast invalid. Agent was of class '%ls', not '%ls'.", *GetName(), AgentId, *Agents[AgentId]->GetClass()->GetName(), *AgentClass->GetName());
		return nullptr;
	}
	
	return Agents[AgentId];
}

void ULearningAgentsManager::GetAgents(TArray<UObject*>& OutAgents, const TArray<int32>& AgentIds, const TSubclassOf<UObject> AgentClass) const
{
	OutAgents.SetNumUninitialized(AgentIds.Num());

	for (int32 AgentIdIdx = 0; AgentIdIdx < AgentIds.Num(); AgentIdIdx++)
	{
		if (!HasAgent(AgentIds[AgentIdIdx]))
		{
			UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found. Be sure to only use AgentIds returned by AddAgent and check that the agent has not been removed.", *GetName(), AgentIds[AgentIdIdx]);
			OutAgents[AgentIdIdx] = nullptr;
			continue;
		}
		
		if (Agents[AgentIds[AgentIdIdx]] == nullptr)
		{
			UE_LOGF(LogLearning, Warning, "%ls: AgentId %d object is nullptr.", *GetName(), AgentIds[AgentIdIdx]);
			OutAgents[AgentIdIdx] = nullptr;
			continue;
		}
		
		if (!Agents[AgentIds[AgentIdIdx]].IsA(AgentClass))
		{
			UE_LOGF(LogLearning, Error, "%ls: AgentId %d cast invalid. Agent was of class '%ls', not '%ls'.", *GetName(), AgentIds[AgentIdIdx], *Agents[AgentIds[AgentIdIdx]]->GetClass()->GetName(), *AgentClass->GetName());
			OutAgents[AgentIdIdx] = nullptr;
			continue;
		}
		
		OutAgents[AgentIdIdx] = Agents[AgentIds[AgentIdIdx]];
	}
}

void ULearningAgentsManager::GetAllAgents(TArray<UObject*>& OutAgents, TArray<int32>& OutAgentIds, const TSubclassOf<UObject> AgentClass) const
{
	OutAgentIds = OccupiedAgentIds;
	GetAgents(OutAgents, OutAgentIds, AgentClass);
}

int32 ULearningAgentsManager::GetAgentId(UObject* Agent) const
{
	const int32 AgentId = Agents.Find(Agent);

	if (AgentId == INDEX_NONE)
	{
		UE_LOGF(LogLearning, Error, "%ls: Cannot get id for Agent %ls, as it was not found in added agent set.", *GetName(), *Agent->GetName());
		return INDEX_NONE;
	}
	else
	{
		return AgentId;
	}
}

void ULearningAgentsManager::GetAgentIds(TArray<int32>& OutAgentIds, const TArray<UObject*>& InAgents) const
{
	OutAgentIds.SetNumUninitialized(InAgents.Num());

	for (int32 AgentIdx = 0; AgentIdx < InAgents.Num(); AgentIdx++)
	{
		const int32 AgentId = Agents.Find(InAgents[AgentIdx]);

		if (AgentId == INDEX_NONE)
		{
			UE_LOGF(LogLearning, Error, "%ls: Cannot get id for Agent %ls, as it was not found in added agent set.", *GetName(), *InAgents[AgentIdx]->GetName());
			OutAgentIds[AgentIdx] = INDEX_NONE;
		}
		else
		{
			OutAgentIds[AgentIdx] = AgentId;
		}
	}
}

int32 ULearningAgentsManager::GetAgentNum() const
{
	return OccupiedAgentIds.Num();
}

bool ULearningAgentsManager::HasAgentObject(UObject* Agent) const
{
	return Agents.Contains(Agent);
}

bool ULearningAgentsManager::HasAgent(const int32 AgentId) const
{
	return OccupiedAgentSet.Contains(AgentId);
}

const UObject* ULearningAgentsManager::GetAgent(const int32 AgentId) const
{
	if (ensure(HasAgent(AgentId)))
	{
		return Agents[AgentId];
	}

	return nullptr;
}

UObject* ULearningAgentsManager::GetAgent(const int32 AgentId)
{
	if (ensure(HasAgent(AgentId)))
	{
		return Agents[AgentId];
	}

	return nullptr;
}

const TArray<int32>& ULearningAgentsManager::GetAllAgentIds() const
{
	return OccupiedAgentIds;
}

UE::Learning::FIndexSet ULearningAgentsManager::GetAllAgentSet() const
{
	return OccupiedAgentSet;
}

TConstArrayView<TObjectPtr<UObject>> ULearningAgentsManager::GetAgents() const
{
	return Agents;
}

void ULearningAgentsManager::AddListener(ULearningAgentsManagerListener* Listener)
{
	Listeners.Add(Listener);
	OnEventAgentIds = OccupiedAgentIds;
	
	if (!OnEventAgentIds.IsEmpty())
	{
		Listener->OnAgentsAdded(OnEventAgentIds);
	}
}

void ULearningAgentsManager::RemoveListener(ULearningAgentsManagerListener* Listener)
{
	OnEventAgentIds = OccupiedAgentIds;
	
	if (!OnEventAgentIds.IsEmpty())
	{
		Listener->OnAgentsRemoved(OnEventAgentIds);
	}

	Listeners.Remove(Listener);
}

void ULearningAgentsManager::UpdateAgentSets()
{
	// Keep OccupiedAgentIds sorted in ascending order for better memory access patterns.
	OccupiedAgentIds.Sort();
	OccupiedAgentSet = OccupiedAgentIds;
	OccupiedAgentSet.TryMakeSlice();

	// Keep VacantAgentIds sorted in descending order so that lowest ids are popped first.
	VacantAgentIds.Sort([](const int32& Lhs, const int32& Rhs) { return Lhs > Rhs; });
	VacantAgentSet = VacantAgentIds;
	VacantAgentSet.TryMakeSlice();
}
