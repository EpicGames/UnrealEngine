// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsManagerListener.h"

#include "LearningAgentsManager.h"
#include "LearningLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsManagerListener)

ULearningAgentsManagerListener::ULearningAgentsManagerListener(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}
ULearningAgentsManagerListener::ULearningAgentsManagerListener(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsManagerListener::~ULearningAgentsManagerListener() = default;

void ULearningAgentsManagerListener::OnAgentsAdded_Implementation(const TArray<int32>& AgentIds)
{
	// Can be overridden for additional logic when agents are added
}

void ULearningAgentsManagerListener::OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds)
{
	// Can be overridden for additional logic when agents are removed
}

void ULearningAgentsManagerListener::OnAgentsReset_Implementation(const TArray<int32>& AgentIds)
{
	// Can be overridden for additional logic when agents are reset
}

void ULearningAgentsManagerListener::OnAgentsManagerTick_Implementation(const TArray<int32>& AgentIds, const float DeltaTime)
{
	// Can be overridden for additional logic when the agent manager is ticked
}

UObject* ULearningAgentsManagerListener::GetAgent(const int32 AgentId, const TSubclassOf<UObject> AgentClass) const
{
	if (!Manager)
	{
		UE_LOGF(LogLearning, Error, "%ls: Manager variable not set.", *GetName());
		return nullptr;
	}

	return Manager->GetAgent(AgentId, AgentClass);
}

void ULearningAgentsManagerListener::GetAgents(const TArray<int32>& AgentIds, const TSubclassOf<UObject> AgentClass, TArray<UObject*>& OutAgents) const
{
	if (!Manager)
	{
		UE_LOGF(LogLearning, Error, "%ls: Manager variable not set.", *GetName());
		OutAgents.Empty();
		return;
	}

	Manager->GetAgents(OutAgents, AgentIds, AgentClass);
}

void ULearningAgentsManagerListener::GetAllAgents(TArray<UObject*>& OutAgents, TArray<int32>& OutAgentIds, const TSubclassOf<UObject> AgentClass) const
{
	if (!Manager)
	{
		UE_LOGF(LogLearning, Error, "%ls: Manager variable not set.", *GetName());
		OutAgents.Empty();
		return;
	}

	Manager->GetAllAgents(OutAgents, OutAgentIds, AgentClass);
}

ULearningAgentsManager* ULearningAgentsManagerListener::GetAgentManager() const
{
	return Manager;
}

const UObject* ULearningAgentsManagerListener::GetAgent(const int32 AgentId) const
{
	checkf(Manager, TEXT("Manager variable not set."));
	return Manager->GetAgent(AgentId);
}

UObject* ULearningAgentsManagerListener::GetAgent(const int32 AgentId)
{
	checkf(Manager, TEXT("Manager variable not set."));
	return Manager->GetAgent(AgentId);
}

bool ULearningAgentsManagerListener::HasAgent(const int32 AgentId) const
{
	checkf(Manager, TEXT("Manager variable not set."));
	return Manager->HasAgent(AgentId);
}

bool ULearningAgentsManagerListener::HasAgentManager() const
{
	return Manager != nullptr;
}

bool ULearningAgentsManagerListener::IsSetup() const
{
	return bIsSetup;
}

const ULearningAgentsVisualLoggerObject* ULearningAgentsManagerListener::GetOrAddVisualLoggerObject(const FName Name)
{
	const TObjectPtr<const ULearningAgentsVisualLoggerObject>* Value = VisualLoggerObjects.Find(Name);

	if (Value)
	{
		return Value->Get();
	}
	else
	{
		const FName UniqueName = MakeUniqueObjectName(this, ULearningAgentsVisualLoggerObject::StaticClass(), Name, EUniqueObjectNameOptions::GloballyUnique);
		const ULearningAgentsVisualLoggerObject* NewLoggerObject = NewObject<ULearningAgentsVisualLoggerObject>(this, UniqueName);
		VisualLoggerObjects.Add(Name, NewLoggerObject);
		return NewLoggerObject;
	}
}
