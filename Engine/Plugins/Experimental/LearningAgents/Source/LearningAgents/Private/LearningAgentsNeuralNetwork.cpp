// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsNeuralNetwork.h"

#include "LearningArray.h"
#include "LearningNeuralNetwork.h"

#include "UObject/Package.h"
#include "Misc/FileHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsNeuralNetwork)

ULearningAgentsNeuralNetwork::ULearningAgentsNeuralNetwork() = default;
ULearningAgentsNeuralNetwork::ULearningAgentsNeuralNetwork(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsNeuralNetwork::~ULearningAgentsNeuralNetwork() = default;

void ULearningAgentsNeuralNetwork::ResetNetwork()
{
	if (NeuralNetworkData)
	{
		NeuralNetworkData->ConditionalBeginDestroy();
		NeuralNetworkData = nullptr;
	}
	
	ForceMarkDirty();
}

void ULearningAgentsNeuralNetwork::LoadNetworkFromSnapshot(const FFilePath& File)
{
	TArray<uint8> NetworkData;

	if (FFileHelper::LoadFileToArray(NetworkData, *File.FilePath))
	{
		if (!NeuralNetworkData->LoadFromSnapshot(NetworkData))
		{
			UE_LOGF(LogLearning, Error, "%ls: Failed to load network. Invalid Format: \"%ls\"", *GetName(), *File.FilePath);
			return;
		}

		ForceMarkDirty();
	}
	else
	{
		UE_LOGF(LogLearning, Error, "%ls: Failed to load network. File not found: \"%ls\"", *GetName(), *File.FilePath);
	}
}

void ULearningAgentsNeuralNetwork::SaveNetworkToSnapshot(const FFilePath& File)
{
	if (!NeuralNetworkData)
	{
		UE_LOGF(LogLearning, Error, "%ls: No network data to save", *GetName());
		return;
	}

	TArray<uint8> NetworkData;
	NetworkData.SetNumUninitialized(NeuralNetworkData->GetSnapshotByteNum());
	NeuralNetworkData->SaveToSnapshot(NetworkData);

	if (!FFileHelper::SaveArrayToFile(NetworkData, *File.FilePath))
	{
		UE_LOGF(LogLearning, Error, "%ls: Failed to save network to file: \"%ls\"", *GetName(), *File.FilePath);
	}
}

void ULearningAgentsNeuralNetwork::LoadNetworkFromAsset(ULearningAgentsNeuralNetwork* NeuralNetworkAsset)
{
	if (!NeuralNetworkAsset || !NeuralNetworkAsset->NeuralNetworkData)
	{
		UE_LOGF(LogLearning, Error, "%ls: Asset is invalid.", *GetName());
		return;
	}

	if (NeuralNetworkAsset == this)
	{
		UE_LOGF(LogLearning, Error, "%ls: Asset is same as the current network.", *GetName());
		return;
	}

	if (!NeuralNetworkData)
	{
		NeuralNetworkData = NewObject<ULearningNeuralNetworkData>(this);
	}

	NeuralNetworkData->InitFrom(NeuralNetworkAsset->NeuralNetworkData);
	ForceMarkDirty();
}

void ULearningAgentsNeuralNetwork::SaveNetworkToAsset(ULearningAgentsNeuralNetwork* NeuralNetworkAsset)
{
	if (!NeuralNetworkAsset)
	{
		UE_LOGF(LogLearning, Error, "%ls: Asset is invalid.", *GetName());
		return;
	}

	if (NeuralNetworkAsset == this)
	{
		UE_LOGF(LogLearning, Error, "%ls: Asset is same as the current network.", *GetName());
		return;
	}

	if (!NeuralNetworkAsset->NeuralNetworkData)
	{
		NeuralNetworkAsset->NeuralNetworkData = NewObject<ULearningNeuralNetworkData>(NeuralNetworkAsset);
	}

	NeuralNetworkAsset->NeuralNetworkData->InitFrom(NeuralNetworkData);
	NeuralNetworkAsset->ForceMarkDirty();
}

void ULearningAgentsNeuralNetwork::ForceMarkDirty()
{
	// Manually mark the package as dirty since just using `Modify` prevents 
	// marking packages as dirty during PIE which is most likely when this
	// is being used.
	if (UPackage* Package = GetPackage())
	{
		const bool bIsDirty = Package->IsDirty();

		if (!bIsDirty)
		{
			Package->SetDirtyFlag(true);
		}

		Package->PackageMarkedDirtyEvent.Broadcast(Package, bIsDirty);
	}
}
