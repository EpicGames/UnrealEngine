// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsObservations.h"

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsDebug.h"

#include "LearningArray.h"
#include "LearningLog.h"

#include "Containers/StaticArray.h"

#include "Runtime/Engine/Public/DrawDebugHelpers.h"
#include "Engine/World.h"
#include "Components/SplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsObservations)

bool operator==(const FLearningAgentsObservationSchemaElement& Lhs, const FLearningAgentsObservationSchemaElement& Rhs)
{
	return Lhs.SchemaElement.Index == Rhs.SchemaElement.Index;
}

bool operator==(const FLearningAgentsObservationObjectElement& Lhs, const FLearningAgentsObservationObjectElement& Rhs)
{
	return Lhs.ObjectElement.Index == Rhs.ObjectElement.Index;
}

uint32 GetTypeHash(const FLearningAgentsObservationSchemaElement& Element)
{
	return (uint32)Element.SchemaElement.Index;
}

uint32 GetTypeHash(const FLearningAgentsObservationObjectElement& Element)
{
	return (uint32)Element.ObjectElement.Index;
}

namespace UE::Learning::Agents::Observation::Private
{
	// We declare this here instead of in LearningArray.h because FName is part of the UObject system and right now the 
	// Learning core module does not have a dependency on the UObject part of UE which is a nice thing to preserve.	
	static inline FString FormatNames(const TLearningArrayView<1, const FName> Array, const int32 MaxItemNum = 16)
	{
		return Array::Format<FName>(Array, [](const FName& Value) { return FString::Printf(TEXT("%s"), *Value.ToString()); }, MaxItemNum);
	}

	static inline bool ContainsDuplicates(const TArrayView<const int32> Indices)
	{
		TSet<int32, DefaultKeyFuncs<int32>, TInlineSetAllocator<32>> IndicesSet;
		IndicesSet.Append(Indices);
		return Indices.Num() != IndicesSet.Num();
	}

	static inline bool ContainsDuplicates(const TArrayView<const FName> ElementNames)
	{
		TSet<FName, DefaultKeyFuncs<FName>, TInlineSetAllocator<32>> ElementNameSet;
		ElementNameSet.Append(ElementNames);
		return ElementNames.Num() != ElementNameSet.Num();
	}

	static inline const TCHAR* GetObservationTypeString(const Learning::Observation::EType ObservationType)
	{
		switch (ObservationType)
		{
		case  Learning::Observation::EType::Null: return TEXT("Null");
		case  Learning::Observation::EType::Continuous: return TEXT("Continuous");
		case  Learning::Observation::EType::DiscreteExclusive: return TEXT("DiscreteExclusive");
		case  Learning::Observation::EType::DiscreteInclusive: return TEXT("DiscreteInclusive");
		case  Learning::Observation::EType::NamedDiscreteExclusive: return TEXT("NamedDiscreteExclusive");
		case  Learning::Observation::EType::NamedDiscreteInclusive: return TEXT("NamedDiscreteInclusive");
		case  Learning::Observation::EType::And: return TEXT("Struct");
		case  Learning::Observation::EType::OrExclusive: return TEXT("ExclusiveUnion");
		case  Learning::Observation::EType::OrInclusive: return TEXT("InclusiveUnion");
		case  Learning::Observation::EType::Array: return TEXT("StaticArray");
		case  Learning::Observation::EType::Set: return TEXT("Set");
		case  Learning::Observation::EType::Encoding: return TEXT("Encoding");
		default:
			checkNoEntry();
			return TEXT("Unimplemented");
		}
	}

	static bool ValidateObservationObjectMatchesSchema(
		const Learning::Observation::FSchema& Schema,
		const Learning::Observation::FSchemaElement SchemaElement,
		const Learning::Observation::FObject& Object,
		const Learning::Observation::FObjectElement ObjectElement)
	{
		// Check Elements are Valid

		if (!Schema.IsValid(SchemaElement))
		{
			UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Invalid Observation Schema Element.");
			return false;
		}

		if (!Object.IsValid(ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Invalid Observation Object Element.");
			return false;
		}

		// Check Names Match

		const FName ObservationSchemaElementTag = Schema.GetTag(SchemaElement);
		const FName ObservationObjectElementTag = Object.GetTag(ObjectElement);

		if (ObservationSchemaElementTag != ObservationObjectElementTag)
		{
			UE_LOGF(LogLearning, Warning, "ValidateObservationObjectMatchesSchema: Observation tag does not match Schema. Expected '%ls', got '%ls'.",
				*ObservationSchemaElementTag.ToString(), *ObservationObjectElementTag.ToString());
		}

		// Check Types Match

		const Learning::Observation::EType ObservationSchemaElementType = Schema.GetType(SchemaElement);
		const Learning::Observation::EType ObservationObjectElementType = Object.GetType(ObjectElement);

		if (ObservationSchemaElementType != ObservationObjectElementType)
		{
			UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' type does not match Schema. Expected type '%ls', got type '%ls'.",
				*ObservationSchemaElementTag.ToString(),
				GetObservationTypeString(ObservationSchemaElementType),
				GetObservationTypeString(ObservationObjectElementType));
			return false;
		}

		// Type Specific Checks

		switch (ObservationSchemaElementType)
		{
		case Learning::Observation::EType::Null: return true;
		
		case Learning::Observation::EType::Continuous:
		{
			const int32 SchemaElementSize = Schema.GetContinuous(SchemaElement).Num;
			const int32 ObjectElementSize = Object.GetContinuous(ObjectElement).Values.Num();

			if (SchemaElementSize != ObjectElementSize)
			{
				UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' size does not match Schema. Expected '%i', got '%i'.",
					*ObservationSchemaElementTag.ToString(),
					SchemaElementSize,
					ObjectElementSize);
				return false;
			}

			return true;
		}

		case Learning::Observation::EType::DiscreteExclusive:
		{
			const int32 SchemaElementSize = Schema.GetDiscreteExclusive(SchemaElement).Num;
			const int32 ObjectElementIndex = Object.GetDiscreteExclusive(ObjectElement).DiscreteIndex;

			if (ObjectElementIndex < 0 || ObjectElementIndex >= SchemaElementSize)
			{
				UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' index out of range for Schema. Expected '<%i', got '%i'.",
					*ObservationSchemaElementTag.ToString(),
					SchemaElementSize,
					ObjectElementIndex);
				return false;
			}

			return true;
		}

		case Learning::Observation::EType::DiscreteInclusive:
		{
			const int32 SchemaElementSize = Schema.GetDiscreteInclusive(SchemaElement).Num;
			const TArrayView<const int32> ObjectElementIndices = Object.GetDiscreteInclusive(ObjectElement).DiscreteIndices;

			if (ObjectElementIndices.Num() > SchemaElementSize)
			{
				UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' too many indices provided. Expected at most '%i', got '%i'.",
					*ObservationSchemaElementTag.ToString(),
					SchemaElementSize,
					ObjectElementIndices.Num());
				return false;
			}

			for (int32 SubElementIdx = 0; SubElementIdx < ObjectElementIndices.Num(); SubElementIdx++)
			{
				if (ObjectElementIndices[SubElementIdx] < 0 || ObjectElementIndices[SubElementIdx] >= SchemaElementSize)
				{
					UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' index out of range for Schema. Expected '<%i', got '%i'.",
						*ObservationSchemaElementTag.ToString(),
						SchemaElementSize,
						ObjectElementIndices[SubElementIdx]);
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::NamedDiscreteExclusive:
		{
			const Learning::Observation::FSchemaNamedDiscreteExclusiveParameters SchemaParameters = Schema.GetNamedDiscreteExclusive(SchemaElement);
			const Learning::Observation::FObjectNamedDiscreteExclusiveParameters ObjectParameters = Object.GetNamedDiscreteExclusive(ObjectElement);
			check(SchemaParameters.ElementNames.Num() == SchemaParameters.ElementNames.Num());

			const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementName);

			if (SchemaSubElementIdx == INDEX_NONE)
			{
				UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' Schema does not include '%ls' observation.",
					*ObservationSchemaElementTag.ToString(),
					*ObjectParameters.ElementName.ToString());
				return false;
			}

			return true;
		}

		case Learning::Observation::EType::NamedDiscreteInclusive:
		{
			const Learning::Observation::FSchemaNamedDiscreteInclusiveParameters SchemaParameters = Schema.GetNamedDiscreteInclusive(SchemaElement);
			const Learning::Observation::FObjectNamedDiscreteInclusiveParameters ObjectParameters = Object.GetNamedDiscreteInclusive(ObjectElement);

			if (ObjectParameters.ElementNames.Num() > SchemaParameters.ElementNames.Num())
			{
				UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' too many sub-observations provided. Expected at most '%i', got '%i'.",
					*ObservationSchemaElementTag.ToString(),
					SchemaParameters.ElementNames.Num(),
					ObjectParameters.ElementNames.Num());
				return false;
			}

			for (int32 ObjectSubElementIdx = 0; ObjectSubElementIdx < ObjectParameters.ElementNames.Num(); ObjectSubElementIdx++)
			{
				const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementNames[ObjectSubElementIdx]);

				if (SchemaSubElementIdx == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' Schema does not include '%ls' observation.",
						*ObservationSchemaElementTag.ToString(),
						*ObjectParameters.ElementNames[ObjectSubElementIdx].ToString());
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::And:
		{
			const Learning::Observation::FSchemaAndParameters SchemaParameters = Schema.GetAnd(SchemaElement);
			const Learning::Observation::FObjectAndParameters ObjectParameters = Object.GetAnd(ObjectElement);
			check(SchemaParameters.Elements.Num() == SchemaParameters.ElementNames.Num());
			check(ObjectParameters.Elements.Num() == ObjectParameters.ElementNames.Num());

			if (SchemaParameters.Elements.Num() != ObjectParameters.Elements.Num())
			{
				UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' number of sub-elements does not match Schema. Expected '%i', got '%i'.",
					*ObservationSchemaElementTag.ToString(),
					SchemaParameters.Elements.Num(),
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 SchemaElementIdx = 0; SchemaElementIdx < SchemaParameters.Elements.Num(); SchemaElementIdx++)
			{
				const int32 ObjectElementIdx = ObjectParameters.ElementNames.Find(SchemaParameters.ElementNames[SchemaElementIdx]);

				if (ObjectElementIdx == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' does not include '%ls' observation required by Schema.",
						*ObservationSchemaElementTag.ToString(),
						*SchemaParameters.ElementNames[SchemaElementIdx].ToString());
					return false;
				}

				if (!ValidateObservationObjectMatchesSchema(
					Schema,
					SchemaParameters.Elements[SchemaElementIdx],
					Object,
					ObjectParameters.Elements[ObjectElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::OrExclusive:
		{
			const Learning::Observation::FSchemaOrExclusiveParameters SchemaParameters = Schema.GetOrExclusive(SchemaElement);
			const Learning::Observation::FObjectOrExclusiveParameters ObjectParameters = Object.GetOrExclusive(ObjectElement);
			check(SchemaParameters.Elements.Num() == SchemaParameters.ElementNames.Num());

			const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementName);

			if (SchemaSubElementIdx == INDEX_NONE)
			{
				UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' Schema does not include '%ls' observation.",
					*ObservationSchemaElementTag.ToString(),
					*ObjectParameters.ElementName.ToString());
				return false;
			}

			return ValidateObservationObjectMatchesSchema(
				Schema,
				SchemaParameters.Elements[SchemaSubElementIdx],
				Object,
				ObjectParameters.Element);
		}

		case Learning::Observation::EType::OrInclusive:
		{
			const Learning::Observation::FSchemaOrInclusiveParameters SchemaParameters = Schema.GetOrInclusive(SchemaElement);
			const Learning::Observation::FObjectOrInclusiveParameters ObjectParameters = Object.GetOrInclusive(ObjectElement);

			if (ObjectParameters.Elements.Num() > SchemaParameters.Elements.Num())
			{
				UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' too many sub-observations provided. Expected at most '%i', got '%i'.",
					*ObservationSchemaElementTag.ToString(),
					SchemaParameters.Elements.Num(),
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 ObjectSubElementIdx = 0; ObjectSubElementIdx < ObjectParameters.Elements.Num(); ObjectSubElementIdx++)
			{
				const int32 SchemaSubElementIdx = SchemaParameters.ElementNames.Find(ObjectParameters.ElementNames[ObjectSubElementIdx]);

				if (SchemaSubElementIdx == INDEX_NONE)
				{
					UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' Schema does not include '%ls' observation.",
						*ObservationSchemaElementTag.ToString(),
						*ObjectParameters.ElementNames[ObjectSubElementIdx].ToString());
					return false;
				}

				if (!ValidateObservationObjectMatchesSchema(
					Schema,
					SchemaParameters.Elements[SchemaSubElementIdx],
					Object,
					ObjectParameters.Elements[ObjectSubElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::Array:
		{
			const Learning::Observation::FSchemaArrayParameters SchemaParameters = Schema.GetArray(SchemaElement);
			const Learning::Observation::FObjectArrayParameters ObjectParameters = Object.GetArray(ObjectElement);

			if (ObjectParameters.Elements.Num() != SchemaParameters.Num)
			{
				UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' array incorrect size. Expected '%i' elements, got '%i'.",
					*ObservationSchemaElementTag.ToString(),
					SchemaParameters.Num,
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 ElementIdx = 0; ElementIdx < ObjectParameters.Elements.Num(); ElementIdx++)
			{
				if (!ValidateObservationObjectMatchesSchema(
					Schema,
					SchemaParameters.Element,
					Object,
					ObjectParameters.Elements[ElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::Set:
		{
			const Learning::Observation::FSchemaSetParameters SchemaParameters = Schema.GetSet(SchemaElement);
			const Learning::Observation::FObjectSetParameters ObjectParameters = Object.GetSet(ObjectElement);

			if (ObjectParameters.Elements.Num() > SchemaParameters.MaxNum)
			{
				UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Observation '%ls' set too large. Expected at most '%i' elements, got '%i'.",
					*ObservationSchemaElementTag.ToString(),
					SchemaParameters.MaxNum,
					ObjectParameters.Elements.Num());
				return false;
			}

			for (int32 ElementIdx = 0; ElementIdx < ObjectParameters.Elements.Num(); ElementIdx++)
			{
				if (!ValidateObservationObjectMatchesSchema(
					Schema,
					SchemaParameters.Element,
					Object,
					ObjectParameters.Elements[ElementIdx]))
				{
					return false;
				}
			}

			return true;
		}

		case Learning::Observation::EType::Encoding:
		{
			const Learning::Observation::FSchemaEncodingParameters SchemaParameters = Schema.GetEncoding(SchemaElement);
			const Learning::Observation::FObjectEncodingParameters ObjectParameters = Object.GetEncoding(ObjectElement);

			return ValidateObservationObjectMatchesSchema(
				Schema,
				SchemaParameters.Element,
				Object,
				ObjectParameters.Element);
		}

		case Learning::Observation::EType::Conv1d:
		{
			const Learning::Observation::FSchemaConv1dParameters SchemaParameters = Schema.GetConv1d(SchemaElement);
			const Learning::Observation::FObjectConv1dParameters ObjectParameters = Object.GetConv1d(ObjectElement);

			return ValidateObservationObjectMatchesSchema(
				Schema,
				SchemaParameters.Element,
				Object,
				ObjectParameters.Element);
		}

		case Learning::Observation::EType::Conv2d:
		{
			const Learning::Observation::FSchemaConv2dParameters SchemaParameters = Schema.GetConv2d(SchemaElement);
			const Learning::Observation::FObjectConv2dParameters ObjectParameters = Object.GetConv2d(ObjectElement);

			return ValidateObservationObjectMatchesSchema(
				Schema,
				SchemaParameters.Element,
				Object,
				ObjectParameters.Element);
		}

		default:
		{
			checkNoEntry();
			return true;
		}
		}
	}

	static void LogObservation(
		const UE::Learning::Observation::FObject& Object, 
		const UE::Learning::Observation::FObjectElement ObjectElement,
		const FString& Indentation,
		const FString& Prefix)
	{
		if (!Object.IsValid(ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "LogObservation: Invalid Observation Object Element.");
			return;
		}

		const UE::Learning::Observation::EType Type = Object.GetType(ObjectElement);
		const FName Tag = Object.GetTag(ObjectElement);

		switch (Type)
		{
		case UE::Learning::Observation::EType::Null:
		{
			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type));
			return;
		}

		case UE::Learning::Observation::EType::Continuous:
		{
			const UE::Learning::Observation::FObjectContinuousParameters Parameters = Object.GetContinuous(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls) %ls", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type), *UE::Learning::Array::FormatFloat(Parameters.Values));
			return;
		}

		case UE::Learning::Observation::EType::DiscreteExclusive:
		{
			const UE::Learning::Observation::FObjectDiscreteExclusiveParameters Parameters = Object.GetDiscreteExclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls) %i", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type), Parameters.DiscreteIndex);
			return;
		}

		case UE::Learning::Observation::EType::DiscreteInclusive:
		{
			const UE::Learning::Observation::FObjectDiscreteInclusiveParameters Parameters = Object.GetDiscreteInclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls) %ls", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type), *UE::Learning::Array::FormatInt32(Parameters.DiscreteIndices));
			return;
		}

		case UE::Learning::Observation::EType::NamedDiscreteExclusive:
		{
			const UE::Learning::Observation::FObjectNamedDiscreteExclusiveParameters Parameters = Object.GetNamedDiscreteExclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls) %ls", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type), *Parameters.ElementName.ToString());
			return;
		}

		case UE::Learning::Observation::EType::NamedDiscreteInclusive:
		{
			const UE::Learning::Observation::FObjectNamedDiscreteInclusiveParameters Parameters = Object.GetNamedDiscreteInclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls) %ls", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type), *UE::Learning::Agents::Observation::Private::FormatNames(Parameters.ElementNames));
			return;
		}

		case UE::Learning::Observation::EType::And:
		{
			const UE::Learning::Observation::FObjectAndParameters Parameters = Object.GetAnd(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				LogObservation(Object, Parameters.Elements[SubElementIdx], *(Indentation + TEXT("    ")), FString::Printf(TEXT("| \"%s\": "), *Parameters.ElementNames[SubElementIdx].ToString()));
			}

			return;
		}

		case UE::Learning::Observation::EType::OrExclusive:
		{
			const UE::Learning::Observation::FObjectOrExclusiveParameters Parameters = Object.GetOrExclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type));
			LogObservation(Object, Parameters.Element, *(Indentation + TEXT("    ")), FString::Printf(TEXT("| \"%s\": "), *Parameters.ElementName.ToString()));

			return;
		}

		case UE::Learning::Observation::EType::OrInclusive:
		{
			const UE::Learning::Observation::FObjectOrInclusiveParameters Parameters = Object.GetOrInclusive(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				LogObservation(Object, Parameters.Elements[SubElementIdx], *(Indentation + TEXT("    ")), FString::Printf(TEXT("| \"%s\": "), *Parameters.ElementNames[SubElementIdx].ToString()));
			}

			return;
		}

		case UE::Learning::Observation::EType::Array:
		{
			const UE::Learning::Observation::FObjectArrayParameters Parameters = Object.GetArray(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				LogObservation(Object, Parameters.Elements[SubElementIdx], *(Indentation + TEXT("    ")), FString::Printf(TEXT("| %3i:"), SubElementIdx));
			}

			return;
		}

		case UE::Learning::Observation::EType::Set:
		{
			const UE::Learning::Observation::FObjectSetParameters Parameters = Object.GetSet(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type));
			for (int32 SubElementIdx = 0; SubElementIdx < Parameters.Elements.Num(); SubElementIdx++)
			{
				LogObservation(Object, Parameters.Elements[SubElementIdx], *(Indentation + TEXT("    ")), FString::Printf(TEXT("| %3i:"), SubElementIdx));
			}

			return;
		}

		case UE::Learning::Observation::EType::Encoding:
		{
			const UE::Learning::Observation::FObjectEncodingParameters Parameters = Object.GetEncoding(ObjectElement);

			UE_LOGF(LogLearning, Display, "%ls%ls \"%ls\" (%ls)", *Indentation, *Prefix, *Tag.ToString(), GetObservationTypeString(Type));
			LogObservation(Object, Parameters.Element, *(Indentation + TEXT("    ")), TEXT("|"));

			return;
		}
		}
	}

	static inline FVector VectorLogSafe(const FVector V, const float Epsilon = UE_SMALL_NUMBER)
	{
		return FVector(
			FMath::Loge(FMath::Max(V.X, Epsilon)),
			FMath::Loge(FMath::Max(V.Y, Epsilon)),
			FMath::Loge(FMath::Max(V.Z, Epsilon)));
	}

	static inline FVector VectorExp(const FVector V)
	{
		return FVector(
			FMath::Exp(V.X),
			FMath::Exp(V.Y),
			FMath::Exp(V.Z));
	}

	static inline Learning::Observation::EPaddingMode GetPaddingMode(const ELearningAgentsPaddingMode PaddingMode)
	{
		switch (PaddingMode)
		{
		case ELearningAgentsPaddingMode::Zeros: return Learning::Observation::EPaddingMode::Zeros;
		case ELearningAgentsPaddingMode::Circular: return Learning::Observation::EPaddingMode::Circular;
		default: checkNoEntry(); return Learning::Observation::EPaddingMode::Zeros;
		}
	}

	static inline Learning::Observation::EEncodingActivationFunction GetEncodingActivationFunction(const ELearningAgentsActivationFunction ActivationFunction)
	{
		switch (ActivationFunction)
		{
		case ELearningAgentsActivationFunction::ReLU: return Learning::Observation::EEncodingActivationFunction::ReLU;
		case ELearningAgentsActivationFunction::ELU: return Learning::Observation::EEncodingActivationFunction::ELU;
		case ELearningAgentsActivationFunction::TanH: return Learning::Observation::EEncodingActivationFunction::TanH;
		default: checkNoEntry(); return Learning::Observation::EEncodingActivationFunction::ReLU;
		}
	}
	
	static inline Learning::Observation::ENormalization GetNormalizationOption(const ELearningAgentsNormalization NormalizationOption)
	{
		switch (NormalizationOption)
		{
		case ELearningAgentsNormalization::Manual: return Learning::Observation::ENormalization::Manual;
		case ELearningAgentsNormalization::AutoShared: return Learning::Observation::ENormalization::AutoShared;
		case ELearningAgentsNormalization::AutoPerDimension: return Learning::Observation::ENormalization::AutoPerDimension;
		default: checkNoEntry(); return Learning::Observation::ENormalization::Manual;
		}
	}
}

FTransform ULearningAgentsObservations::ProjectTransformOntoGroundPlane(const FTransform Transform, const FVector LocalForwardVector, const float GroundPlaneHeight)
{
	FVector Position = Transform.GetLocation();
	Position.Z = GroundPlaneHeight;

	const FVector Direction = (FVector(1.0f, 1.0f, 0.0f) * Transform.TransformVectorNoScale(LocalForwardVector)).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);

	return FTransform(FQuat::FindBetweenNormals(FVector::ForwardVector, Direction), Position, Transform.GetScale3D());
}

UEnum* ULearningAgentsObservations::FindEnumByName(const FString& Name)
{
	return FindObject<UEnum>(nullptr, *Name);
}

bool ULearningAgentsObservations::ValidateObservationObjectMatchesSchema(
	const ULearningAgentsObservationSchema* Schema,
	const FLearningAgentsObservationSchemaElement SchemaElement,
	const ULearningAgentsObservationObject* Object,
	const FLearningAgentsObservationObjectElement ObjectElement)
{
	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Schema is nullptr.");
		return false;
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "ValidateObservationObjectMatchesSchema: Object is nullptr.");
		return false;
	}

	return UE::Learning::Agents::Observation::Private::ValidateObservationObjectMatchesSchema(
		Schema->ObservationSchema,
		SchemaElement.SchemaElement,
		Object->ObservationObject,
		ObjectElement.ObjectElement);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyNullObservation(ULearningAgentsObservationSchema* Schema, const FName Tag)
{
	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyNullObservation: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	return { Schema->ObservationSchema.CreateNull(Tag)};
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyContinuousObservation(ULearningAgentsObservationSchema* Schema, const int32 Size, const FLearningAgentsObservationNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyContinuousObservation: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (Size < 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyContinuousObservation: Invalid Continuous Observation Size '%i'.", Size);
		return FLearningAgentsObservationSchemaElement();
	}

	if (Size == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyContinuousObservation: Specifying zero-sized Continuous Observation.");
	}

	return { Schema->ObservationSchema.CreateContinuous({ Size, NormalizationSettings.Scales, NormalizationSettings.Offsets, UE::Learning::Agents::Observation::Private::GetNormalizationOption(NormalizationSettings.NormalizationOption)}, Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyExclusiveDiscreteObservation(ULearningAgentsObservationSchema* Schema, const int32 Size, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveDiscreteObservation: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (Size < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveDiscreteObservation: Invalid Exclusive Discrete Observation Size '%i'.", Size);
		return FLearningAgentsObservationSchemaElement();
	}

	return { Schema->ObservationSchema.CreateDiscreteExclusive({ Size }, Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyInclusiveDiscreteObservation(ULearningAgentsObservationSchema* Schema, const int32 Size, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveDiscreteObservation: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (Size < 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveDiscreteObservation: Invalid Inclusive Discrete Observation Size '%i'.", Size);
		return FLearningAgentsObservationSchemaElement();
	}

	if (Size == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyInclusiveDiscreteObservation: Specifying zero-sized Inclusive Discrete Observation.");
	}

	return { Schema->ObservationSchema.CreateDiscreteInclusive({ Size }, Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyNamedExclusiveDiscreteObservation(ULearningAgentsObservationSchema* Schema, const TArray<FName>& ElementNames, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyNamedExclusiveDiscreteObservationFromArrayView(Schema, ElementNames, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyNamedExclusiveDiscreteObservationFromArrayView(ULearningAgentsObservationSchema* Schema, const TArrayView<const FName> ElementNames, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedExclusiveDiscreteObservationFromArrayView: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (ElementNames.Num() < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedExclusiveDiscreteObservationFromArrayView: Invalid Named Exclusive Discrete Observation Size '%i'.", ElementNames.Num());
		return FLearningAgentsObservationSchemaElement();
	}

	if (UE::Learning::Agents::Observation::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedExclusiveDiscreteObservationFromArrayView: Element Names contain duplicates.");
		return FLearningAgentsObservationSchemaElement();
	}

	return { Schema->ObservationSchema.CreateNamedDiscreteExclusive({ ElementNames }, Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyNamedInclusiveDiscreteObservation(ULearningAgentsObservationSchema* Schema, const TArray<FName>& ElementNames, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyNamedInclusiveDiscreteObservationFromArrayView(Schema, ElementNames, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyNamedInclusiveDiscreteObservationFromArrayView(ULearningAgentsObservationSchema* Schema, const TArrayView<const FName> ElementNames, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedInclusiveDiscreteObservationFromArrayView: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (ElementNames.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyNamedInclusiveDiscreteObservationFromArrayView: Specifying zero-sized Named Inclusive Discrete Observation.");
	}

	if (UE::Learning::Agents::Observation::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "SpecifyNamedInclusiveDiscreteObservationFromArrayView: Element Names contain duplicates.");
		return FLearningAgentsObservationSchemaElement();
	}

	return { Schema->ObservationSchema.CreateNamedDiscreteInclusive({ ElementNames }, Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyCountObservation(ULearningAgentsObservationSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyContinuousObservation(Schema, 1, FLearningAgentsObservationNormalizationSettings(), bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyStructObservation(ULearningAgentsObservationSchema* Schema, const TMap<FName, FLearningAgentsObservationSchemaElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyStructObservation: Specifying zero-sized Struct Observation.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<int32, TInlineAllocator<16>> SubElementIndices;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsObservationSchemaElement, TInlineAllocator<16>> SubElements;
	SubElementIndices.Empty(Elements.Num());
	SubElementNames.Empty(Elements.Num());
	SubElements.Empty(Elements.Num());

	int32 Index = 0;
	for (const TPair<FName, FLearningAgentsObservationSchemaElement>& Element : Elements)
	{
		SubElementIndices.Add(Index);
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
		Index++;
	}

	// Sort Elements According to FName

	SubElementIndices.Sort([SubElementNames](const int32 Lhs, const int32 Rhs)
	{
		return SubElementNames[Lhs].ToString().ToLower() < SubElementNames[Rhs].ToString().ToLower();
	});

	TArray<FName, TInlineAllocator<16>> SortedSubElementNames;
	TArray<FLearningAgentsObservationSchemaElement, TInlineAllocator<16>> SortedSubElements;
	SortedSubElementNames.SetNumUninitialized(SubElementNum);
	SortedSubElements.SetNumUninitialized(SubElementNum);
	for (int32 Idx = 0; Idx < SubElementNum; Idx++)
	{
		SortedSubElementNames[Idx] = SubElementNames[SubElementIndices[Idx]];
		SortedSubElements[Idx] = SubElements[SubElementIndices[Idx]];
	}

	return SpecifyStructObservationFromArrayViews(Schema, SortedSubElementNames, SortedSubElements, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyStructObservationFromArrays(ULearningAgentsObservationSchema* Schema, const TArray<FName>& ElementNames, const TArray<FLearningAgentsObservationSchemaElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyStructObservationFromArrayViews(Schema, ElementNames, Elements, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyStructObservationFromArrayViews(ULearningAgentsObservationSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsObservationSchemaElement> Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyStructObservationFromArrayViews: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyStructObservationFromArrayViews: Specifying zero-sized Struct Observation.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "SpecifyStructObservationFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsObservationSchemaElement();
	}

	if (UE::Learning::Agents::Observation::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "SpecifyStructObservationFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsObservationSchemaElement();
	}

	TArray<UE::Learning::Observation::FSchemaElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsObservationSchemaElement& Element : Elements)
	{
		if (!Schema->ObservationSchema.IsValid(Element.SchemaElement))
		{
			UE_LOGF(LogLearning, Error, "SpecifyStructObservationFromArrayViews: Invalid Observation Object.");
			return FLearningAgentsObservationSchemaElement();
		}

		SubElements.Add(Element.SchemaElement);
	}

	return { Schema->ObservationSchema.CreateAnd({ ElementNames, SubElements }, Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyExclusiveUnionObservation(ULearningAgentsObservationSchema* Schema, const TMap<FName, FLearningAgentsObservationSchemaElement>& Elements, const int32 EncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (EncodingSize < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionObservation: Invalid Observation EncodingSize '%i' - must be greater than zero.", EncodingSize);
		return FLearningAgentsObservationSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyExclusiveUnionObservation: Specifying zero-sized Exclusive Union Observation.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<int32, TInlineAllocator<16>> SubElementIndices;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsObservationSchemaElement, TInlineAllocator<16>> SubElements;
	SubElementIndices.Empty(Elements.Num());
	SubElementNames.Empty(Elements.Num());
	SubElements.Empty(Elements.Num());

	int32 Index = 0;
	for (const TPair<FName, FLearningAgentsObservationSchemaElement>& Element : Elements)
	{
		SubElementIndices.Add(Index);
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
		Index++;
	}

	// Sort Elements According to FName

	SubElementIndices.Sort([SubElementNames](const int32 Lhs, const int32 Rhs)
	{
		return SubElementNames[Lhs].ToString().ToLower() < SubElementNames[Rhs].ToString().ToLower();
	});

	TArray<FName, TInlineAllocator<16>> SortedSubElementNames;
	TArray<FLearningAgentsObservationSchemaElement, TInlineAllocator<16>> SortedSubElements;
	SortedSubElementNames.SetNumUninitialized(SubElementNum);
	SortedSubElements.SetNumUninitialized(SubElementNum);
	for (int32 Idx = 0; Idx < SubElementNum; Idx++)
	{
		SortedSubElementNames[Idx] = SubElementNames[SubElementIndices[Idx]];
		SortedSubElements[Idx] = SubElements[SubElementIndices[Idx]];
	}

	return SpecifyExclusiveUnionObservationFromArrayViews(Schema, SortedSubElementNames, SortedSubElements, EncodingSize, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyExclusiveUnionObservationFromArrays(ULearningAgentsObservationSchema* Schema, const TArray<FName>& ElementNames, const TArray<FLearningAgentsObservationSchemaElement>& Elements, const int32 EncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyExclusiveUnionObservationFromArrayViews(Schema, ElementNames, Elements, EncodingSize, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyExclusiveUnionObservationFromArrayViews(ULearningAgentsObservationSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsObservationSchemaElement> Elements, const int32 EncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionObservationFromArrayViews: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (EncodingSize < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionObservationFromArrayViews: Invalid Observation EncodingSize '%i' - must be greater than zero.", EncodingSize);
		return FLearningAgentsObservationSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyExclusiveUnionObservationFromArrayViews: Specifying zero-sized Exclusive Union Observation.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionObservationFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsObservationSchemaElement();
	}

	if (UE::Learning::Agents::Observation::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionObservationFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsObservationSchemaElement();
	}

	TArray<UE::Learning::Observation::FSchemaElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsObservationSchemaElement& Element : Elements)
	{
		if (!Schema->ObservationSchema.IsValid(Element.SchemaElement))
		{
			UE_LOGF(LogLearning, Error, "SpecifyExclusiveUnionObservationFromArrayViews: Invalid Observation Object.");
			return FLearningAgentsObservationSchemaElement();
		}

		SubElements.Add(Element.SchemaElement);
	}

	return { Schema->ObservationSchema.CreateOrExclusive({ ElementNames, SubElements, EncodingSize }, Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyInclusiveUnionObservation(ULearningAgentsObservationSchema* Schema, const TMap<FName, FLearningAgentsObservationSchemaElement>& Elements, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (AttentionEncodingSize < 1 || AttentionHeadNum < 1 || ValueEncodingSize < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionObservation: Invalid Observation Parameters: AttentionEncodingSize: %i, AttentionHeadNum: %i, ValueEncodingSize: %i - must be greater than zero.", AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize);
		return FLearningAgentsObservationSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyInclusiveUnionObservation: Specifying zero-sized Inclusive Union Observation.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<int32, TInlineAllocator<16>> SubElementIndices;
	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsObservationSchemaElement, TInlineAllocator<16>> SubElements;
	SubElementIndices.Empty(SubElementNum);
	SubElementNames.Empty(SubElementNum);
	SubElements.Empty(SubElementNum);

	int32 Index = 0;
	for (const TPair<FName, FLearningAgentsObservationSchemaElement>& Element : Elements)
	{
		SubElementIndices.Add(Index);
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
		Index++;
	}

	// Sort Elements According to FName

	SubElementIndices.Sort([SubElementNames](const int32 Lhs, const int32 Rhs)
	{
		return SubElementNames[Lhs].ToString().ToLower() < SubElementNames[Rhs].ToString().ToLower();
	});

	TArray<FName, TInlineAllocator<16>> SortedSubElementNames;
	TArray<FLearningAgentsObservationSchemaElement, TInlineAllocator<16>> SortedSubElements;
	SortedSubElementNames.SetNumUninitialized(SubElementNum);
	SortedSubElements.SetNumUninitialized(SubElementNum);
	for (int32 Idx = 0; Idx < SubElementNum; Idx++)
	{
		SortedSubElementNames[Idx] = SubElementNames[SubElementIndices[Idx]];
		SortedSubElements[Idx] = SubElements[SubElementIndices[Idx]];
	}

	return SpecifyInclusiveUnionObservationFromArrayViews(Schema, SortedSubElementNames, SortedSubElements, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyInclusiveUnionObservationFromArrays(ULearningAgentsObservationSchema* Schema, const TArray<FName>& ElementNames, const TArray<FLearningAgentsObservationSchemaElement>& Elements, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyInclusiveUnionObservationFromArrayViews(Schema, ElementNames, Elements, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyInclusiveUnionObservationFromArrayViews(ULearningAgentsObservationSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsObservationSchemaElement> Elements, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionObservationFromArrayViews: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (AttentionEncodingSize < 1 || AttentionHeadNum < 1 || ValueEncodingSize < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionObservationFromArrayViews: Invalid Observation Parameters: AttentionEncodingSize: %i, AttentionHeadNum: %i, ValueEncodingSize: %i - must be greater than zero.", AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize);
		return FLearningAgentsObservationSchemaElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyInclusiveUnionObservationFromArrayViews: Specifying zero-sized Inclusive Union Observation.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionObservationFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsObservationSchemaElement();
	}

	if (UE::Learning::Agents::Observation::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionObservationFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsObservationSchemaElement();
	}

	TArray<UE::Learning::Observation::FSchemaElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsObservationSchemaElement& Element : Elements)
	{
		if (!Schema->ObservationSchema.IsValid(Element.SchemaElement))
		{
			UE_LOGF(LogLearning, Error, "SpecifyInclusiveUnionObservationFromArrayViews: Invalid Observation Object.");
			return FLearningAgentsObservationSchemaElement();
		}

		SubElements.Add(Element.SchemaElement);
	}

	return { Schema->ObservationSchema.CreateOrInclusive({ ElementNames, SubElements, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize }, Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyStaticArrayObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const int32 Num, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyStaticArrayObservation: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (Num < 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyStaticArrayObservation: Invalid Observation Array Num %i.", Num);
		return FLearningAgentsObservationSchemaElement();
	}

	if (Num == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifyStaticArrayObservation: Specifying zero-sized Static Array Observation.");
	}

	if (!Schema->ObservationSchema.IsValid(Element.SchemaElement))
	{
		UE_LOGF(LogLearning, Error, "SpecifyStaticArrayObservation: Invalid Observation Object.");
		return FLearningAgentsObservationSchemaElement();
	}

	return { Schema->ObservationSchema.CreateArray({ Element.SchemaElement, Num }, Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifySetObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const int32 MaxNum, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifySetObservation: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (MaxNum < 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifySetObservation: Invalid Observation Set MaxNum %i - must be greater than or equal to zero.", MaxNum);
		return FLearningAgentsObservationSchemaElement();
	}

	if (MaxNum == 0)
	{
		UE_LOGF(LogLearning, Warning, "SpecifySetObservation: Specifying zero-sized Set Observation.");
	}

	if (AttentionEncodingSize < 1 || AttentionHeadNum < 1 || ValueEncodingSize < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifySetObservation: Invalid Observation Parameters: AttentionEncodingSize: %i, AttentionHeadNum: %i, ValueEncodingSize: %i - must be greater than zero.", AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize);
		return FLearningAgentsObservationSchemaElement();
	}

	if (!Schema->ObservationSchema.IsValid(Element.SchemaElement))
	{
		UE_LOGF(LogLearning, Error, "SpecifySetObservation: Invalid Observation Object.");
		return FLearningAgentsObservationSchemaElement();
	}

	return { Schema->ObservationSchema.CreateSet({ Element.SchemaElement, MaxNum, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize }, Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyPairObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element0, const FLearningAgentsObservationSchemaElement Element1, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyStructObservationFromArrayViews(Schema, { TEXT("Key"), TEXT("Value") }, { Element0, Element1 }, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyArrayObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const int32 MaxNum, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifySetObservation(Schema, SpecifyPairObservation(Schema, SpecifyCountObservation(Schema), Element), MaxNum, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyMapObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement KeyElement, const FLearningAgentsObservationSchemaElement ValueElement, const int32 MaxNum, const int32 AttentionEncodingSize, const int32 AttentionHeadNum, const int32 ValueEncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifySetObservation(Schema, SpecifyPairObservation(Schema, KeyElement, ValueElement), MaxNum, AttentionEncodingSize, AttentionHeadNum, ValueEncodingSize, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyEnumObservation(ULearningAgentsObservationSchema* Schema, const UEnum* Enum, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "SpecifyEnumObservation: Enum is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	return SpecifyContinuousObservation(Schema, Enum->NumEnums() - 1, FLearningAgentsObservationNormalizationSettings(), bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyBitmaskObservation(ULearningAgentsObservationSchema* Schema, const UEnum* Enum, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "SpecifyBitmaskObservation: Enum is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_LOGF(LogLearning, Error, "SpecifyBitmaskObservation: Too many values in Enum to use as Bitmask (%i).", Enum->NumEnums() - 1);
		return FLearningAgentsObservationSchemaElement();
	}

	return SpecifyContinuousObservation(Schema, Enum->NumEnums() - 1, FLearningAgentsObservationNormalizationSettings(), bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyOptionalObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const int32 EncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyExclusiveUnionObservationFromArrayViews(Schema, { TEXT("Null"), TEXT("Valid") }, { SpecifyNullObservation(Schema), Element }, EncodingSize, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyEitherObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement A, const FLearningAgentsObservationSchemaElement B, const int32 EncodingSize, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyExclusiveUnionObservationFromArrayViews(Schema, { TEXT("A"), TEXT("B") }, { A, B }, EncodingSize, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyEncodingObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const int32 EncodingSize, const int32 HiddenLayerNum, const ELearningAgentsActivationFunction ActivationFunction, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyEncodingObservation: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}

	if (EncodingSize < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyEncodingObservation: Invalid Observation EncodingSize '%i' - must be greater than zero.", EncodingSize);
		return FLearningAgentsObservationSchemaElement();
	}

	if (HiddenLayerNum < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyEncodingObservation: Invalid Observation HiddenLayerNum '%i' - must be greater than zero.", HiddenLayerNum);
		return FLearningAgentsObservationSchemaElement();
	}

	if (!Schema->ObservationSchema.IsValid(Element.SchemaElement))
	{
		UE_LOGF(LogLearning, Error, "SpecifyEncodingObservation: Invalid Observation Object.");
		return FLearningAgentsObservationSchemaElement();
	}

	return { Schema->ObservationSchema.CreateEncoding({ Element.SchemaElement, EncodingSize, HiddenLayerNum, UE::Learning::Agents::Observation::Private::GetEncodingActivationFunction(ActivationFunction) }, Tag)};
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyBoolObservation(ULearningAgentsObservationSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyContinuousObservation(Schema, 1, FLearningAgentsObservationNormalizationSettings(), bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyConv1dObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const FConv1dObservationParams& Conv1dParams, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv1dObservation: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}
	if (!Schema->ObservationSchema.IsValid(Element.SchemaElement))
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv1dObservation: Invalid Observation Object.");
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv1dParams.InputLength < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv1dObservation: Invalid Observation InputLength '%i' - must be greater than zero.", Conv1dParams.InputLength);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv1dParams.InChannels < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv1dObservation: Invalid Observation InChannels '%i' - must be greater than zero.", Conv1dParams.InChannels);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv1dParams.OutChannels < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv1dObservation: Invalid Observation OutChannels '%i' - must be greater than zero.", Conv1dParams.OutChannels);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv1dParams.KernelSize < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv1dObservation: Invalid Observation KernelSize '%i' - must be greater than zero.", Conv1dParams.KernelSize);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv1dParams.Padding < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv1dObservation: Invalid Observation Padding '%i' - must be greater than zero.", Conv1dParams.Padding);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv1dParams.KernelSize > Conv1dParams.InputLength + 2 * Conv1dParams.Padding)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv1dObservation: Invalid KernelSize '%i' - Kernel size is too large.", Conv1dParams.KernelSize);
		return FLearningAgentsObservationSchemaElement();
	}

	return { Schema->ObservationSchema.CreateConv1d(
	{
		Element.SchemaElement,
		Conv1dParams.InputLength,
		Conv1dParams.InChannels,
		Conv1dParams.OutChannels,
		Conv1dParams.KernelSize,
		Conv1dParams.Padding,
		UE::Learning::Agents::Observation::Private::GetPaddingMode(Conv1dParams.PaddingMode),
		UE::Learning::Agents::Observation::Private::GetEncodingActivationFunction(Conv1dParams.ActivationFunction)
	},
	Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyConv2dObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const FConv2dObservationParams& Conv2dParams, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	if (!Schema)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv2dObservation: Schema is nullptr.");
		return FLearningAgentsObservationSchemaElement();
	}
	if (!Schema->ObservationSchema.IsValid(Element.SchemaElement))
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv2dObservation: Invalid Observation Object.");
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv2dParams.InputHeight < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv2dObservation: Invalid Observation InputHeight '%i' - must be greater than zero.", Conv2dParams.InputHeight);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv2dParams.InputWidth < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv2dObservation: Invalid Observation InputWidth '%i' - must be greater than zero.", Conv2dParams.InputWidth);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv2dParams.InChannels < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv2dObservation: Invalid Observation InChannels '%i' - must be greater than zero.", Conv2dParams.InChannels);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv2dParams.OutChannels < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv2dObservation: Invalid Observation OutChannels '%i' - must be greater than zero.", Conv2dParams.OutChannels);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv2dParams.KernelSize < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv2dObservation: Invalid Observation KernelSize '%i' - must be greater than zero.", Conv2dParams.KernelSize);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv2dParams.Stride < 1)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv2dObservation: Invalid Observation Stride '%i' - must be greater than zero.", Conv2dParams.Stride);
		return FLearningAgentsObservationSchemaElement();
	}
	if (Conv2dParams.Padding < 0)
	{
		UE_LOGF(LogLearning, Error, "SpecifyConv2dObservation: Invalid Observation Padding '%i' - must be greater than zero.", Conv2dParams.Padding);
		return FLearningAgentsObservationSchemaElement();
	}

	return { Schema->ObservationSchema.CreateConv2d(
	{
		Element.SchemaElement,
		Conv2dParams.InputHeight,
		Conv2dParams.InputWidth,
		Conv2dParams.InChannels,
		Conv2dParams.OutChannels,
		Conv2dParams.KernelSize,
		Conv2dParams.Stride,
		Conv2dParams.Padding,
		UE::Learning::Agents::Observation::Private::GetPaddingMode(ELearningAgentsPaddingMode::Zeros),
		UE::Learning::Agents::Observation::Private::GetEncodingActivationFunction(Conv2dParams.ActivationFunction)
	},
	Tag) };
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyFloatObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsFloatObservationNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	FLearningAgentsObservationNormalizationSettings Settings;
	Settings.Scales = { NormalizationSettings.Scale };
	Settings.Offsets = { NormalizationSettings.Offset };
	Settings.NormalizationOption = NormalizationSettings.NormalizationOption;

	return SpecifyContinuousObservation(Schema, 1, Settings, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyLocationObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsVectorObservationNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	FLearningAgentsObservationNormalizationSettings Settings;
	Settings.Scales = { (float)NormalizationSettings.Scales.X, (float)NormalizationSettings.Scales.Y, (float)NormalizationSettings.Scales.Z };
	Settings.Offsets = { (float)NormalizationSettings.Offsets.X, (float)NormalizationSettings.Offsets.Y, (float)NormalizationSettings.Offsets.Z };
	Settings.NormalizationOption = NormalizationSettings.NormalizationOption;

	return SpecifyContinuousObservation(Schema, 3, Settings, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyRotationObservation(ULearningAgentsObservationSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyContinuousObservation(Schema, 6, FLearningAgentsObservationNormalizationSettings(), bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyScaleObservation(ULearningAgentsObservationSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyContinuousObservation(Schema, 3, FLearningAgentsObservationNormalizationSettings(), bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyTransformObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsVectorObservationNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyStructObservationFromArrayViews(Schema,
		{
			TEXT("Location"),
			TEXT("Rotation"),
			TEXT("Scale")
		},
		{
			SpecifyLocationObservation(Schema, NormalizationSettings),
			SpecifyRotationObservation(Schema),
			SpecifyScaleObservation(Schema)
		},
		bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyAngleObservation(ULearningAgentsObservationSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyContinuousObservation(Schema, 2, FLearningAgentsObservationNormalizationSettings(), bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyVelocityObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsVectorObservationNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	FLearningAgentsObservationNormalizationSettings Settings;
	Settings.Scales = { (float)NormalizationSettings.Scales.X, (float)NormalizationSettings.Scales.Y, (float)NormalizationSettings.Scales.Z };
	Settings.Offsets = { (float)NormalizationSettings.Offsets.X, (float)NormalizationSettings.Offsets.Y, (float)NormalizationSettings.Offsets.Z };
	Settings.NormalizationOption = NormalizationSettings.NormalizationOption;

	return SpecifyContinuousObservation(Schema, 3, Settings, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyDirectionObservation(ULearningAgentsObservationSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyContinuousObservation(Schema, 3, FLearningAgentsObservationNormalizationSettings(), bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyLocationAlongSplineObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsVectorObservationNormalizationSettings& NormalizationSettings, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyLocationObservation(Schema, NormalizationSettings, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyProportionAlongSplineObservation(ULearningAgentsObservationSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyExclusiveUnionObservationFromArrayViews(Schema,
		{
			TEXT("Angle"),
			TEXT("Proportion")
		},
		{
			SpecifyAngleObservation(Schema),
			SpecifyFloatObservation(Schema)
		},
		8,
		bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyDirectionAlongSplineObservation(ULearningAgentsObservationSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyDirectionObservation(Schema, bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyProportionAlongRayObservation(ULearningAgentsObservationSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyFloatObservation(Schema, FLearningAgentsFloatObservationNormalizationSettings(), bActive, Tag);
}

FLearningAgentsObservationSchemaElement ULearningAgentsObservations::SpecifyLidarObservation(ULearningAgentsObservationSchema* Schema, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return SpecifyNullObservation(Schema, Tag);
	}

	return SpecifyFloatObservation(Schema, FLearningAgentsFloatObservationNormalizationSettings(), bActive, Tag);
}

void ULearningAgentsObservations::LogObservation(const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element)
{
	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "LogObservation: Object is nullptr.");
		return;
	}

	UE::Learning::Agents::Observation::Private::LogObservation(Object->ObservationObject, Element.ObjectElement, TEXT(""), TEXT(""));
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeNullObservation(ULearningAgentsObservationObject* Object, const FName Tag)
{
	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeNullObservation: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	return { Object->ObservationObject.CreateNull(Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeContinuousObservation(
	ULearningAgentsObservationObject* Object,
	const TArray<float>& Values,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeContinuousObservationFromArrayView(Object, Values, bActive, Tag, VisualLoggerSettings);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeContinuousObservationFromArrayView(
	ULearningAgentsObservationObject* Object,
	const TArrayView<const float> Values,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeContinuousObservationFromArrayView: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (Values.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeContinuousObservationFromArrayView: Creating zero-sized Continuous Observation.");
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nValues: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*UE::Learning::Array::FormatFloat(Values));
	}
#endif

	return { Object->ObservationObject.CreateContinuous({ Values }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeExclusiveDiscreteObservation(
	ULearningAgentsObservationObject* Object, 
	const int32 DiscreteIndex,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveDiscreteObservation: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nIndex: [%i]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			DiscreteIndex);
	}
#endif

	return { Object->ObservationObject.CreateDiscreteExclusive({ DiscreteIndex }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeInclusiveDiscreteObservation(
	ULearningAgentsObservationObject* Object, 
	const TArray<int32>& DiscreteIndices, 
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeInclusiveDiscreteObservationFromArrayView(Object, DiscreteIndices, bActive, Tag, VisualLoggerSettings);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeInclusiveDiscreteObservationFromArrayView(
	ULearningAgentsObservationObject* Object, 
	const TArrayView<const int32> DiscreteIndices, 
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveDiscreteObservation: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (UE::Learning::Agents::Observation::Private::ContainsDuplicates(DiscreteIndices))
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveDiscreteObservationFromArrayView: Indices contain duplicates.");
		return FLearningAgentsObservationObjectElement();
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nIndices: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*UE::Learning::Array::FormatInt32(DiscreteIndices));
	}
#endif

	return { Object->ObservationObject.CreateDiscreteInclusive({ DiscreteIndices }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeNamedExclusiveDiscreteObservation(
	ULearningAgentsObservationObject* Object,
	const FName ElementName,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeNamedExclusiveDiscreteObservation: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nName: [%s]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*ElementName.ToString());
	}
#endif

	return { Object->ObservationObject.CreateNamedDiscreteExclusive({ ElementName }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeNamedInclusiveDiscreteObservation(
	ULearningAgentsObservationObject* Object,
	const TArray<FName>& ElementNames,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeNamedInclusiveDiscreteObservationFromArrayView(Object, ElementNames, bActive, Tag, VisualLoggerSettings);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeNamedInclusiveDiscreteObservationFromArrayView(
	ULearningAgentsObservationObject* Object,
	const TArrayView<const FName> ElementNames,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeNamedInclusiveDiscreteObservationFromArrayView: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (UE::Learning::Agents::Observation::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "MakeNamedInclusiveDiscreteObservationFromArrayView: Names contain duplicates.");
		return FLearningAgentsObservationObjectElement();
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nNames: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*UE::Learning::Agents::Observation::Private::FormatNames(ElementNames, 256));
	}
#endif

	return { Object->ObservationObject.CreateNamedDiscreteInclusive({ ElementNames }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeCountObservation(
	ULearningAgentsObservationObject* Object, 
	const int32 Num, 
	const int32 MaxNum, 
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (MaxNum == 0)
	{
		UE_LOGF(LogLearning, Error, "MakeCountObservation: MaxNum must not be zero for Count Observation.");
		return FLearningAgentsObservationObjectElement();
	}

	const float Encoded = (float)Num / (float)MaxNum;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nNum: [%i]\nMax Num: [%i]\nEncoded: [%6.2f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Num,
			MaxNum,
			Encoded);
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, { Encoded }, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeStructObservation(ULearningAgentsObservationObject* Object, const TMap<FName, FLearningAgentsObservationObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeStructObservation: Creating zero-sized Struct Observation.");
	}

	const int32 SubElementNum = Elements.Num();

	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsObservationObjectElement, TInlineAllocator<16>> SubElements;
	SubElementNames.Empty(SubElementNum);
	SubElements.Empty(SubElementNum);

	for (const TPair<FName, FLearningAgentsObservationObjectElement>& Element : Elements)
	{
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
	}

	return MakeStructObservationFromArrayViews(Object, SubElementNames, SubElements, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeStructObservationFromArrays(ULearningAgentsObservationObject* Object, const TArray<FName>& ElementNames, const TArray<FLearningAgentsObservationObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeStructObservationFromArrayViews(Object, ElementNames, Elements, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeStructObservationFromArrayViews(ULearningAgentsObservationObject* Object, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsObservationObjectElement> Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeStructObservationFromArrayViews: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeStructObservationFromArrayViews: Creating zero-sized Struct Observation.");
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "MakeStructObservationFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsObservationObjectElement();
	}

	if (UE::Learning::Agents::Observation::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "MakeStructObservationFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsObservationObjectElement();
	}

	TArray<UE::Learning::Observation::FObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsObservationObjectElement& Element : Elements)
	{
		if (!Object->ObservationObject.IsValid(Element.ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeStructObservationFromArrayViews: Invalid Observation Object.");
			return FLearningAgentsObservationObjectElement();
		}

		SubElements.Add(Element.ObjectElement);
	}

	return { Object->ObservationObject.CreateAnd({ ElementNames, SubElements }, Tag)};
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeExclusiveUnionObservation(ULearningAgentsObservationObject* Object, const FName ElementName, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveUnionObservation: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "MakeExclusiveUnionObservation: Invalid Observation Object.");
		return FLearningAgentsObservationObjectElement();
	}

	return { Object->ObservationObject.CreateOrExclusive({ ElementName, Element.ObjectElement }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeInclusiveUnionObservation(ULearningAgentsObservationObject* Object, const TMap<FName, FLearningAgentsObservationObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const int32 SubElementNum = Elements.Num();

	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsObservationObjectElement, TInlineAllocator<16>> SubElements;
	SubElementNames.Empty(SubElementNum);
	SubElements.Empty(SubElementNum);

	for (const TPair<FName, FLearningAgentsObservationObjectElement>& Element : Elements)
	{
		SubElementNames.Add(Element.Key);
		SubElements.Add(Element.Value);
	}

	return MakeInclusiveUnionObservationFromArrayViews(Object, SubElementNames, SubElements, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeInclusiveUnionObservationFromArrays(ULearningAgentsObservationObject* Object, const TArray<FName>& ElementNames, const TArray<FLearningAgentsObservationObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeInclusiveUnionObservationFromArrayViews(Object, ElementNames, Elements, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeInclusiveUnionObservationFromArrayViews(ULearningAgentsObservationObject* Object, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsObservationObjectElement> Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveUnionObservationFromArrayViews: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (Elements.Num() != ElementNames.Num())
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveUnionObservationFromArrayViews: Number of elements (%i) must match number of names (%i).", Elements.Num(), ElementNames.Num());
		return FLearningAgentsObservationObjectElement();
	}

	if (UE::Learning::Agents::Observation::Private::ContainsDuplicates(ElementNames))
	{
		UE_LOGF(LogLearning, Error, "MakeInclusiveUnionObservationFromArrayViews: Element Names contain duplicates.");
		return FLearningAgentsObservationObjectElement();
	}

	TArray<UE::Learning::Observation::FObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsObservationObjectElement& Element : Elements)
	{
		if (!Object->ObservationObject.IsValid(Element.ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeInclusiveUnionObservationFromArrayViews: Invalid Observation Object.");
			return FLearningAgentsObservationObjectElement();
		}

		SubElements.Add(Element.ObjectElement);
	}

	return { Object->ObservationObject.CreateOrInclusive({ ElementNames, SubElements }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeStaticArrayObservation(ULearningAgentsObservationObject* Object, const TArray<FLearningAgentsObservationObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeStaticArrayObservationFromArrayView(Object, Elements, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeStaticArrayObservationFromArrayView(ULearningAgentsObservationObject* Object, const TArrayView<const FLearningAgentsObservationObjectElement> Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeStaticArrayObservationFromArrayView: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "MakeStaticArrayObservationFromArrayView: Creating zero-sized Static Array Observation.");
	}

	TArray<UE::Learning::Observation::FObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsObservationObjectElement& Element : Elements)
	{
		if (!Object->ObservationObject.IsValid(Element.ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeStaticArrayObservationFromArrayView: Invalid Observation Object.");
			return FLearningAgentsObservationObjectElement();
		}

		SubElements.Add(Element.ObjectElement);
	}

	return { Object->ObservationObject.CreateArray({ SubElements }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeSetObservation(ULearningAgentsObservationObject* Object, const TSet<FLearningAgentsObservationObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeSetObservation: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	TArray<UE::Learning::Observation::FObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsObservationObjectElement& Element : Elements)
	{
		if (!Object->ObservationObject.IsValid(Element.ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeSetObservation: Invalid Observation Object.");
			return FLearningAgentsObservationObjectElement();
		}

		SubElements.Add(Element.ObjectElement);
	}

	return { Object->ObservationObject.CreateSet({ SubElements }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeSetObservationFromArray(ULearningAgentsObservationObject* Object, const TArray<FLearningAgentsObservationObjectElement>& Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeSetObservationFromArrayView(Object, Elements, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeSetObservationFromArrayView(ULearningAgentsObservationObject* Object, const TArrayView<const FLearningAgentsObservationObjectElement> Elements, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeSetObservationFromArrayView: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	TArray<UE::Learning::Observation::FObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (const FLearningAgentsObservationObjectElement& Element : Elements)
	{
		if (!Object->ObservationObject.IsValid(Element.ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeSetObservationFromArrayView: Invalid Observation Object.");
			return FLearningAgentsObservationObjectElement();
		}

		SubElements.Add(Element.ObjectElement);
	}

	return { Object->ObservationObject.CreateSet({ SubElements }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakePairObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Key, const FLearningAgentsObservationObjectElement Value, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeStructObservationFromArrayViews(Object, { TEXT("Key"), TEXT("Value") }, { Key, Value }, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeArrayObservation(ULearningAgentsObservationObject* Object, const TArray<FLearningAgentsObservationObjectElement>& Elements, const int32 MaxNum, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeArrayObservationFromArrayView(Object, Elements, MaxNum, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeArrayObservationFromArrayView(ULearningAgentsObservationObject* Object, const TArrayView<const FLearningAgentsObservationObjectElement> Elements, const int32 MaxNum, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeArrayObservationFromArrayView: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	TArray<FLearningAgentsObservationObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Elements.Num());

	for (int32 ElementIdx = 0; ElementIdx < Elements.Num(); ElementIdx++)
	{
		const FLearningAgentsObservationObjectElement Element = Elements[ElementIdx];

		if (!Object->ObservationObject.IsValid(Element.ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeArrayObservationFromArrayView: Invalid Observation Object.");
			return FLearningAgentsObservationObjectElement();
		}

		SubElements.Add(MakePairObservation(Object, MakeCountObservation(Object, ElementIdx, MaxNum), Element));
	}

	return MakeSetObservationFromArrayView(Object, SubElements, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeMapObservation(ULearningAgentsObservationObject* Object, const TMap<FLearningAgentsObservationObjectElement, FLearningAgentsObservationObjectElement>& Map, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeMapObservation: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	TArray<FLearningAgentsObservationObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Map.Num());

	for (TPair<FLearningAgentsObservationObjectElement, FLearningAgentsObservationObjectElement> Item : Map)
	{
		if (!Object->ObservationObject.IsValid(Item.Key.ObjectElement) || !Object->ObservationObject.IsValid(Item.Value.ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeMapObservation: Invalid Observation Object.");
			return FLearningAgentsObservationObjectElement();
		}

		SubElements.Add(MakePairObservation(Object, Item.Key, Item.Value));
	}

	return MakeSetObservationFromArrayView(Object, SubElements, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeMapObservationFromArrays(ULearningAgentsObservationObject* Object, const TArray<FLearningAgentsObservationObjectElement>& Keys, const TArray<FLearningAgentsObservationObjectElement>& Values, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeMapObservationFromArrayViews(Object, Keys, Values, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeMapObservationFromArrayViews(ULearningAgentsObservationObject* Object, const TArrayView<const FLearningAgentsObservationObjectElement> Keys, const TArrayView<const FLearningAgentsObservationObjectElement> Values, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeMapObservationFromArrayViews: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (Keys.Num() != Values.Num())
	{
		UE_LOGF(LogLearning, Error, "MakeMapObservationFromArrayViews: Number of keys (%i) must match number of values (%i).", Keys.Num(), Values.Num());
		return FLearningAgentsObservationObjectElement();
	}

	TArray<FLearningAgentsObservationObjectElement, TInlineAllocator<16>> SubElements;
	SubElements.Empty(Keys.Num());

	for (int32 ElementIdx = 0; ElementIdx < Keys.Num(); ElementIdx++)
	{
		if (!Object->ObservationObject.IsValid(Keys[ElementIdx].ObjectElement) || 
			!Object->ObservationObject.IsValid(Values[ElementIdx].ObjectElement))
		{
			UE_LOGF(LogLearning, Error, "MakeMapObservationFromArrayViews: Invalid Observation Object.");
			return FLearningAgentsObservationObjectElement();
		}

		SubElements.Add(MakePairObservation(Object, Keys[ElementIdx], Values[ElementIdx]));
	}

	return MakeSetObservationFromArrayView(Object, SubElements, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeEnumObservation(
	ULearningAgentsObservationObject* Object, 
	const UEnum* Enum, 
	const uint8 EnumValue, 
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "MakeEnumObservation: Enum is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	const int32 EnumValueIndex = Enum->GetIndexByValue(EnumValue);

	if (EnumValueIndex == INDEX_NONE || EnumValueIndex < 0 || EnumValueIndex >= Enum->NumEnums() - 1)
	{
		UE_LOGF(LogLearning, Error, "MakeEnumObservation: EnumValue %i not valid for Enum '%ls'.", EnumValue , *Enum->GetName());
		return FLearningAgentsObservationObjectElement();
	}

	TArray<float, TInlineAllocator<32>> OneHot;
	OneHot.Init(0.0f, Enum->NumEnums() - 1);
	OneHot[EnumValueIndex] = 1.0f;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nEnum: %s\nSize: [%i]\nValue: [%s]\nIndex: [%i]\nEncoded: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*Enum->GetName(),
			Enum->NumEnums() - 1,
			*Enum->GetDisplayNameTextByValue(EnumValue).ToString(),
			EnumValueIndex,
			*UE::Learning::Array::FormatFloat(OneHot, Enum->NumEnums() - 1));
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, OneHot, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeBitmaskObservation(
	ULearningAgentsObservationObject* Object, 
	const UEnum* Enum, 
	const int32 BitmaskValue, 
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "MakeBitmaskObservation: Enum is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_LOGF(LogLearning, Error, "MakeBitmaskObservation: Too many values in Enum to use as Bitmask (%i).", Enum->NumEnums() - 1);
		return FLearningAgentsObservationObjectElement();
	}

	TArray<float, TInlineAllocator<32>> OneHot;
	OneHot.Init(0.0f, Enum->NumEnums() - 1);

	for (int32 EnumIdx = 0; EnumIdx < Enum->NumEnums() - 1; EnumIdx++)
	{
		if (BitmaskValue & (1 << EnumIdx))
		{
			OneHot[EnumIdx] = 1.0f;
		}
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		FString ValuesString;
		FString IndicesString;

		for (int32 EnumIdx = 0; EnumIdx < Enum->NumEnums() - 1; EnumIdx++)
		{
			if (BitmaskValue & (1 << EnumIdx))
			{
				ValuesString += Enum->GetDisplayNameTextByIndex(EnumIdx).ToString() + TEXT(" ");
				IndicesString += FString::FromInt(EnumIdx) + TEXT(" ");
			}
		}

		ValuesString = ValuesString.TrimEnd();
		IndicesString = IndicesString.TrimEnd();

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nEnum: %s\nSize: [%i]\nValues: [%s]\nIndices: [%s]\nEncoded: %s"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			*Enum->GetName(),
			Enum->NumEnums() - 1,
			*ValuesString,
			*IndicesString,
			*UE::Learning::Array::FormatFloat(OneHot, Enum->NumEnums() - 1));
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, OneHot, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeOptionalObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const ELearningAgentsOptionalObservation Option, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeExclusiveUnionObservation(
		Object,
		Option == ELearningAgentsOptionalObservation::Null ? TEXT("Null") : TEXT("Valid"),
		Option == ELearningAgentsOptionalObservation::Null ? MakeNullObservation(Object) : Element,
		bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeOptionalNullObservation(ULearningAgentsObservationObject* Object, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeExclusiveUnionObservation(Object, TEXT("Null"), MakeNullObservation(Object), bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeOptionalValidObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeExclusiveUnionObservation(Object, TEXT("Valid"), Element, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeEitherObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const ELearningAgentsEitherObservation Either, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeExclusiveUnionObservation(Object, Either == ELearningAgentsEitherObservation::A ? TEXT("A") : TEXT("B"), Element, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeEitherAObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement A, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeExclusiveUnionObservation(Object, TEXT("A"), A, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeEitherBObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement B, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeExclusiveUnionObservation(Object, TEXT("B"), B, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeEncodingObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeEncodingObservation: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "MakeEncodingObservation: Invalid Observation Object.");
		return FLearningAgentsObservationObjectElement();
	}

	return { Object->ObservationObject.CreateEncoding({ Element.ObjectElement }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeBoolObservation(
	ULearningAgentsObservationObject* Object,
	const bool bValue,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const float EncodedBool = bValue ? 1.0f : -1.0f;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nValue: [%s]\nEncoded: [% 6.2f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			bValue ? TEXT("true") : TEXT("false"),
			EncodedBool);
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, { EncodedBool }, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeConv1dObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeConv1dObservation: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "MakeConv1dObservation: Invalid Observation Object.");
		return FLearningAgentsObservationObjectElement();
	}

	return { Object->ObservationObject.CreateConv1d({ Element.ObjectElement }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeConv2dObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "MakeConv2dObservation: Object is nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "MakeConv2dObservation: Invalid Observation Object.");
		return FLearningAgentsObservationObjectElement();
	}

	return { Object->ObservationObject.CreateConv2d({ Element.ObjectElement }, Tag) };
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeFloatObservation(
	ULearningAgentsObservationObject* Object,
	const float Value,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nValue: [% 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Value);
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, { Value }, bActive, Tag, FLearningAgentsObservationVisualLoggerSettings());
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeLocationObservation(
	ULearningAgentsObservationObject* Object,
	const FVector Location,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const FVector LocalLocation = RelativeTransform.InverseTransformPosition(Location);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			Location,
			10,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			RelativeTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocation: [% 6.1f % 6.1f % 6.1f]\nLocal Location: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Location.X, Location.Y, Location.Z,
			LocalLocation.X, LocalLocation.Y, LocalLocation.Z);
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, {
		(float)LocalLocation.X,
		(float)LocalLocation.Y,
		(float)LocalLocation.Z,
		}, bActive, Tag, FLearningAgentsObservationVisualLoggerSettings());
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeRotationObservation(
	ULearningAgentsObservationObject* Object,
	const FRotator Rotation,
	const FRotator RelativeRotation,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	// Visual logging is handled in the MakeRotationObservationFromQuat

	return MakeRotationObservationFromQuat(
		Object,
		FQuat::MakeFromRotator(Rotation), FQuat::MakeFromRotator(RelativeRotation), bActive, Tag,
		VisualLoggerSettings);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeRotationObservationFromQuat(
	ULearningAgentsObservationObject* Object,
	const FQuat Rotation,
	const FQuat RelativeRotation,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const FQuat LocalRotation = RelativeRotation.Inverse() * Rotation;
	const FVector LocalAxisForward = LocalRotation.GetForwardVector();
	const FVector LocalAxisRight = LocalRotation.GetRightVector();

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			VisualLoggerSettings.DebugLocation,
			LocalRotation.Rotator(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nRotation: [% 6.1f % 6.1f % 6.1f % 6.1f]\nLocal Rotation: [% 6.1f % 6.1f % 6.1f % 6.1f]\nEncoded Forward: [% 6.2f % 6.2f % 6.2f]\nEncoded Right: [% 6.2f % 6.2f % 6.2f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Rotation.W, Rotation.X, Rotation.Y, Rotation.Z,
			LocalRotation.W, LocalRotation.X, LocalRotation.Y, LocalRotation.Z,
			LocalAxisForward.X, LocalAxisForward.Y, LocalAxisForward.Z,
			LocalAxisRight.X, LocalAxisRight.Y, LocalAxisRight.Z);
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, {
		(float)LocalAxisForward.X,
		(float)LocalAxisForward.Y,
		(float)LocalAxisForward.Z,
		(float)LocalAxisRight.X,
		(float)LocalAxisRight.Y,
		(float)LocalAxisRight.Z,
		}, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeScaleObservation(
	ULearningAgentsObservationObject* Object,
	const FVector Scale,
	const FVector RelativeScale,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const FVector LocalLogScale = 
		UE::Learning::Agents::Observation::Private::VectorLogSafe(Scale) - 
		UE::Learning::Agents::Observation::Private::VectorLogSafe(RelativeScale);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nScale: [% 6.1f % 6.1f % 6.1f]\nEncoded: [% 6.2f % 6.2f % 6.2f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Scale.X, Scale.Y, Scale.Z,
			LocalLogScale.X, LocalLogScale.Y, LocalLogScale.Z);
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, {
		(float)LocalLogScale.X,
		(float)LocalLogScale.Y,
		(float)LocalLogScale.Z,
		}, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeTransformObservation(
	ULearningAgentsObservationObject* Object,
	const FTransform Transform,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const FTransform LocalTransform = Transform * RelativeTransform.Inverse();
	
#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		const FVector LocalLocation = LocalTransform.GetLocation();
		const FRotator LocalRotation = LocalTransform.Rotator();
		const FVector LocalScale = LocalTransform.GetScale3D();

		const FVector Location = Transform.GetLocation();
		const FRotator Rotation = Transform.Rotator();
		const FVector Scale = Transform.GetScale3D();

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			Location,
			Rotation,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocation: [% 6.1f % 6.1f % 6.1f]\nLocal Location: [% 6.1f % 6.1f % 6.1f]\nRotation: [% 6.1f % 6.1f % 6.1f]\nLocal Rotation: [% 6.1f % 6.1f % 6.1f]\nScale: [% 6.1f % 6.1f % 6.1f]\nLocal Scale: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Location.X, Location.Y, Location.Z,
			LocalLocation.X, LocalLocation.Y, LocalLocation.Z,
			Rotation.Roll, Rotation.Pitch, Rotation.Yaw,
			LocalRotation.Roll, LocalRotation.Pitch, LocalRotation.Yaw,
			Scale.X, Scale.Y, Scale.Z,
			LocalScale.X, LocalScale.Y, LocalScale.Z);
	}
#endif

	return MakeStructObservationFromArrayViews(Object,
		{
			TEXT("Location"),
			TEXT("Rotation"),
			TEXT("Scale")
		},
		{
			MakeLocationObservation(Object, LocalTransform.GetLocation(), FTransform::Identity),
			MakeRotationObservationFromQuat(Object, LocalTransform.GetRotation(), FQuat::Identity),
			MakeScaleObservation(Object, LocalTransform.GetScale3D(), FVector::OneVector)
		},
		bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeAngleObservation(
	ULearningAgentsObservationObject* Object, 
	const float Angle, 
	const float RelativeAngle, 
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const float LocalAngle = FMath::FindDeltaAngleDegrees(RelativeAngle, Angle);
	const float EncodedX = FMath::Sin(FMath::DegreesToRadians(LocalAngle));
	const float EncodedY = FMath::Cos(FMath::DegreesToRadians(LocalAngle));

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ANGLE_DEGREES(VisualLoggerObject, LogLearning, Display,
			Angle,
			RelativeAngle,
			VisualLoggerSettings.Location,
			10.0f,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nAngle: [% 6.1f]\nRelative Angle: [% 6.1f]\nLocal Angle: [% 6.1f]\nEncoded: [% 6.2f % 6.2f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Angle,
			RelativeAngle,
			LocalAngle,
			EncodedX, EncodedY);
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, { EncodedX, EncodedY }, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeAngleObservationRadians(
	ULearningAgentsObservationObject* Object, 
	const float Angle, 
	const float RelativeAngle, 
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	return MakeAngleObservation(
		Object, 
		FMath::RadiansToDegrees(Angle), 
		FMath::RadiansToDegrees(RelativeAngle), 
		bActive,
		Tag, 
		VisualLoggerSettings);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeVelocityObservation(
	ULearningAgentsObservationObject* Object,
	const FVector Velocity,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const FVector LocalVelocity = RelativeTransform.InverseTransformVectorNoScale(Velocity);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ARROW(VisualLoggerObject, LogLearning, Display,
			VisualLoggerSettings.DebugLocation,
			VisualLoggerSettings.DebugLocation + Velocity,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			RelativeTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nVelocity: [% 6.1f % 6.1f % 6.1f]\nLocal Velocity: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Velocity.X, Velocity.Y, Velocity.Z,
			LocalVelocity.X, LocalVelocity.Y, LocalVelocity.Z);
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, { (float)LocalVelocity.X, (float)LocalVelocity.Y, (float)LocalVelocity.Z }, bActive, Tag, FLearningAgentsObservationVisualLoggerSettings());
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeDirectionObservation(
	ULearningAgentsObservationObject* Object,
	const FVector Direction,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const FVector LocalDirection = RelativeTransform.InverseTransformVectorNoScale(Direction).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ARROW(VisualLoggerObject, LogLearning, Display,
			VisualLoggerSettings.DebugLocation,
			VisualLoggerSettings.DebugLocation + VisualLoggerSettings.ArrowLength * Direction,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RelativeTransform.GetTranslation(),
			RelativeTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nDirection: [% 6.1f % 6.1f % 6.1f]\nLocal Direction: [% 6.1f % 6.1f % 6.1f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			Direction.X, Direction.Y, Direction.Z,
			LocalDirection.X, LocalDirection.Y, LocalDirection.Z);
	}
#endif

	return MakeContinuousObservationFromArrayView(Object, {
		(float)LocalDirection.X,
		(float)LocalDirection.Y,
		(float)LocalDirection.Z,
		}, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeLocationAlongSplineObservation(
	ULearningAgentsObservationObject* Object,
	const USplineComponent* SplineComponent,
	const float DistanceAlongSpline,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!SplineComponent)
	{
		UE_LOGF(LogLearning, Error, "MakeLocationAlongSplineObservation: SplineComponent was nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	const float TotalDistance = SplineComponent->GetSplineLength();
	const float LoopedDistance = SplineComponent->IsClosedLoop() ? FMath::Wrap(DistanceAlongSpline, 0.0f, TotalDistance) : FMath::Clamp(DistanceAlongSpline, 0.0f, TotalDistance);

	return MakeLocationObservation(
		Object,
		SplineComponent->GetLocationAtDistanceAlongSpline(LoopedDistance, ESplineCoordinateSpace::World),
		RelativeTransform,
		bActive,
		Tag,
		VisualLoggerSettings);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeProportionAlongSplineObservation(
	ULearningAgentsObservationObject* Object, 
	const USplineComponent* SplineComponent, 
	const float DistanceAlongSpline, 
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!SplineComponent)
	{
		UE_LOGF(LogLearning, Error, "MakeProportionAlongSplineObservation: SplineComponent was nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	if (SplineComponent->IsClosedLoop())
	{
		const float TotalDistance = SplineComponent->GetSplineLength();
		const float WrapDistance = FMath::Wrap(DistanceAlongSpline, 0.0f, TotalDistance);
		const float Proportion = WrapDistance / FMath::Max(TotalDistance, UE_SMALL_NUMBER);
		const float Angle = FMath::Wrap(UE_TWO_PI * Proportion, -UE_PI, UE_PI);
		
		return MakeExclusiveUnionObservation(Object, TEXT("Angle"), MakeAngleObservationRadians(
			Object, 
			Angle,
			0.0f,
			bActive,
			TEXT("AngleObservation"), VisualLoggerSettings),
			bActive, Tag);
	}
	else
	{
		const float Proportion = FMath::Clamp(DistanceAlongSpline / FMath::Max(SplineComponent->GetSplineLength(), UE_SMALL_NUMBER), 0.0f, 1.0f);

		return MakeExclusiveUnionObservation(Object, TEXT("Proportion"), MakeFloatObservation(
			Object, 
			Proportion, 
			bActive,
			TEXT("FloatObservation"), VisualLoggerSettings),
			bActive, Tag);
	}
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeDirectionAlongSplineObservation(
	ULearningAgentsObservationObject* Object,
	const USplineComponent* SplineComponent,
	const float DistanceAlongSpline,
	const FTransform RelativeTransform,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	if (!SplineComponent)
	{
		UE_LOGF(LogLearning, Error, "MakeDirectionAlongSplineObservation: SplineComponent was nullptr.");
		return FLearningAgentsObservationObjectElement();
	}

	const float TotalDistance = SplineComponent->GetSplineLength();
	const float LoopedDistance = SplineComponent->IsClosedLoop() ? FMath::Wrap(DistanceAlongSpline, 0.0f, TotalDistance) : FMath::Clamp(DistanceAlongSpline, 0.0f, TotalDistance);

	FLearningAgentsObservationVisualLoggerSettings DirectionVisualLoggerSettings = VisualLoggerSettings;
#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	DirectionVisualLoggerSettings.DebugLocation = (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener) ? SplineComponent->GetLocationAtDistanceAlongSpline(LoopedDistance, ESplineCoordinateSpace::World) : FVector::ZeroVector;

#else
	DirectionVisualLoggerSettings.DebugLocation = FVector::ZeroVector;
#endif
	
	return MakeDirectionObservation(
		Object, 
		SplineComponent->GetDirectionAtDistanceAlongSpline(LoopedDistance, ESplineCoordinateSpace::World), 
		RelativeTransform, 
		bActive,
		Tag,
		DirectionVisualLoggerSettings);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeProportionAlongRayObservation(
	ULearningAgentsObservationObject* Object, 
	const FVector RayStart, 
	const FVector RayEnd, 
	const FTransform RayTransform, 
	const ECollisionChannel CollisionChannel,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const FVector RayStartWorld = RayTransform.TransformPosition(RayStart);
	const FVector RayEndWorld = RayTransform.TransformPosition(RayEnd);
	const float RayDistance = FVector::Distance(RayStartWorld, RayEndWorld);

	FCollisionObjectQueryParams ObjectQueryParams;
	ObjectQueryParams.AddObjectTypesToQuery(CollisionChannel);

	FHitResult TraceHit;
	const bool bHit = Object->GetWorld()->LineTraceSingleByObjectType(TraceHit, RayStartWorld, RayEndWorld, ObjectQueryParams);

	const float Encoded = bHit ? 1.0f - TraceHit.Time : 0.0f;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			RayStartWorld,
			RayEndWorld,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		if (bHit)
		{
			UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
				RayStartWorld + TraceHit.Time * (RayEndWorld - RayStartWorld),
				10.f,
				VisualLoggerSettings.Color.ToFColor(true),
				TEXT(""));
		}

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RayTransform.GetTranslation(),
			RayTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nRay Start: [% 6.1f % 6.1f % 6.1f]\nRay End: [% 6.1f % 6.1f % 6.1f]\nRay Start World: [% 6.1f % 6.1f % 6.1f]\nRay End World: [% 6.1f % 6.1f % 6.1f]\nCollision Channel: [%s]\nHit: [%s]\nEncoded: [% 6.2f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			RayStart.X, RayStart.Y, RayStart.Z,
			RayEnd.X, RayEnd.Y, RayEnd.Z,
			RayStartWorld.X, RayStartWorld.Y, RayStartWorld.Z,
			RayEndWorld.X, RayEndWorld.Y, RayEndWorld.Z,
			*StaticEnum<ECollisionChannel>()->GetNameStringByValue(CollisionChannel),
			bHit ? TEXT("true") : TEXT("false"),
			Encoded);
	}
#endif

	return MakeFloatObservation(Object, Encoded, bActive, Tag);
}

FLearningAgentsObservationObjectElement ULearningAgentsObservations::MakeLidarObservation(
	ULearningAgentsObservationObject* Object, 
	const FVector RayStart,
	const FVector RayEnd,
	const FTransform RayTransform, 
	const ECollisionChannel CollisionChannel, 
	const AActor* ActorToIgnore,
	const bool bActive,
	const FName Tag,
	const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings)
{
	if (!bActive)
	{
		return MakeNullObservation(Object, Tag);
	}

	const FVector RayStartWorld = RayTransform.TransformPosition(RayStart);
	const FVector RayEndWorld = RayTransform.TransformPosition(RayEnd);
	const float RayRange = FVector::Dist(RayEndWorld, RayStartWorld);

	if (RayRange <= 0)
	{
		UE_LOGF(LogLearning, Error, "MakeLiDARObservation: Ray range cannot be 0.");
		return FLearningAgentsObservationObjectElement();
	}

	FCollisionObjectQueryParams ObjectQueryParams;
	ObjectQueryParams.AddObjectTypesToQuery(CollisionChannel);

	FCollisionQueryParams TraceParams;
	TraceParams.AddIgnoredActor(ActorToIgnore);

	FHitResult TraceHit;
	const bool bHit = Object->GetWorld()->LineTraceSingleByObjectType(TraceHit, RayStartWorld, RayEndWorld, ObjectQueryParams, TraceParams);

	const float Encoded = bHit ? FMath::Clamp(1.0f - TraceHit.Distance / RayRange, 0.0f, 1.0f) : 0.0f;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (VisualLoggerSettings.bEnabled && VisualLoggerSettings.Listener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerSettings.Listener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			RayStartWorld,
			RayEndWorld,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		if (bHit)
		{
			UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
				TraceHit.Location,
				10.f,
				VisualLoggerSettings.Color.ToFColor(true),
				TEXT(""));
		}

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(VisualLoggerObject, LogLearning, Display,
			RayTransform.GetTranslation(),
			RayTransform.GetRotation(),
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerSettings.Location,
			VisualLoggerSettings.Color.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nRay Start: [% 6.1f % 6.1f % 6.1f]\nRay End: [% 6.1f % 6.1f % 6.1f]\nRay Start World: [% 6.1f % 6.1f % 6.1f]\nRay End World: [% 6.1f % 6.1f % 6.1f]\nCollision Channel: [%s]\nHit: [%s]\nEncoded: [% 6.2f]"),
			*VisualLoggerSettings.Listener->GetName(),
			*Tag.ToString(),
			VisualLoggerSettings.AgentId,
			RayStart.X, RayStart.Y, RayStart.Z,
			RayEnd.X, RayEnd.Y, RayEnd.Z,
			RayStartWorld.X, RayStartWorld.Y, RayStartWorld.Z,
			RayEndWorld.X, RayEndWorld.Y, RayEndWorld.Z,
			*StaticEnum<ECollisionChannel>()->GetNameStringByValue(CollisionChannel),
			bHit ? TEXT("true") : TEXT("false"),
			Encoded);
	}
#endif

	return MakeFloatObservation(Object, Encoded, bActive, Tag);
}

bool ULearningAgentsObservations::GetNullObservation(const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FName Tag)
{
	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetNullObservation: Object is nullptr.");
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetNullObservation: Invalid Observation Object.");
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetNullObservation: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Null)
	{
		UE_LOGF(LogLearning, Error, "GetNullObservation: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Null));
		return false;
	}

	return true;
}

bool ULearningAgentsObservations::GetContinuousObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetContinuousObservationNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetContinuousObservationNum: Invalid Observation Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetContinuousObservationNum: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Continuous)
	{
		UE_LOGF(LogLearning, Error, "GetContinuousObservationNum: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Continuous));
		OutNum = 0;
		return false;
	}

	OutNum = Object->ObservationObject.GetContinuous(Element.ObjectElement).Values.Num();
	return true;
}

bool ULearningAgentsObservations::GetContinuousObservation(TArray<float>& OutValues, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutValueNum = 0;
	if (!GetContinuousObservationNum(OutValueNum, Object, Element, bActive, Tag))
	{
		OutValues.Empty();
		return false;
	}

	OutValues.SetNumUninitialized(OutValueNum);

	if (!GetContinuousObservationToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutValues.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsObservations::GetContinuousObservationToArrayView(TArrayView<float> OutValues, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetContinuousObservationToArrayView: Object is nullptr.");
		UE::Learning::Array::Zero<1, float>(OutValues);
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetContinuousObservationToArrayView: Invalid Observation Object.");
		UE::Learning::Array::Zero<1, float>(OutValues);
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetContinuousObservationToArrayView: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Continuous)
	{
		UE_LOGF(LogLearning, Error, "GetContinuousObservationToArrayView: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Continuous));
		UE::Learning::Array::Zero<1, float>(OutValues);
		return false;
	}

	const TArrayView<const float> Values = Object->ObservationObject.GetContinuous(Element.ObjectElement).Values;

	if (Values.Num() != OutValues.Num())
	{
		UE_LOGF(LogLearning, Error, "GetContinuousObservationToArrayView: Observation '%ls' size does not match. Observation is '%i' values but asked for '%i'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			Values.Num(), OutValues.Num());
		UE::Learning::Array::Zero<1, float>(OutValues);
		return false;
	}

	UE::Learning::Array::Copy<1, float>(OutValues, Values);
	return true;
}

bool ULearningAgentsObservations::GetExclusiveDiscreteObservation(int32& OutIndex, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetExclusiveDiscreteObservation: Object is nullptr.");
		OutIndex = 0;
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetExclusiveDiscreteObservation: Invalid Observation Object.");
		OutIndex = 0;
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetExclusiveDiscreteObservation: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::DiscreteExclusive)
	{
		UE_LOGF(LogLearning, Error, "GetExclusiveDiscreteObservation: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::DiscreteExclusive));
		OutIndex = 0;
		return false;
	}

	OutIndex = Object->ObservationObject.GetDiscreteExclusive(Element.ObjectElement).DiscreteIndex;
	return true;
}

bool ULearningAgentsObservations::GetInclusiveDiscreteObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteObservationNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteObservationNum: Invalid Observation Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetInclusiveDiscreteObservationNum: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::DiscreteInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteObservationNum: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::DiscreteInclusive));
		OutNum = 0;
		return false;
	}

	OutNum = Object->ObservationObject.GetDiscreteInclusive(Element.ObjectElement).DiscreteIndices.Num();
	return true;
}

bool ULearningAgentsObservations::GetInclusiveDiscreteObservation(TArray<int32>& OutIndices, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutIndicesNum;
	if (!GetInclusiveDiscreteObservationNum(OutIndicesNum, Object, Element, bActive, Tag))
	{
		OutIndices.Empty();
		return false;
	}

	OutIndices.SetNumUninitialized(OutIndicesNum);
	return GetInclusiveDiscreteObservationToArrayView(OutIndices, Object, Element, bActive, Tag);
}

bool ULearningAgentsObservations::GetInclusiveDiscreteObservationToArrayView(TArrayView<int32> OutIndices, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteObservationToArrayView: Object is nullptr.");
		UE::Learning::Array::Zero<1, int32>(OutIndices);
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteObservationToArrayView: Invalid Observation Object.");
		UE::Learning::Array::Zero<1, int32>(OutIndices);
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetInclusiveDiscreteObservationToArrayView: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::DiscreteInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteObservationToArrayView: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::DiscreteInclusive));
		UE::Learning::Array::Zero<1, int32>(OutIndices);
		return false;
	}

	const TArrayView<const int32> Indices = Object->ObservationObject.GetDiscreteInclusive(Element.ObjectElement).DiscreteIndices;

	if (Indices.Num() != OutIndices.Num())
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveDiscreteObservationToArrayView: Observation '%ls' size does not match. Observation is '%i' values but asked for '%i'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			Indices.Num(), OutIndices.Num());
		UE::Learning::Array::Zero<1, int32>(OutIndices);
		return false;
	}

	UE::Learning::Array::Copy<1, int32>(OutIndices, Indices);
	return true;
}

bool ULearningAgentsObservations::GetNamedExclusiveDiscreteObservation(FName& OutName, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetNamedExclusiveDiscreteObservation: Object is nullptr.");
		OutName = NAME_None;
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetNamedExclusiveDiscreteObservation: Invalid Observation Object.");
		OutName = NAME_None;
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetNamedExclusiveDiscreteObservation: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::NamedDiscreteExclusive)
	{
		UE_LOGF(LogLearning, Error, "GetNamedExclusiveDiscreteObservation: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::NamedDiscreteExclusive));
		OutName = NAME_None;
		return false;
	}

	OutName = Object->ObservationObject.GetNamedDiscreteExclusive(Element.ObjectElement).ElementName;
	return true;
}

bool ULearningAgentsObservations::GetNamedInclusiveDiscreteObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteObservationNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteObservationNum: Invalid Observation Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetNamedInclusiveDiscreteObservationNum: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::NamedDiscreteInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteObservationNum: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::NamedDiscreteInclusive));
		OutNum = 0;
		return false;
	}

	OutNum = Object->ObservationObject.GetNamedDiscreteInclusive(Element.ObjectElement).ElementNames.Num();
	return true;
}

bool ULearningAgentsObservations::GetNamedInclusiveDiscreteObservation(TArray<FName>& OutNames, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutNamesNum;
	if (!GetNamedInclusiveDiscreteObservationNum(OutNamesNum, Object, Element, bActive, Tag))
	{
		OutNames.Empty();
		return false;
	}

	OutNames.SetNumUninitialized(OutNamesNum);
	return GetNamedInclusiveDiscreteObservationToArrayView(OutNames, Object, Element, bActive, Tag);
}

bool ULearningAgentsObservations::GetNamedInclusiveDiscreteObservationToArrayView(TArrayView<FName> OutNames, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteObservationToArrayView: Object is nullptr.");
		UE::Learning::Array::Set<1, FName>(OutNames, NAME_None);
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteObservationToArrayView: Invalid Observation Object.");
		UE::Learning::Array::Set<1, FName>(OutNames, NAME_None);
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetNamedInclusiveDiscreteObservationToArrayView: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::NamedDiscreteInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteObservationToArrayView: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::NamedDiscreteInclusive));
		UE::Learning::Array::Set<1, FName>(OutNames, NAME_None);
		return false;
	}

	const TArrayView<const FName> Names = Object->ObservationObject.GetNamedDiscreteInclusive(Element.ObjectElement).ElementNames;

	if (Names.Num() != OutNames.Num())
	{
		UE_LOGF(LogLearning, Error, "GetNamedInclusiveDiscreteObservationToArrayView: Observation '%ls' size does not match. Observation is '%i' values but asked for '%i'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			Names.Num(), OutNames.Num());
		UE::Learning::Array::Set<1, FName>(OutNames, NAME_None);
		return false;
	}

	UE::Learning::Array::Copy<1, FName>(OutNames, Names);
	return true;
}

bool ULearningAgentsObservations::GetCountObservation(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const int32 MaxNum, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	float FloatNum = 0.0f;
	if (!GetContinuousObservationToArrayView(MakeArrayView(&FloatNum, 1), Object, Element, bActive, Tag))
	{
		OutNum = -1;
		return false;
	}

	OutNum = FMath::RoundToInt(FloatNum * MaxNum);
	return true;
}

bool ULearningAgentsObservations::GetStructObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationNum: Invalid Observation Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStructObservationNum: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::And)
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationNum: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::And));
		OutNum = 0;
		return false;
	}

	const UE::Learning::Observation::FObjectAndParameters Parameters = Object->ObservationObject.GetAnd(Element.ObjectElement);

	OutNum = Parameters.Elements.Num();
	return true;
}

bool ULearningAgentsObservations::GetStructObservation(TMap<FName, FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetStructObservationNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsObservationObjectElement, TInlineAllocator<16>> SubElements;
	SubElementNames.SetNumUninitialized(OutElementNum);
	SubElements.SetNumUninitialized(OutElementNum);

	if (!GetStructObservationToArrayViews(SubElementNames, SubElements, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.Empty(OutElementNum);
	for (int32 ElementIdx = 0; ElementIdx < OutElementNum; ElementIdx++)
	{
		OutElements.Add(SubElementNames[ElementIdx], SubElements[ElementIdx]);
	}

	return true;
}

bool ULearningAgentsObservations::GetStructObservationElement(FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FName ElementName, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationElement: Object is nullptr.");
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationElement: Invalid Observation Object.");
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStructObservationElement: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::And)
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationElement: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::And));
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	const UE::Learning::Observation::FObjectAndParameters Parameters = Object->ObservationObject.GetAnd(Element.ObjectElement);

	const int32 ElementIdx = Parameters.ElementNames.Find(ElementName);

	if (ElementIdx == INDEX_NONE)
	{
		UE_LOGF(LogLearning, Error, "GetStructActionElement: Element '%ls' not found.", *ElementName.ToString());
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	OutElement = { Parameters.Elements[ElementIdx] };
	return true;
}

bool ULearningAgentsObservations::GetStructObservationToArrays(TArray<FName>& OutElementNames, TArray<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetStructObservationNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	OutElementNames.SetNumUninitialized(OutElementNum);
	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetStructObservationToArrayViews(OutElementNames, OutElements, Object, Element, bActive, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsObservations::GetStructObservationToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FLearningAgentsObservationObjectElement> OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationToArrayViews: Object is nullptr.");
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationToArrayViews: Invalid Observation Object.");
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStructObservationToArrayViews: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::And)
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationToArrayViews: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::And));
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	const UE::Learning::Observation::FObjectAndParameters Parameters = Object->ObservationObject.GetAnd(Element.ObjectElement);

	if (Parameters.Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "GetStructObservationToArrayViews: Getting zero-sized And Observation.");
	}

	if (Parameters.Elements.Num() != OutElements.Num())
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationToArrayViews: Observation '%ls' size does not match. Observation is '%i' elements but asked for '%i'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			Parameters.Elements.Num(), OutElements.Num());
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
	{
		if (!Object->ObservationObject.IsValid(Parameters.Elements[ElementIdx]))
		{
			UE_LOGF(LogLearning, Error, "GetStructObservationToArrayViews: Invalid Observation Object.");
			UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
			UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
			return false;
		}

		OutElementNames[ElementIdx] = Parameters.ElementNames[ElementIdx];
		OutElements[ElementIdx] = { Parameters.Elements[ElementIdx] };
	}

	return true;
}

bool ULearningAgentsObservations::GetExclusiveUnionObservation(FName& OutElementName, FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationToArrayViews: Object is nullptr.");
		OutElementName = NAME_None;
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationToArrayViews: Invalid Observation Object.");
		OutElementName = NAME_None;
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStructObservationToArrayViews: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::OrExclusive)
	{
		UE_LOGF(LogLearning, Error, "GetStructObservationToArrayViews: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::OrExclusive));
		OutElementName = NAME_None;
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	const UE::Learning::Observation::FObjectOrExclusiveParameters Parameters = Object->ObservationObject.GetOrExclusive(Element.ObjectElement);
	OutElementName = Parameters.ElementName;
	OutElement = { Parameters.Element };
	return true;
}

bool ULearningAgentsObservations::GetInclusiveUnionObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionObservationNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionObservationNum: Invalid Observation Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetInclusiveUnionObservationNum: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::OrInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionObservationNum: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::OrInclusive));
		OutNum = 0;
		return false;
	}

	const UE::Learning::Observation::FObjectOrInclusiveParameters Parameters = Object->ObservationObject.GetOrInclusive(Element.ObjectElement);

	OutNum = Parameters.Elements.Num();
	return true;
}

bool ULearningAgentsObservations::GetInclusiveUnionObservation(TMap<FName, FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetInclusiveUnionObservationNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	TArray<FName, TInlineAllocator<16>> SubElementNames;
	TArray<FLearningAgentsObservationObjectElement, TInlineAllocator<16>> SubElements;
	SubElementNames.SetNumUninitialized(OutElementNum);
	SubElements.SetNumUninitialized(OutElementNum);

	if (!GetInclusiveUnionObservationToArrayViews(SubElementNames, SubElements, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.Empty(OutElementNum);
	for (int32 ElementIdx = 0; ElementIdx < OutElementNum; ElementIdx++)
	{
		OutElements.Add(SubElementNames[ElementIdx], SubElements[ElementIdx]);
	}

	return true;
}

bool ULearningAgentsObservations::GetInclusiveUnionObservationToArrays(TArray<FName>& OutElementNames, TArray<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetInclusiveUnionObservationNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	OutElementNames.SetNumUninitialized(OutElementNum);
	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetInclusiveUnionObservationToArrayViews(OutElementNames, OutElements, Object, Element, bActive, Tag))
	{
		OutElementNames.Empty();
		OutElements.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsObservations::GetInclusiveUnionObservationToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FLearningAgentsObservationObjectElement> OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionObservationToArrayViews: Object is nullptr.");
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionObservationToArrayViews: Invalid Observation Object.");
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetInclusiveUnionObservationToArrayViews: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::OrInclusive)
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionObservationToArrayViews: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::OrInclusive));
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	const UE::Learning::Observation::FObjectOrInclusiveParameters Parameters = Object->ObservationObject.GetOrInclusive(Element.ObjectElement);

	if (Parameters.Elements.Num() != OutElements.Num())
	{
		UE_LOGF(LogLearning, Error, "GetInclusiveUnionObservationToArrayViews: Observation '%ls' size does not match. Observation is '%i' elements but asked for '%i'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			Parameters.Elements.Num(), OutElements.Num());
		UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
	{
		if (!Object->ObservationObject.IsValid(Parameters.Elements[ElementIdx]))
		{
			UE_LOGF(LogLearning, Error, "GetInclusiveUnionObservationToArrayViews: Invalid Observation Object.");
			UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
			UE::Learning::Array::Set<1, FName>(OutElementNames, NAME_None);
			return false;
		}

		OutElementNames[ElementIdx] = Parameters.ElementNames[ElementIdx];
		OutElements[ElementIdx] = { Parameters.Elements[ElementIdx] };
	}

	return true;
}

bool ULearningAgentsObservations::GetStaticArrayObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayObservationNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayObservationNum: Invalid Observation Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStaticArrayObservationNum: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Array)
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayObservationNum: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Array));
		OutNum = 0;
		return false;
	}

	OutNum = Object->ObservationObject.GetArray(Element.ObjectElement).Elements.Num();
	return true;
}

bool ULearningAgentsObservations::GetStaticArrayObservation(TArray<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetStaticArrayObservationNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetStaticArrayObservationToArrayView(OutElements, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsObservations::GetStaticArrayObservationToArrayView(TArrayView<FLearningAgentsObservationObjectElement> OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayObservationNum: Object is nullptr.");
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayObservationToArrayView: Invalid Observation Object.");
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetStaticArrayObservationToArrayView: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Array)
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayObservationToArrayView: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Array));
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	const UE::Learning::Observation::FObjectArrayParameters Parameters = Object->ObservationObject.GetArray(Element.ObjectElement);

	if (Parameters.Elements.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "GetStaticArrayObservationToArrayView: Getting zero-sized Static Array Observation.");
	}

	if (Parameters.Elements.Num() != OutElements.Num())
	{
		UE_LOGF(LogLearning, Error, "GetStaticArrayObservationToArrayView: Observation '%ls' size does not match. Observation is '%i' elements but asked for '%i'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			Parameters.Elements.Num(), OutElements.Num());
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
	{
		if (!Object->ObservationObject.IsValid(Parameters.Elements[ElementIdx]))
		{
			UE_LOGF(LogLearning, Error, "GetStaticArrayObservationToArrayView: Invalid Observation Object.");
			UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
			return false;
		}

		OutElements[ElementIdx] = { Parameters.Elements[ElementIdx] };
	}

	return true;
}

bool ULearningAgentsObservations::GetSetObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetSetObservationNum: Object is nullptr.");
		OutNum = 0;
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetSetObservationNum: Invalid Observation Object.");
		OutNum = 0;
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetSetObservationNum: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Set)
	{
		UE_LOGF(LogLearning, Error, "GetSetObservationNum: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Set));
		OutNum = 0;
		return false;
	}

	OutNum = Object->ObservationObject.GetSet(Element.ObjectElement).Elements.Num();
	return true;
}

bool ULearningAgentsObservations::GetSetObservation(TSet<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetSetObservationNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetSetObservation: Object is nullptr.");
		OutElements.Empty();
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetSetObservation: Invalid Observation Object.");
		OutElements.Empty();
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetSetObservation: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Set)
	{
		UE_LOGF(LogLearning, Error, "GetSetObservation: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Set));
		OutElements.Empty();
		return false;
	}

	const UE::Learning::Observation::FObjectSetParameters Parameters = Object->ObservationObject.GetSet(Element.ObjectElement);

	OutElements.Empty(Parameters.Elements.Num());
	for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
	{
		if (!Object->ObservationObject.IsValid(Parameters.Elements[ElementIdx]))
		{
			UE_LOGF(LogLearning, Error, "GetSetObservation: Invalid Observation Object.");
			OutElements.Empty();
			return false;
		}

		OutElements.Add({ Parameters.Elements[ElementIdx] });
	}

	return true;
}

bool ULearningAgentsObservations::GetSetObservationToArray(TArray<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetSetObservationNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetSetObservationToArrayView(OutElements, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsObservations::GetSetObservationToArrayView(TArrayView<FLearningAgentsObservationObjectElement> OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetSetObservationToArrayView: Object is nullptr.");
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetSetObservationToArrayView: Invalid Observation Object.");
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetSetObservationToArrayView: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Set)
	{
		UE_LOGF(LogLearning, Error, "GetSetObservationToArrayView: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Set));
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	const UE::Learning::Observation::FObjectSetParameters Parameters = Object->ObservationObject.GetSet(Element.ObjectElement);

	if (Parameters.Elements.Num() != OutElements.Num())
	{
		UE_LOGF(LogLearning, Error, "GetSetObservationToArrayView: Observation '%ls' size does not match. Observation is '%i' elements but asked for '%i'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			Parameters.Elements.Num(), OutElements.Num());
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	for (int32 ElementIdx = 0; ElementIdx < Parameters.Elements.Num(); ElementIdx++)
	{
		if (!Object->ObservationObject.IsValid(Parameters.Elements[ElementIdx]))
		{
			UE_LOGF(LogLearning, Error, "GetSetObservationToArrayView: Invalid Observation Object.");
			UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
			return false;
		}

		OutElements[ElementIdx] = { Parameters.Elements[ElementIdx] };
	}

	return true;
}

bool ULearningAgentsObservations::GetPairObservation(FLearningAgentsObservationObjectElement& OutKey, FLearningAgentsObservationObjectElement& OutValue, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	TStaticArray<FName, 2> OutElementNames;
	TStaticArray<FLearningAgentsObservationObjectElement, 2> OutElements;
	if (!GetStructObservationToArrayViews(OutElementNames, OutElements, Object, Element, bActive, Tag))
	{
		OutKey = FLearningAgentsObservationObjectElement();
		OutValue = FLearningAgentsObservationObjectElement();
		return false;
	}

	OutKey = OutElements[MakeArrayView(OutElementNames).Find(TEXT("Key"))];
	OutValue = OutElements[MakeArrayView(OutElementNames).Find(TEXT("Value"))];
	return true;
}

bool ULearningAgentsObservations::GetArrayObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	return GetSetObservationNum(OutNum, Object, Element, bActive, Tag);
}

bool ULearningAgentsObservations::GetArrayObservation(TArray<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const int32 MaxNum, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetArrayObservationNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.SetNumUninitialized(OutElementNum);

	if (!GetArrayObservationToArrayView(OutElements, Object, Element, MaxNum, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsObservations::GetArrayObservationToArrayView(TArrayView<FLearningAgentsObservationObjectElement> OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const int32 MaxNum, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	TArray<FLearningAgentsObservationObjectElement, TInlineAllocator<16>> Pairs;
	Pairs.SetNumUninitialized(OutElements.Num());
	if (!GetSetObservationToArrayView(Pairs, Object, Element, bActive, Tag))
	{
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
		return false;
	}

	for (int32 PairIdx = 0; PairIdx < Pairs.Num(); PairIdx++)
	{
		FLearningAgentsObservationObjectElement Key, Value;
		if (!GetPairObservation(Key, Value, Object, Pairs[PairIdx]))
		{
			UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
			return false;
		}

		int32 Index = INDEX_NONE;
		if (!GetCountObservation(Index, Object, Key, MaxNum))
		{
			UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutElements, FLearningAgentsObservationObjectElement());
			return false;
		}
		Index = FMath::Clamp(Index, 0, OutElements.Num() - 1);

		OutElements[Index] = Value;
	}

	return true;
}

bool ULearningAgentsObservations::GetMapObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	return GetSetObservationNum(OutNum, Object, Element, bActive, Tag);
}

bool ULearningAgentsObservations::GetMapObservation(TMap<FLearningAgentsObservationObjectElement, FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetMapObservationNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	TArray<FLearningAgentsObservationObjectElement, TInlineAllocator<16>> Pairs;
	Pairs.SetNumUninitialized(OutElementNum);
	if (!GetSetObservationToArrayView(Pairs, Object, Element, bActive, Tag))
	{
		OutElements.Empty();
		return false;
	}

	OutElements.Empty(OutElementNum);
	for (int32 PairIdx = 0; PairIdx < OutElementNum; PairIdx++)
	{
		FLearningAgentsObservationObjectElement Key, Value;
		if (!GetPairObservation(Key, Value, Object, Pairs[PairIdx]))
		{
			OutElements.Empty();
			return false;
		}

		OutElements.Add(Key, Value);
	}

	return true;
}

bool ULearningAgentsObservations::GetMapObservationToArrays(TArray<FLearningAgentsObservationObjectElement>& OutKeys, TArray<FLearningAgentsObservationObjectElement>& OutValues, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	int32 OutElementNum = 0;
	if (!GetMapObservationNum(OutElementNum, Object, Element, bActive, Tag))
	{
		OutKeys.Empty();
		OutValues.Empty();
		return false;
	}

	OutKeys.SetNumUninitialized(OutElementNum);
	OutValues.SetNumUninitialized(OutElementNum);
	if (!GetMapObservationToArrayViews(OutKeys, OutValues, Object, Element, bActive, Tag))
	{
		OutKeys.Empty();
		OutValues.Empty();
		return false;
	}

	return true;
}

bool ULearningAgentsObservations::GetMapObservationToArrayViews(TArrayView<FLearningAgentsObservationObjectElement> OutKeys, TArrayView<FLearningAgentsObservationObjectElement> OutValues, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	TArray<FLearningAgentsObservationObjectElement, TInlineAllocator<16>> Pairs;
	Pairs.SetNumUninitialized(OutKeys.Num());
	if (!GetSetObservationToArrayView(Pairs, Object, Element, bActive, Tag))
	{
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutKeys, FLearningAgentsObservationObjectElement());
		UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutValues, FLearningAgentsObservationObjectElement());
		return false;
	}

	for (int32 PairIdx = 0; PairIdx < Pairs.Num(); PairIdx++)
	{
		FLearningAgentsObservationObjectElement Key, Value;
		if (!GetPairObservation(Key, Value, Object, Pairs[PairIdx]))
		{
			UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutKeys, FLearningAgentsObservationObjectElement());
			UE::Learning::Array::Set<1, FLearningAgentsObservationObjectElement>(OutValues, FLearningAgentsObservationObjectElement());
			return false;
		}

		OutKeys[PairIdx] = Key;
		OutValues[PairIdx] = Value;
	}

	return true;
}

bool ULearningAgentsObservations::GetEnumObservation(uint8& OutEnumValue, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const UEnum* Enum, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "GetEnumObservation: Enum is nullptr.");
		OutEnumValue = 0;
		return false;
	}

	int32 EnumValueNum;
	if (!GetContinuousObservationNum(EnumValueNum, Object, Element, bActive, Tag))
	{
		OutEnumValue = 0;
		return false;
	}
	
	if (EnumValueNum != Enum->NumEnums() - 1)
	{
		UE_LOGF(LogLearning, Error, "GetEnumObservation: Too many values for Enum '%ls'. Expected %i, got %i.", *Enum->GetName(), Enum->NumEnums() - 1, EnumValueNum);
		OutEnumValue = 0;
		return false;
	}

	TArray<float, TInlineAllocator<32>> OneHot;
	OneHot.SetNumUninitialized(EnumValueNum);
	if (!GetContinuousObservationToArrayView(OneHot, Object, Element, bActive, Tag))
	{
		OutEnumValue = 0;
		return false;
	}

	int32 EnumValueIndex = INDEX_NONE;
	for (int32 EnumIdx = 0; EnumIdx < EnumValueNum; EnumIdx++)
	{
		if (OneHot[EnumIdx])
		{
			EnumValueIndex = EnumIdx;
			break;
		}
	}

	if (EnumValueIndex == INDEX_NONE)
	{
		UE_LOGF(LogLearning, Error, "GetEnumObservation: Index not found.");
		OutEnumValue = 0;
		return false;
	}

	const int32 EnumValue = Enum->GetValueByIndex(EnumValueIndex);

	if (EnumValue == INDEX_NONE)
	{
		UE_LOGF(LogLearning, Error, "GetEnumObservation: Enum Value not found for index %i.", EnumValueIndex);
		OutEnumValue = 0;
		return false;
	}

	OutEnumValue = (uint8)EnumValue;
	return true;
}

bool ULearningAgentsObservations::GetBitmaskObservation(int32& OutBitmaskValue, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const UEnum* Enum, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Enum)
	{
		UE_LOGF(LogLearning, Error, "GetBitmaskObservation: Enum is nullptr.");
		OutBitmaskValue = 0;
		return false;
	}

	if (Enum->NumEnums() - 1 > 32)
	{
		UE_LOGF(LogLearning, Error, "GetBitmaskObservation: Too many values in Enum to use as Bitmask (%i).", Enum->NumEnums() - 1);
		OutBitmaskValue = 0;
		return false;
	}

	int32 EnumValueNum;
	if (!GetContinuousObservationNum(EnumValueNum, Object, Element, bActive, Tag))
	{
		OutBitmaskValue = 0;
		return false;
	}

	if (EnumValueNum != Enum->NumEnums() - 1)
	{
		UE_LOGF(LogLearning, Error, "GetBitmaskObservation: Too many values for Enum '%ls'. Expected %i, got %i.", *Enum->GetName(), Enum->NumEnums() - 1, EnumValueNum);
		OutBitmaskValue = 0;
		return false;
	}

	TArray<float, TInlineAllocator<32>> OneHot;
	OneHot.Init(0.0f, EnumValueNum);
	if (!GetContinuousObservationToArrayView(OneHot, Object, Element, bActive, Tag))
	{
		OutBitmaskValue = 0;
		return false;
	}

	OutBitmaskValue = 0;
	for (int32 OneHotIdx = 0; OneHotIdx < EnumValueNum; OneHotIdx++)
	{
		if (OneHot[OneHotIdx])
		{
			OutBitmaskValue |= (1 << OneHotIdx);
		}
	}
	return true;
}


bool ULearningAgentsObservations::GetOptionalObservation(ELearningAgentsOptionalObservation& OutOption, FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	FName OutName = NAME_None;
	if (!GetExclusiveUnionObservation(OutName, OutElement, Object, Element, bActive, Tag))
	{
		OutOption = ELearningAgentsOptionalObservation::Null;
		return false;
	}

	OutOption = OutName == TEXT("Null") ? ELearningAgentsOptionalObservation::Null : ELearningAgentsOptionalObservation::Valid;
	return true;
}

bool ULearningAgentsObservations::GetEitherObservation(ELearningAgentsEitherObservation& OutEither, FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	FName OutName = NAME_None;
	if (!GetExclusiveUnionObservation(OutName, OutElement, Object, Element, bActive, Tag))
	{
		OutEither = ELearningAgentsEitherObservation::A;
		return false;
	}

	OutEither = OutName == TEXT("A") ? ELearningAgentsEitherObservation::A : ELearningAgentsEitherObservation::B;
	return true;
}

bool ULearningAgentsObservations::GetEncodingObservation(FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetEncodingObservation: Object is nullptr.");
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetEncodingObservation: Invalid Observation Object.");
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetEncodingObservation: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Encoding)
	{
		UE_LOGF(LogLearning, Error, "GetEncodingObservation: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Encoding));
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	OutElement = { Object->ObservationObject.GetEncoding(Element.ObjectElement).Element };

	return true;
}

bool ULearningAgentsObservations::GetBoolObservation(bool& bOutValue, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	float OutValue = 0.0f;
	if (!GetFloatObservation(OutValue, Object, Element, bActive, Tag))
	{
		bOutValue = false;
		return false;
	}

	bOutValue = OutValue >= 0.0f;
	return true;
}

bool ULearningAgentsObservations::GetFloatObservation(float& OutValue, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!GetContinuousObservationToArrayView(MakeArrayView(&OutValue, 1), Object, Element, bActive, Tag))
	{
		OutValue = 0.0f;
		return false;
	}

	return true;
}

bool ULearningAgentsObservations::GetLocationObservation(FVector& OutLocation, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	TStaticArray<float, 3> OutValues;
	if (!GetContinuousObservationToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutLocation = FVector::ZeroVector;
		return false;
	}

	OutLocation = RelativeTransform.TransformPosition(FVector(OutValues[0], OutValues[1], OutValues[2]));
	return true;
}

bool ULearningAgentsObservations::GetRotationObservation(FRotator& OutRotation, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FRotator RelativeRotation, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	FQuat OutRotationQuat;
	if (!GetRotationObservationAsQuat(OutRotationQuat, Object, Element, FQuat::MakeFromRotator(RelativeRotation), bActive, Tag))
	{
		OutRotation = FRotator::ZeroRotator;
		return false;
	}

	OutRotation = OutRotationQuat.Rotator();
	return true;
}

bool ULearningAgentsObservations::GetRotationObservationAsQuat(FQuat& OutRotation, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FQuat RelativeRotation, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	TStaticArray<float, 6> OutValues;
	if (!GetContinuousObservationToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutRotation = FQuat::Identity;
		return false;
	}

	const FVector LocalAxisForward = FVector(OutValues[0], OutValues[1], OutValues[2]);
	const FVector LocalAxisRight = FVector(OutValues[3], OutValues[4], OutValues[5]);
	const FVector AxisUp = LocalAxisForward.Cross(LocalAxisRight).GetSafeNormal(UE_SMALL_NUMBER, FVector::UpVector);
	const FVector AxisRight = AxisUp.Cross(LocalAxisForward).GetSafeNormal(UE_SMALL_NUMBER, FVector::RightVector);
	const FVector AxisForward = LocalAxisForward.GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);

	FMatrix RotationMatrix = FMatrix::Identity;
	RotationMatrix.SetAxis(0, AxisForward);
	RotationMatrix.SetAxis(1, AxisRight);
	RotationMatrix.SetAxis(2, AxisUp);

	OutRotation = RelativeRotation * RotationMatrix.ToQuat();
	return true;
}

bool ULearningAgentsObservations::GetScaleObservation(FVector& OutScale, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FVector RelativeScale, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	TStaticArray<float, 3> OutValues;
	if (!GetContinuousObservationToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutScale = FVector::OneVector;
		return false;
	}

	OutScale = RelativeScale * UE::Learning::Agents::Observation::Private::VectorExp(FVector(OutValues[0], OutValues[1], OutValues[2]));
	return true;
}

bool ULearningAgentsObservations::GetTransformObservation(FTransform& OutTransform, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	TStaticArray<FName, 3> OutElementNames;
	TStaticArray<FLearningAgentsObservationObjectElement, 3> OutElements;
	if (!GetStructObservationToArrayViews(OutElementNames, OutElements, Object, Element, bActive, Tag))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	const int32 LocationElement = MakeArrayView(OutElementNames).Find(TEXT("Location"));
	FVector OutLocation;
	if (LocationElement == INDEX_NONE || !GetLocationObservation(OutLocation, Object, OutElements[LocationElement], RelativeTransform))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	const int32 RotationElement = MakeArrayView(OutElementNames).Find(TEXT("Rotation"));
	FQuat OutRotation;
	if (RotationElement == INDEX_NONE || !GetRotationObservationAsQuat(OutRotation, Object, OutElements[RotationElement], RelativeTransform.GetRotation()))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	const int32 ScaleElement = MakeArrayView(OutElementNames).Find(TEXT("Scale"));
	FVector OutScale;
	if (ScaleElement == INDEX_NONE || !GetScaleObservation(OutScale, Object, OutElements[ScaleElement], RelativeTransform.GetScale3D()))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	OutTransform = FTransform(OutRotation, OutLocation, OutScale);
	return true;
}

bool ULearningAgentsObservations::GetAngleObservationRadians(float& OutAngle, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const float RelativeAngle, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	TStaticArray<float, 2> OutValues;
	if (!GetContinuousObservationToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutAngle = 0.0f;
		return false;
	}

	OutAngle = RelativeAngle + FMath::Atan2(OutValues[0], OutValues[1]);
	return true;
}


bool ULearningAgentsObservations::GetAngleObservation(float& OutAngle, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const float RelativeAngle, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!GetAngleObservationRadians(OutAngle, Object, Element, FMath::DegreesToRadians(RelativeAngle), bActive, Tag))
	{
		return false;
	}

	OutAngle = FMath::RadiansToDegrees(OutAngle);
	return true;
}

bool ULearningAgentsObservations::GetVelocityObservation(FVector& OutVelocity, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	TStaticArray<float, 3> OutValues;
	if (!GetContinuousObservationToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutVelocity = FVector::ZeroVector;
		return false;
	}

	OutVelocity = RelativeTransform.TransformVectorNoScale(FVector(OutValues[0], OutValues[1], OutValues[2]));
	return true;
}

bool ULearningAgentsObservations::GetDirectionObservation(FVector& OutDirection, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	TStaticArray<float, 3> OutValues;
	if (!GetContinuousObservationToArrayView(OutValues, Object, Element, bActive, Tag))
	{
		OutDirection = FVector::ForwardVector;
		return false;
	}

	OutDirection = RelativeTransform.TransformVectorNoScale(FVector(OutValues[0], OutValues[1], OutValues[2]).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector));
	return true;
}

bool ULearningAgentsObservations::GetLocationAlongSplineObservation(FVector& OutLocation, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	return GetLocationObservation(OutLocation, Object, Element, RelativeTransform, bActive, Tag);
}

bool ULearningAgentsObservations::GetProportionAlongSplineObservation(bool& bOutIsClosedLoop, float& OutAngle, float& OutPropotion, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	FName SubName;
	FLearningAgentsObservationObjectElement SubElement;
	if (!GetExclusiveUnionObservation(SubName, SubElement, Object, Element, bActive, Tag))
	{
		bOutIsClosedLoop = false;
		OutAngle = 0.0f;
		OutPropotion = 0.0;
		return false;
	}

	if (SubName == TEXT("Angle"))
	{
		bOutIsClosedLoop = true;
		OutPropotion = 0.0f;
		return GetAngleObservation(OutAngle, Object, SubElement);
	}
	else
	{
		bOutIsClosedLoop = false;
		OutAngle = 0.0f;
		return GetFloatObservation(OutPropotion, Object, SubElement);
	}
}

bool ULearningAgentsObservations::GetDirectionAlongSplineObservation(FVector& OutDirection, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	return GetDirectionObservation(OutDirection, Object, Element, RelativeTransform, bActive, Tag);
}

bool ULearningAgentsObservations::GetProportionAlongRayObservation(float& OutProportion, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!GetFloatObservation(OutProportion, Object, Element, bActive, Tag))
	{
		OutProportion = 0.0f;
		return false;
	}

	OutProportion = 1.0f - OutProportion;
	return true;
}

bool ULearningAgentsObservations::GetConv1dObservation(FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetConv1dObservation: Object is nullptr.");
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetConv1dObservation: Invalid Observation Object.");
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetConv1dObservation: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Conv1d)
	{
		UE_LOGF(LogLearning, Error, "GetConv1dObservation: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Conv1d));
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	OutElement = { Object->ObservationObject.GetConv1d(Element.ObjectElement).Element };

	return true;
}

bool ULearningAgentsObservations::GetConv2dObservation(FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive, const FName Tag)
{
	if (!bActive)
	{
		return GetNullObservation(Object, Element, Tag);
	}

	if (!Object)
	{
		UE_LOGF(LogLearning, Error, "GetConv2dObservation: Object is nullptr.");
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	if (!Object->ObservationObject.IsValid(Element.ObjectElement))
	{
		UE_LOGF(LogLearning, Error, "GetConv2dObservation: Invalid Observation Object.");
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	if (Object->ObservationObject.GetTag(Element.ObjectElement) != Tag)
	{
		UE_LOGF(LogLearning, Warning, "GetConv2dObservation: Observation tag does not match. Observation is '%ls' but asked for '%ls'.", *Object->ObservationObject.GetTag(Element.ObjectElement).ToString(), *Tag.ToString());
	}

	if (Object->ObservationObject.GetType(Element.ObjectElement) != UE::Learning::Observation::EType::Conv2d)
	{
		UE_LOGF(LogLearning, Error, "GetConv2dObservation: Observation '%ls' type does not match. Observation is '%ls' but asked for '%ls'.",
			*Object->ObservationObject.GetTag(Element.ObjectElement).ToString(),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(Object->ObservationObject.GetType(Element.ObjectElement)),
			UE::Learning::Agents::Observation::Private::GetObservationTypeString(UE::Learning::Observation::EType::Conv2d));
		OutElement = FLearningAgentsObservationObjectElement();
		return false;
	}

	OutElement = { Object->ObservationObject.GetConv2d(Element.ObjectElement).Element };

	return true;
}
