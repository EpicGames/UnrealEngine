// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsPolicy.h"

#include "LearningAgentsManager.h"
#include "LearningAgentsInteractor.h"
#include "LearningNeuralNetwork.h"
#include "LearningPolicy.h"
#include "LearningLog.h"
#include "LearningRandom.h"

#include "UObject/Package.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "GameFramework/Actor.h"
#include "Dom/JsonObject.h"

#include "NNERuntimeBasicCpuBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsPolicy)

ULearningAgentsPolicy::ULearningAgentsPolicy() : Super(FObjectInitializer::Get()) {}
ULearningAgentsPolicy::ULearningAgentsPolicy(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsPolicy::~ULearningAgentsPolicy() = default;

ULearningAgentsPolicy* ULearningAgentsPolicy::MakePolicy(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	TSubclassOf<ULearningAgentsPolicy> Class,
	const FName Name,
	ULearningAgentsNeuralNetwork* EncoderNeuralNetworkAsset,
	ULearningAgentsNeuralNetwork* PolicyNeuralNetworkAsset,
	ULearningAgentsNeuralNetwork* DecoderNeuralNetworkAsset,
	const bool bReinitializeEncoderNetwork,
	const bool bReinitializePolicyNetwork,
	const bool bReinitializeDecoderNetwork,
	const FLearningAgentsPolicySettings& PolicySettings,
	const int32 Seed)
{
	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "MakePolicy: InManager is nullptr.");
		return nullptr;
	}

	if (!Class)
	{
		UE_LOGF(LogLearning, Error, "MakePolicy: Class is nullptr.");
		return nullptr;
	}

	const FName UniqueName = MakeUniqueObjectName(InManager, Class, Name, EUniqueObjectNameOptions::GloballyUnique);

	ULearningAgentsPolicy* Policy = NewObject<ULearningAgentsPolicy>(InManager, Class, UniqueName);
	if (!Policy) { return nullptr; }

	Policy->SetupPolicy(
		InManager,
		InInteractor,
		EncoderNeuralNetworkAsset,
		PolicyNeuralNetworkAsset,
		DecoderNeuralNetworkAsset,
		bReinitializeEncoderNetwork,
		bReinitializePolicyNetwork,
		bReinitializeDecoderNetwork,
		PolicySettings,
		Seed);

	return Policy->IsSetup() ? Policy : nullptr;
}

void ULearningAgentsPolicy::SetupPolicy(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	ULearningAgentsNeuralNetwork* EncoderNeuralNetworkAsset,
	ULearningAgentsNeuralNetwork* PolicyNeuralNetworkAsset,
	ULearningAgentsNeuralNetwork* DecoderNeuralNetworkAsset,
	const bool bReinitializeEncoderNetwork,
	const bool bReinitializePolicyNetwork,
	const bool bReinitializeDecoderNetwork,
	const FLearningAgentsPolicySettings& InPolicySettings,
	const int32 Seed)
{
	if (IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup already run!", *GetName());
		return;
	}

	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "%ls: InManager is nullptr.", *GetName());
		return;
	}

	if (!InInteractor)
	{
		UE_LOGF(LogLearning, Error, "%ls: InInteractor is nullptr.", *GetName());
		return;
	}

	if (!InInteractor->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InInteractor->GetName());
		return;
	}

	// Warn if Network Assets are not unique

	if (PolicyNeuralNetworkAsset && PolicyNeuralNetworkAsset == EncoderNeuralNetworkAsset)
	{
		UE_LOGF(LogLearning, Warning, "%ls: Identical Network Assets given as both Policy and Encoder: %ls.", *GetName(), *PolicyNeuralNetworkAsset->GetName());
	}


	if (PolicyNeuralNetworkAsset && PolicyNeuralNetworkAsset == DecoderNeuralNetworkAsset)
	{
		UE_LOGF(LogLearning, Warning, "%ls: Identical Network Assets given as both Policy and Decoder: %ls.", *GetName(), *PolicyNeuralNetworkAsset->GetName());
	}

	if (EncoderNeuralNetworkAsset && EncoderNeuralNetworkAsset == DecoderNeuralNetworkAsset)
	{
		UE_LOGF(LogLearning, Warning, "%ls: Identical Network Assets given as both Encoder and Decoder: %ls.", *GetName(), *EncoderNeuralNetworkAsset->GetName());
	}

	// Begin Setup

	Manager = InManager;
	Interactor = InInteractor;
	PolicySettings = InPolicySettings;

	// Compatibility Hashes and Sizes

	const int32 MemoryStateSize = PolicySettings.MemoryCell == ELearningAgentsMemoryCell::NoMemoryCell ? 0: PolicySettings.MemoryStateSize;

	const int32 ObservationVectorSize = Interactor->GetObservationVectorSize();
	const int32 ObservationEncodedVectorSize = Interactor->GetObservationEncodedVectorSize();
	const int32 ActionEncodedVectorSize = Interactor->GetActionEncodedVectorSize();
	const int32 ActionDistributionVectorSize = Interactor->GetActionDistributionVectorSize();

	const int32 ObservationCompatibilityHash = UE::Learning::Observation::GetSchemaObjectsCompatibilityHash(Interactor->GetObservationSchema()->ObservationSchema, Interactor->GetObservationSchemaElement().SchemaElement);
	const int32 ActionCompatibilityHash = UE::Learning::Action::GetSchemaObjectsCompatibilityHash(Interactor->GetActionSchema()->ActionSchema, Interactor->GetActionSchemaElement().SchemaElement);
	
	const int32 PolicyHashData[3] = { MemoryStateSize, ObservationEncodedVectorSize, ActionEncodedVectorSize };
	const int32 PolicyCompatibilityHash = CityHash32((const char*)PolicyHashData, 3 * sizeof(int32));

	// Encoder

	if (EncoderNeuralNetworkAsset)
	{
		EncoderNetwork = EncoderNeuralNetworkAsset;

		if (EncoderNeuralNetworkAsset->NeuralNetworkData && !bReinitializeEncoderNetwork)
		{
			if (EncoderNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash() != ObservationCompatibilityHash)
			{
				UE_LOGF(LogLearning, Error, "%ls: Encoder Network Asset provided during Setup is incompatible with Schema. Network hash is %i vs Schema hash %i.", *GetName(),
					EncoderNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash(),
					ObservationCompatibilityHash);
				return;
			}

			if (EncoderNeuralNetworkAsset->NeuralNetworkData->GetInputSize() != ObservationVectorSize ||
				EncoderNeuralNetworkAsset->NeuralNetworkData->GetOutputSize() != ObservationEncodedVectorSize)
			{
				UE_LOGF(LogLearning, Error, "%ls: Encoder Network Asset provided during Setup is incorrect size: Got inputs of size %i, expected %i. Got outputs of size %i, expected %i.", *GetName(),
					EncoderNeuralNetworkAsset->NeuralNetworkData->GetInputSize(), ObservationVectorSize,
					EncoderNeuralNetworkAsset->NeuralNetworkData->GetOutputSize(), ObservationEncodedVectorSize);
				return;
			}
		}
	}

	if (!EncoderNetwork)
	{
		const FName UniqueName = MakeUniqueObjectName(this, ULearningAgentsNeuralNetwork::StaticClass(), TEXT("EncoderNetwork"), EUniqueObjectNameOptions::GloballyUnique);
		EncoderNetwork = NewObject<ULearningAgentsNeuralNetwork>(this, UniqueName);
	}

	if (!EncoderNetwork->NeuralNetworkData || bReinitializeEncoderNetwork)
	{
		if (!EncoderNetwork->NeuralNetworkData)
		{
			EncoderNetwork->NeuralNetworkData = NewObject<ULearningNeuralNetworkData>(EncoderNetwork);
		}

		TArray<uint8> FileData;
		uint32 EncoderInputSize, EncoderOutputSize;
		UE::Learning::Observation::GenerateEncoderNetworkFileDataFromSchema(
			FileData,
			EncoderInputSize,
			EncoderOutputSize,
			Interactor->GetObservationSchema()->ObservationSchema,
			Interactor->GetObservationSchemaElement().SchemaElement,
			UE::Learning::Observation::FNetworkSettings(),
			UE::Learning::Random::Int(Seed ^ 0x658868dd));

		check(EncoderInputSize == ObservationVectorSize);
		check(EncoderOutputSize == ObservationEncodedVectorSize);

		EncoderNetwork->NeuralNetworkData->Init(EncoderInputSize, EncoderOutputSize, ObservationCompatibilityHash, FileData);
		EncoderNetwork->ForceMarkDirty();
	}

	// Policy

	if (PolicyNeuralNetworkAsset)
	{
		PolicyNetwork = PolicyNeuralNetworkAsset;

		if (PolicyNeuralNetworkAsset->NeuralNetworkData && !bReinitializePolicyNetwork)
		{
			if (PolicyNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash() != PolicyCompatibilityHash)
			{
				UE_LOGF(LogLearning, Error, "%ls: Policy Network Asset provided during Setup is incompatible with Schema. Network hash is %i vs Schema hash %i.", *GetName(),
					PolicyNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash(),
					PolicyCompatibilityHash);
				return;
			}

			if (PolicyNeuralNetworkAsset->NeuralNetworkData->GetInputSize() != ObservationEncodedVectorSize + MemoryStateSize ||
				PolicyNeuralNetworkAsset->NeuralNetworkData->GetOutputSize() != ActionEncodedVectorSize + MemoryStateSize)
			{
				UE_LOGF(LogLearning, Error, "%ls: Policy Network Asset provided during Setup is incorrect size: Got inputs of size %i, expected %i. Got outputs of size %i, expected %i.", *GetName(),
					PolicyNeuralNetworkAsset->NeuralNetworkData->GetInputSize(), ObservationEncodedVectorSize + MemoryStateSize,
					PolicyNeuralNetworkAsset->NeuralNetworkData->GetOutputSize(), ActionEncodedVectorSize + MemoryStateSize);
				return;
			}
		}
	}

	if (!PolicyNetwork)
	{
		const FName UniqueName = MakeUniqueObjectName(this, ULearningAgentsNeuralNetwork::StaticClass(), TEXT("PolicyNetwork"), EUniqueObjectNameOptions::GloballyUnique);
		PolicyNetwork = NewObject<ULearningAgentsNeuralNetwork>(this, UniqueName);
	}

	if (!PolicyNetwork->NeuralNetworkData || bReinitializePolicyNetwork)
	{
		if (!PolicyNetwork->NeuralNetworkData)
		{
			PolicyNetwork->NeuralNetworkData = NewObject<ULearningNeuralNetworkData>(PolicyNetwork);
		}

		UE::NNE::RuntimeBasic::FModelBuilder Builder(UE::Learning::Random::Int(Seed ^ 0x69315bf9));

		const int32 PolicyHiddenLayerSize = PolicySettings.HiddenLayerSize;
		const int32 PolicyHiddenLayerNum = PolicySettings.HiddenLayerNum;
		const ELearningAgentsActivationFunction PolicyActivationFunction = PolicySettings.ActivationFunction;
		const float PolicyInitialEncodedActionScale = PolicySettings.InitialEncodedActionScale;

		// Scale down memory cell initial weights to improve stability at beginning of training
		UE::NNE::RuntimeBasic::FModelBuilder::FLinearLayerSettings MemoryCellWeightSettings;
		MemoryCellWeightSettings.WeightInitializationSettings.Scale = 0.1f;

		TArray<uint8> FileData;
		uint32 PolicyInputSize = 0, PolicyOutputSize = 0;

		switch (PolicySettings.MemoryCell)
		{
		case ELearningAgentsMemoryCell::NoMemoryCell:
			Builder.WriteFileDataAndReset(FileData, PolicyInputSize, PolicyOutputSize,
				Builder.MakeSequence({
					Builder.MakeMLP(
						ObservationEncodedVectorSize,
						ActionEncodedVectorSize,
						PolicyHiddenLayerSize,
						PolicyHiddenLayerNum + 2, // Add 2 to account for input and output layers
						UE::Learning::Agents::NeuralNetwork::GetBuilderActivationFunction(PolicyActivationFunction)),
					Builder.MakeDenormalize(
						ActionEncodedVectorSize,
						Builder.MakeValuesZero(ActionEncodedVectorSize),
						Builder.MakeValuesConstant(ActionEncodedVectorSize, PolicyInitialEncodedActionScale)) }));
			break;
		case ELearningAgentsMemoryCell::LearningAgentsGRU:
			Builder.WriteFileDataAndReset(FileData, PolicyInputSize, PolicyOutputSize,
				Builder.MakeMemoryBackbone(
					MemoryStateSize,
					Builder.MakeMLP(
						ObservationEncodedVectorSize,
						PolicyHiddenLayerSize,
						PolicyHiddenLayerSize,
						PolicyHiddenLayerNum / 2 + 2, // Add 2 to account for input and output layers
						UE::Learning::Agents::NeuralNetwork::GetBuilderActivationFunction(PolicyActivationFunction),
						true),
					Builder.MakeMemoryCellLayer(
						PolicyHiddenLayerSize,
						PolicyHiddenLayerSize,
						MemoryStateSize,
						MemoryCellWeightSettings),
					Builder.MakeSequence({
						Builder.MakeMLP(
							PolicyHiddenLayerSize,
							ActionEncodedVectorSize,
							PolicyHiddenLayerSize,
							PolicyHiddenLayerNum / 2 + 2, // Add 2 to account for input and output layers
							UE::Learning::Agents::NeuralNetwork::GetBuilderActivationFunction(PolicyActivationFunction)),
						Builder.MakeDenormalize(
							ActionEncodedVectorSize,
							Builder.MakeValuesZero(ActionEncodedVectorSize),
							Builder.MakeValuesConstant(ActionEncodedVectorSize, PolicyInitialEncodedActionScale))
						})
				));
			break;
		case ELearningAgentsMemoryCell::StandardGRU:
			Builder.WriteFileDataAndReset(FileData, PolicyInputSize, PolicyOutputSize,
				Builder.MakeMemoryBackbone(
					MemoryStateSize,
					Builder.MakeMLP(
						ObservationEncodedVectorSize,
						PolicyHiddenLayerSize,
						PolicyHiddenLayerSize,
						PolicyHiddenLayerNum / 2 + 2, // Add 2 to account for input and output layers
						UE::Learning::Agents::NeuralNetwork::GetBuilderActivationFunction(PolicyActivationFunction),
						true),
					Builder.MakeGRUCellLayer(
						PolicyHiddenLayerSize,
						MemoryStateSize,
						MemoryCellWeightSettings),
					Builder.MakeSequence({
						Builder.MakeMLP(
							MemoryStateSize,
							ActionEncodedVectorSize,
							PolicyHiddenLayerSize,
							PolicyHiddenLayerNum / 2 + 2, // Add 2 to account for input and output layers
							UE::Learning::Agents::NeuralNetwork::GetBuilderActivationFunction(PolicyActivationFunction)),
						Builder.MakeDenormalize(
							ActionEncodedVectorSize,
							Builder.MakeValuesZero(ActionEncodedVectorSize),
							Builder.MakeValuesConstant(ActionEncodedVectorSize, PolicyInitialEncodedActionScale))
						})
				));
			break;
		default:
			checkNoEntry();
		}

		check(PolicyInputSize == ObservationEncodedVectorSize + MemoryStateSize);
		check(PolicyOutputSize == ActionEncodedVectorSize + MemoryStateSize);

		PolicyNetwork->NeuralNetworkData->Init(PolicyInputSize, PolicyOutputSize, PolicyCompatibilityHash, FileData);
		PolicyNetwork->ForceMarkDirty();
	}

	// Decoder

	if (DecoderNeuralNetworkAsset)
	{
		DecoderNetwork = DecoderNeuralNetworkAsset;

		if (DecoderNeuralNetworkAsset->NeuralNetworkData && !bReinitializeDecoderNetwork)
		{
			if (DecoderNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash() != ActionCompatibilityHash)
			{
				UE_LOGF(LogLearning, Error, "%ls: Decoder Network Asset provided during Setup is incompatible with Schema. Network hash is %i vs Schema hash %i.", *GetName(),
					DecoderNeuralNetworkAsset->NeuralNetworkData->GetCompatibilityHash(),
					ActionCompatibilityHash);
				return;
			}

			if (DecoderNeuralNetworkAsset->NeuralNetworkData->GetInputSize() != ActionEncodedVectorSize ||
				DecoderNeuralNetworkAsset->NeuralNetworkData->GetOutputSize() != ActionDistributionVectorSize)
			{
				UE_LOGF(LogLearning, Error, "%ls: Decoder Network Asset provided during Setup is incorrect size: Got inputs of size %i, expected %i. Got outputs of size %i, expected %i.", *GetName(),
					DecoderNeuralNetworkAsset->NeuralNetworkData->GetInputSize(), ActionEncodedVectorSize,
					DecoderNeuralNetworkAsset->NeuralNetworkData->GetOutputSize(), ActionDistributionVectorSize);
				return;
			}
		}
	}

	if (!DecoderNetwork)
	{
		const FName UniqueName = MakeUniqueObjectName(this, ULearningAgentsNeuralNetwork::StaticClass(), TEXT("DecoderNetwork"), EUniqueObjectNameOptions::GloballyUnique);
		DecoderNetwork = NewObject<ULearningAgentsNeuralNetwork>(this, UniqueName);
	}

	if (!DecoderNetwork->NeuralNetworkData || bReinitializeDecoderNetwork)
	{
		if (!DecoderNetwork->NeuralNetworkData)
		{
			DecoderNetwork->NeuralNetworkData = NewObject<ULearningNeuralNetworkData>(DecoderNetwork);
		}

		TArray<uint8> FileData;
		uint32 DecoderInputSize, DecoderOutputSize;
		UE::Learning::Action::GenerateDecoderNetworkFileDataFromSchema(
			FileData,
			DecoderInputSize,
			DecoderOutputSize,
			Interactor->GetActionSchema()->ActionSchema,
			Interactor->GetActionSchemaElement().SchemaElement,
			UE::Learning::Action::FNetworkSettings(),
			UE::Learning::Random::Int(Seed ^ 0xfa88bb7f));

		check(DecoderInputSize == ActionEncodedVectorSize);
		check(DecoderOutputSize == ActionDistributionVectorSize);

		DecoderNetwork->NeuralNetworkData->Init(DecoderInputSize, DecoderOutputSize, ActionCompatibilityHash, FileData);
		DecoderNetwork->ForceMarkDirty();
	}

	// Create Encoder / Policy / Decoder Objects
	UE::Learning::FNeuralNetworkInferenceSettings InferenceSettings;
	InferenceSettings.bParallelEvaluation = PolicySettings.bUseParallelEvaluation;

	EncoderObject = MakeShared<UE::Learning::FNeuralNetworkFunction>(
		Manager->GetMaxAgentNum(),
		EncoderNetwork->NeuralNetworkData->GetNetwork(),
		InferenceSettings);

	PolicyObject = MakeShared<UE::Learning::FNeuralNetworkPolicy>(
		Manager->GetMaxAgentNum(),
		ObservationEncodedVectorSize,
		ActionEncodedVectorSize,
		MemoryStateSize,
		PolicyNetwork->NeuralNetworkData->GetNetwork(),
		InferenceSettings);

	DecoderObject = MakeShared<UE::Learning::FNeuralNetworkFunction>(
		Manager->GetMaxAgentNum(),
		DecoderNetwork->NeuralNetworkData->GetNetwork(),
		InferenceSettings);

	// State Variables

	GlobalSeed = Seed;
	Seeds.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	UE::Learning::Array::Set<1, uint32>(Seeds, INDEX_NONE);

	ObservationVectorsEncoded.SetNumUninitialized({ Manager->GetMaxAgentNum(), ObservationEncodedVectorSize });
	ActionVectorsEncoded.SetNumUninitialized({ Manager->GetMaxAgentNum(), ActionEncodedVectorSize });
	ActionDistributionVectors.SetNumUninitialized({ Manager->GetMaxAgentNum(), ActionDistributionVectorSize });
	UE::Learning::Array::Set(ObservationVectorsEncoded, FLT_MAX);
	UE::Learning::Array::Set(ActionVectorsEncoded, FLT_MAX);
	UE::Learning::Array::Set(ActionDistributionVectors, FLT_MAX);

	PreEvaluationMemoryState.SetNumUninitialized({ Manager->GetMaxAgentNum(), MemoryStateSize });
	MemoryState.SetNumUninitialized({ Manager->GetMaxAgentNum(), MemoryStateSize });
	UE::Learning::Array::Set(PreEvaluationMemoryState, FLT_MAX);
	UE::Learning::Array::Set(MemoryState, FLT_MAX);

	ObservationVectorEncodedIteration.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	ActionVectorEncodedIteration.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	MemoryStateIteration.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	UE::Learning::Array::Set<1, uint64>(ObservationVectorEncodedIteration, INDEX_NONE);
	UE::Learning::Array::Set<1, uint64>(ActionVectorEncodedIteration, INDEX_NONE);
	UE::Learning::Array::Set<1, uint64>(MemoryStateIteration, INDEX_NONE);

	bIsSetup = true;

	Manager->AddListener(this);
}

TSharedRef<FJsonObject> ULearningAgentsPolicy::AsJsonConfig() const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	ConfigObject->SetNumberField(TEXT("HiddenLayerNum"), PolicySettings.HiddenLayerNum);
	ConfigObject->SetNumberField(TEXT("HiddenLayerSize"), PolicySettings.HiddenLayerSize);
	ConfigObject->SetStringField(TEXT("MemoryCell"), StaticEnum<ELearningAgentsMemoryCell>()->GetNameStringByValue(static_cast<int64>(PolicySettings.MemoryCell)));
	ConfigObject->SetNumberField(TEXT("MemoryStateSize"), GetMemoryStateSize());
	ConfigObject->SetNumberField(TEXT("InitialEncodedActionScale"), PolicySettings.InitialEncodedActionScale);
	ConfigObject->SetStringField(TEXT("ActivationFunction"), StaticEnum<ELearningAgentsActivationFunction>()->GetNameStringByValue(static_cast<int64>(PolicySettings.ActivationFunction)));
	return ConfigObject;
}

void ULearningAgentsPolicy::OnAgentsAdded_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Random::SampleIntArray(Seeds, GlobalSeed, AgentIds);
	UE::Learning::Array::Set<2, float>(ObservationVectorsEncoded, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<2, float>(ActionVectorsEncoded, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<2, float>(ActionDistributionVectors, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<2, float>(PreEvaluationMemoryState, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(MemoryState, 0.0f, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ObservationVectorEncodedIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ActionVectorEncodedIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, uint64>(MemoryStateIteration, 0, AgentIds);
}

void ULearningAgentsPolicy::OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Random::SampleIntArray(Seeds, GlobalSeed, AgentIds);
	UE::Learning::Array::Set<2, float>(ObservationVectorsEncoded, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<2, float>(ActionVectorsEncoded, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<2, float>(ActionDistributionVectors, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<2, float>(PreEvaluationMemoryState, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<2, float>(MemoryState, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ObservationVectorEncodedIteration, INDEX_NONE, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ActionVectorEncodedIteration, INDEX_NONE, AgentIds);
	UE::Learning::Array::Set<1, uint64>(MemoryStateIteration, INDEX_NONE, AgentIds);
}

void ULearningAgentsPolicy::OnAgentsReset_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Random::SampleIntArray(Seeds, GlobalSeed, AgentIds);
	UE::Learning::Array::Set<2, float>(ObservationVectorsEncoded, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<2, float>(ActionVectorsEncoded, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<2, float>(ActionDistributionVectors, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<2, float>(PreEvaluationMemoryState, 0.0f, AgentIds);
	UE::Learning::Array::Set<2, float>(MemoryState, 0.0f, AgentIds);
	UE::Learning::Array::Set<1, uint64>(MemoryStateIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ObservationVectorEncodedIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, uint64>(ActionVectorEncodedIteration, 0, AgentIds);
}

UE::Learning::FNeuralNetworkFunction& ULearningAgentsPolicy::GetEncoderObject()
{
	return *EncoderObject;
}

UE::Learning::FNeuralNetworkPolicy& ULearningAgentsPolicy::GetPolicyObject()
{
	return *PolicyObject;
}

UE::Learning::FNeuralNetworkFunction& ULearningAgentsPolicy::GetDecoderObject()
{
	return *DecoderObject;
}

TLearningArrayView<2, const float> ULearningAgentsPolicy::GetPreEvaluationMemoryState() const
{
	return PreEvaluationMemoryState;
}

TLearningArrayView<2, const float> ULearningAgentsPolicy::GetMemoryState() const
{
	return MemoryState;
}

TLearningArrayView<2, const float> ULearningAgentsPolicy::GetObservationVectorsEncoded() const
{
	return ObservationVectorsEncoded;
}

bool ULearningAgentsPolicy::HasEncodedObservationsForAgent(const int32 AgentId) const
{
	return ObservationVectorEncodedIteration[AgentId] > 0;
}

ULearningAgentsNeuralNetwork* ULearningAgentsPolicy::GetEncoderNetworkAsset()
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return nullptr;
	}

	return EncoderNetwork;
}

ULearningAgentsNeuralNetwork* ULearningAgentsPolicy::GetPolicyNetworkAsset()
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return nullptr;
	}

	return PolicyNetwork;
}

ULearningAgentsNeuralNetwork* ULearningAgentsPolicy::GetDecoderNetworkAsset()
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return nullptr;
	}

	return DecoderNetwork;
}

void ULearningAgentsPolicy::EncodeObservations()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsPolicy::EncodeObservations);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	// Check which agents have had their observation vector set

	ValidAgentIds.Empty(Manager->GetMaxAgentNum());
	for (const int32 AgentId : Manager->GetAllAgentSet())
	{
		if (Interactor->GetObservationIteration(AgentId) == 0)
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i does not have an observation vector ready be encoded. Was GatherObservations run without error?", *GetName(), AgentId);
			continue;
		}

		ValidAgentIds.Add(AgentId);
	}

	ValidAgentSet = ValidAgentIds;
	ValidAgentSet.TryMakeSlice();

	// Encode Observations

	if (EncoderObject->GetNeuralNetwork()->GetInputSize() != Interactor->GetObservationVectorSize())
	{
		UE_LOGF(LogLearning, Error, "%ls: Encoder Network Input size doesn't match. Network input size is %i but Encoder expects %i.", *GetName(),
			EncoderObject->GetNeuralNetwork()->GetInputSize(), Interactor->GetObservationVectorSize());
		return;
	}

	if (EncoderObject->GetNeuralNetwork()->GetOutputSize() != ObservationVectorsEncoded.Num<1>())
	{
		UE_LOGF(LogLearning, Error, "%ls: Encoder Network Output size don't match. Network output size is %i but Encoder expects %lli.", *GetName(),
			EncoderObject->GetNeuralNetwork()->GetOutputSize(), ObservationVectorsEncoded.Num<1>());
		return;
	}

	EncoderObject->Evaluate(ObservationVectorsEncoded, Interactor->GetObservationVectorsArrayView(), ValidAgentSet);

	for (const int32 AgentId : ValidAgentSet)
	{
		ObservationVectorEncodedIteration[AgentId]++;
	}
}

void ULearningAgentsPolicy::EvaluatePolicy()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsPolicy::EvaluatePolicy);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	// Check agents actually have encoded observations

	ValidAgentIds.Empty(Manager->GetMaxAgentNum());

	for (const int32 AgentId : Manager->GetAllAgentSet())
	{
		if (ObservationVectorEncodedIteration[AgentId] == 0)
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i has not encoded observations so policy will not be evaluated for it. Was EncodeObservations run without error?", *GetName(), AgentId);
			continue;
		}

		ValidAgentIds.Add(AgentId);
	}

	ValidAgentSet = ValidAgentIds;
	ValidAgentSet.TryMakeSlice();

	// Copy pre-evaluation memory state

	UE::Learning::Array::Copy<2, float>(PreEvaluationMemoryState, MemoryState, ValidAgentSet);

	// Evaluate policy

	if (PolicyObject->GetNeuralNetwork()->GetInputSize() != ObservationVectorsEncoded.Num<1>() + MemoryState.Num<1>())
	{
		UE_LOGF(LogLearning, Error, "%ls: Policy Network Input size doesn't match. Network input size is %i but Policy expects %lli.", *GetName(),
			PolicyObject->GetNeuralNetwork()->GetInputSize(), ObservationVectorsEncoded.Num<1>() + MemoryState.Num<1>());
		return;
	}

	if (PolicyObject->GetNeuralNetwork()->GetOutputSize() != ActionVectorsEncoded.Num<1>() + MemoryState.Num<1>())
	{
		UE_LOGF(LogLearning, Error, "%ls: Policy Network Output size don't match. Network output size is %i but Policy expects %lli.", *GetName(),
			PolicyObject->GetNeuralNetwork()->GetOutputSize(), ActionVectorsEncoded.Num<1>() + MemoryState.Num<1>());
		return;
	}

	PolicyObject->Evaluate(
		ActionVectorsEncoded,
		MemoryState,
		ObservationVectorsEncoded,
		PreEvaluationMemoryState,
		ValidAgentSet);

	// Increment policy evaluation and action generation iteration

	for (const int32 AgentId : ValidAgentSet)
	{
		MemoryStateIteration[AgentId]++;
		ActionVectorEncodedIteration[AgentId]++;
	}
}

void ULearningAgentsPolicy::DecodeAndSampleActions(const float ActionNoiseScale)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsPolicy::DecodeAndSampleActions);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	// Check which agents have had their encoded action vector set and action modifiers made

	ValidAgentIds.Empty(Manager->GetMaxAgentNum());

	for (const int32 AgentId : Manager->GetAllAgentSet())
	{
		if (ActionVectorEncodedIteration[AgentId] == 0)
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i does not have an encoded action so actions will not be decoded for it. Was EvaluatePolicy run without error?", *GetName(), AgentId);
			continue;
		}

		if (Interactor->GetActionModifierIteration(AgentId) == 0)
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i does not have an action modifier made so actions will not be decoded for it. Was MakeActionModifiers run without error?", *GetName(), AgentId);
			continue;
		}

		ValidAgentIds.Add(AgentId);
	}

	ValidAgentSet = ValidAgentIds;
	ValidAgentSet.TryMakeSlice();

	// Decode to produce action distribution vectors

	if (DecoderObject->GetNeuralNetwork()->GetInputSize() != ActionVectorsEncoded.Num<1>())
	{
		UE_LOGF(LogLearning, Error, "%ls: Decoder Network Input size doesn't match. Network input size is %i but Decoder expects %lli.", *GetName(),
			DecoderObject->GetNeuralNetwork()->GetInputSize(), ActionVectorsEncoded.Num<1>());
		return;
	}

	if (DecoderObject->GetNeuralNetwork()->GetOutputSize() != ActionDistributionVectors.Num<1>())
	{
		UE_LOGF(LogLearning, Error, "%ls: Decoder Network Output size don't match. Network output size is %i but Decoder expects %lli.", *GetName(),
			DecoderObject->GetNeuralNetwork()->GetOutputSize(), ActionDistributionVectors.Num<1>());
		return;
	}

	DecoderObject->Evaluate(ActionDistributionVectors, ActionVectorsEncoded, ValidAgentSet);
	
	// Sample actions from action distribution vectors

	for (const int32 AgentId : ValidAgentSet)
	{
		UE::Learning::Action::SampleVectorFromDistributionVector(
			Seeds[AgentId],
			Interactor->GetActionVectorsArrayView()[AgentId],
			ActionDistributionVectors[AgentId],
			Interactor->GetActionModifierVectorsArrayView()[AgentId],
			Interactor->GetActionSchema()->ActionSchema,
			Interactor->GetActionSchemaElement().SchemaElement,
			ActionNoiseScale);
	}

	for (const int32 AgentId : ValidAgentSet)
	{
		Interactor->GetActionVectorIterationArrayView()[AgentId]++;
	}
}

void ULearningAgentsPolicy::RunInference(const float ActionNoiseScale)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsPolicy::RunInference);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	Interactor->GatherObservations();
	Interactor->MakeActionModifiers();
	EncodeObservations();
	EvaluatePolicy();
	DecodeAndSampleActions(ActionNoiseScale);
	Interactor->PerformActions();
}

void ULearningAgentsPolicy::GetMemoryState(TArray<float>& OutMemoryState, const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		OutMemoryState.Empty();
		return;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		OutMemoryState.Empty();
		return;
	}

	OutMemoryState.SetNumUninitialized(MemoryState.Num<1>());
	UE::Learning::Array::Copy<1, float>(OutMemoryState, MemoryState[AgentId]);
}

void ULearningAgentsPolicy::SetMemoryState(const int32 AgentId, const TArray<float>& InMemoryState)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return;
	}

	if (InMemoryState.Num() != MemoryState.Num<1>())
	{
		UE_LOGF(LogLearning, Error, "%ls: Memory State is incorrect size. Expected %lli, got %i.", *GetName(), MemoryState.Num<1>(), InMemoryState.Num());
		return;
	}

	UE::Learning::Array::Copy<1, float>(MemoryState[AgentId], InMemoryState);
}

int32 ULearningAgentsPolicy::GetMemoryStateSize() const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0;
	}

	return MemoryState.Num<1>();
}
