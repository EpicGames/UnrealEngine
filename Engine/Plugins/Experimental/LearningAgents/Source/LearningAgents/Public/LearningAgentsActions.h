// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningAction.h"

#include "LearningAgentsNeuralNetwork.h" // Included for ELearningAgentsActivationFunction

#include "Engine/EngineTypes.h"
#include "GameFramework/OnlineReplStructs.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "LearningAgentsActions.generated.h"

#define UE_API LEARNINGAGENTS_API

class ULearningAgentsActionSchema;
class ULearningAgentsActionObject;
class ULearningAgentsActionModifier;
struct FLearningAgentsActionSchemaElement;
struct FLearningAgentsActionObjectElement;
struct FLearningAgentsActionModifierElement;

/** An element of an Action Schema */
USTRUCT(BlueprintType)
struct FLearningAgentsActionSchemaElement
{
	GENERATED_BODY()

	UE::Learning::Action::FSchemaElement SchemaElement;
};

/** An element of an Action Object */
USTRUCT(BlueprintType)
struct FLearningAgentsActionObjectElement
{
	GENERATED_BODY()

	UE::Learning::Action::FObjectElement ObjectElement;
};

/** An element of an Action Modifier */
USTRUCT(BlueprintType)
struct FLearningAgentsActionModifierElement
{
	GENERATED_BODY()

	UE::Learning::Action::FModifierElement ModifierElement;
};

/** Settings for normalizing observations */
USTRUCT(BlueprintType)
struct FLearningAgentsActionNormalizationSettings
{
	GENERATED_BODY()

	/** The normalization method to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	ELearningAgentsNormalization NormalizationOption = ELearningAgentsNormalization::Manual;

	/** Per-element standard deviation values used for normalization. If empty, no scaling is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	TArray<float> Scales;

	/** Per-element mean values used for normalization. If empty, no bias is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	TArray<float> Offsets;
};

/** Settings for normalizing observations */
USTRUCT(BlueprintType)
struct FLearningAgentsFloatActionNormalizationSettings
{
	GENERATED_BODY()

	/** The normalization method to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	ELearningAgentsNormalization NormalizationOption = ELearningAgentsNormalization::Manual;

	/** Per-element standard deviation values used for normalization. If empty, no scaling is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	float Scale = 1.0f;

	/** Per-element mean values used for normalization. If empty, no bias is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	float Offset = 0.0f;
};

/** Settings for normalizing observations */
USTRUCT(BlueprintType)
struct FLearningAgentsVectorActionNormalizationSettings
{
	GENERATED_BODY()

	/** The normalization method to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	ELearningAgentsNormalization NormalizationOption = ELearningAgentsNormalization::Manual;

	/** Per-element standard deviation values used for normalization. If empty, no scaling is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	FVector Scales = FVector::OneVector;

	/** Per-element mean values used for normalization. If empty, no bias is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	FVector Offsets = FVector::ZeroVector;
};

class ULearningAgentsManagerListener;

/** Settings for visual logger output */
USTRUCT(BlueprintType)
struct FLearningAgentsVisualLoggerSettings
{
	GENERATED_BODY()

	/** Whether visual logging is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	bool bEnabled = false;

	/** The listener object which is making this action. This must be set to use logging. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsManagerListener> Listener = nullptr;

	/** The agent id associated with this action. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	int32 AgentId = -1;

	/** An additional location for visualizing a specific action element (e.g., rotation arrow origin, velocity arrow start). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	FVector DebugLocation = FVector::ZeroVector;

	/** A location for the visual logger text information in the world. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	FVector Location = FVector::ZeroVector;

	/** The color for the visual logger display. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	FLinearColor Color = FLinearColor::Blue;

	/** The length of debug arrows to display for direction/velocity actions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	float ArrowLength = 100.0f;
};

/** Comparison and Hashing operators for Action Elements */

LEARNINGAGENTS_API bool operator==(const FLearningAgentsActionSchemaElement& Lhs, const FLearningAgentsActionSchemaElement& Rhs);
LEARNINGAGENTS_API bool operator==(const FLearningAgentsActionObjectElement& Lhs, const FLearningAgentsActionObjectElement& Rhs);
LEARNINGAGENTS_API bool operator==(const FLearningAgentsActionModifierElement& Lhs, const FLearningAgentsActionModifierElement& Rhs);

LEARNINGAGENTS_API uint32 GetTypeHash(const FLearningAgentsActionSchemaElement& Element);
LEARNINGAGENTS_API uint32 GetTypeHash(const FLearningAgentsActionObjectElement& Element);
LEARNINGAGENTS_API uint32 GetTypeHash(const FLearningAgentsActionModifierElement& Element);

template<>
struct TStructOpsTypeTraits<FLearningAgentsActionSchemaElement> : public TStructOpsTypeTraitsBase2<FLearningAgentsActionSchemaElement>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};

template<>
struct TStructOpsTypeTraits<FLearningAgentsActionObjectElement> : public TStructOpsTypeTraitsBase2<FLearningAgentsActionObjectElement>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};

template<>
struct TStructOpsTypeTraits<FLearningAgentsActionModifierElement> : public TStructOpsTypeTraitsBase2<FLearningAgentsActionModifierElement>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};

/**
 * Action Schema
 *
 * This object is used to construct a schema describing some structure of actions.
 */
UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsActionSchema : public UObject
{
	GENERATED_BODY()

public:

	UE::Learning::Action::FSchema ActionSchema;
};

/**
 * Action Object
 *
 * This object is used to construct or get the values of actions.
 */
UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsActionObject : public UObject
{
	GENERATED_BODY()

public:

	UE::Learning::Action::FObject ActionObject;
};

/**
 * Action Modifier
 *
 * This object is used to construct or get the values of action modifiers.
 */
UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsActionModifier : public UObject
{
	GENERATED_BODY()

public:

	UE::Learning::Action::FModifier ActionModifier;
};

/** Enum Type representing either action A or action B */
UENUM(BlueprintType)
enum class ELearningAgentsEitherAction : uint8
{
	A,
	B,
};

/** Enum Type representing either a Null action or some Valid action */
UENUM(BlueprintType)
enum class ELearningAgentsOptionalAction : uint8
{
	Null,
	Valid,
};

UCLASS(MinimalAPI)
class ULearningAgentsActions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Validates that the given action object matches the schema. Will log errors on objects that don't match.
	 *
	 * @param Schema				Action Schema
	 * @param SchemaElement			Action Schema Element
	 * @param Object				Action Object
	 * @param ObjectElement			Action Object Element
	 * @returns						true if the object matches the schema
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents")
	static UE_API bool ValidateActionObjectMatchesSchema(
		const ULearningAgentsActionSchema* Schema,
		const FLearningAgentsActionSchemaElement SchemaElement,
		const ULearningAgentsActionObject* Object,
		const FLearningAgentsActionObjectElement ObjectElement);

	/**
	 * Validates that the given action modifier matches the schema. Will log errors on modifiers that don't match.
	 *
	 * @param Schema				Action Schema
	 * @param SchemaElement			Action Schema Element
	 * @param Modifier				Action Modifier
	 * @param ModifierElement		Action Modifier Element
	 * @returns						true if the modifier matches the schema
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents")
	static UE_API bool ValidateActionModifierMatchesSchema(
		const ULearningAgentsActionSchema* Schema,
		const FLearningAgentsActionSchemaElement SchemaElement,
		const ULearningAgentsActionModifier* Modifier,
		const FLearningAgentsActionModifierElement ModifierElement);

	/**
	 * Logs an Action Object Element. Useful for debugging.
	 *
	 * @param Object				Action Object
	 * @param ObjectElement			Action Object Element
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents")
	static UE_API void LogAction(const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element);

public:

	/**
	 * Specifies a new null action. This represents an empty action and can be useful when an action is needed which does nothing.
	 *
	 * @param Schema The Action Schema
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsActionSchemaElement SpecifyNullAction(ULearningAgentsActionSchema* Schema, const FName Tag = TEXT("NullAction"));

	/**
	 * Specifies a new continuous action. This represents an action made up of several float values sampled from a Gaussian distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Size The number of float values in the action.
	 * @param Normalization The normalization used for this action.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "NormalizationSettings"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyContinuousAction(ULearningAgentsActionSchema* Schema, const int32 Size, const FLearningAgentsActionNormalizationSettings& NormalizationSettings = FLearningAgentsActionNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("ContinuousAction"));

	/**
	 * Specifies a new exclusive discrete action. This represents an action which is an exclusive choice from a number of discrete options, sampled
	 * from a Categorical distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Size The number of discrete options in the action.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a uniform distribution over options. Should sum to one.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyExclusiveDiscreteAction(ULearningAgentsActionSchema* Schema, const int32 Size, const TArray<float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("DiscreteExclusiveAction"));
	
	/**
	 * Specifies a new exclusive discrete action. This represents an action which is an exclusive choice from a number of discrete options, sampled
	 * from a Categorical distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Size The number of discrete options in the action.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a uniform distribution over options. Should sum to one.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	static UE_API FLearningAgentsActionSchemaElement SpecifyExclusiveDiscreteActionFromArrayView(ULearningAgentsActionSchema* Schema, const int32 Size, const TArrayView<const float> PriorProbabilities = {}, const bool bActive = true, const FName Tag = TEXT("DiscreteExclusiveAction"));

	/**
	 * Specifies a new named exclusive discrete action. This represents an action which is an exclusive choice from a number of discrete options, sampled
	 * from a Categorical distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Names The names of the discrete options in the action.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a uniform distribution over options. Should sum to one.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyNamedExclusiveDiscreteAction(ULearningAgentsActionSchema* Schema, const TArray<FName>& Names, const TMap<FName, float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("NamedDiscreteExclusiveAction"));

	/**
	 * Specifies a new named exclusive discrete action. This represents an action which is an exclusive choice from a number of discrete options, sampled
	 * from a Categorical distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Names The names of the discrete options in the action.
	 * @param PriorProbabilityNames The names of the prior probabilities. Can be left empty to use a uniform distribution over options.
	 * @param PriorProbabilityValues The prior probabilities of each option. Can be left empty to use a uniform distribution over options. Should sum to one.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	static UE_API FLearningAgentsActionSchemaElement SpecifyNamedExclusiveDiscreteActionFromArrayViews(ULearningAgentsActionSchema* Schema, const TArrayView<const FName> Names, const TArrayView<const float> PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("NamedDiscreteExclusiveAction"));

	/**
	 * Specifies a new inclusive discrete action. This represents an action which is an inclusive choice from a number of discrete options, sampled
	 * from a Bernoulli distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Size The number of discrete options in the action.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a probability of 0.5 for each option.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyInclusiveDiscreteAction(ULearningAgentsActionSchema* Schema, const int32 Size, const TArray<float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("DiscreteInclusiveAction"));
	
	/**
	 * Specifies a new inclusive discrete action. This represents an action which is an inclusive choice from a number of discrete options, sampled
	 * from a Bernoulli distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Size The number of discrete options in the action.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a probability of 0.5 for each option.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	static UE_API FLearningAgentsActionSchemaElement SpecifyInclusiveDiscreteActionFromArrayView(ULearningAgentsActionSchema* Schema, const int32 Size, const TArrayView<const float> PriorProbabilities = {}, const bool bActive = true, const FName Tag = TEXT("DiscreteInclusiveAction"));

	/**
	 * Specifies a new named inclusive discrete action. This represents an action which is an inclusive choice from a number of discrete options, sampled
	 * from a Bernoulli distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Names The names of the discrete options in the action.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a probability of 0.5 for each option.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyNamedInclusiveDiscreteAction(ULearningAgentsActionSchema* Schema, const TArray<FName> Names, const TMap<FName, float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("NamedDiscreteInclusiveAction"));

	/**
	 * Specifies a new named inclusive discrete action. This represents an action which is an inclusive choice from a number of discrete options, sampled
	 * from a Bernoulli distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Names The names of the discrete options in the action.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a probability of 0.5 for each option.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	static UE_API FLearningAgentsActionSchemaElement SpecifyNamedInclusiveDiscreteActionFromArrayViews(ULearningAgentsActionSchema* Schema, const TArrayView<const FName> Names, const TArrayView<const float> PriorProbabilities = {}, const bool bActive = true, const FName Tag = TEXT("NamedDiscreteInclusiveAction"));

	/**
	 * Specifies a new struct action. This represents an action which is made up of a number of named sub-actions.
	 *
	 * @param Schema The Action Schema
	 * @param Elements The sub-actions.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionSchemaElement SpecifyStructAction(ULearningAgentsActionSchema* Schema, const TMap<FName, FLearningAgentsActionSchemaElement>& Elements, const bool bActive = true, const FName Tag = TEXT("StructAction"));

	/**
	 * Specifies a new struct action. This represents an action which is made up of a number of named sub-actions.
	 *
	 * @param Schema The Action Schema
	 * @param ElementNames The names of the sub-actions.
	 * @param Elements The corresponding sub-actions. Must be the same size as ElementNames.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionSchemaElement SpecifyStructActionFromArrays(ULearningAgentsActionSchema* Schema, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionSchemaElement>& Elements, const bool bActive = true, const FName Tag = TEXT("StructAction"));
	
	/**
	 * Specifies a new struct action. This represents an action which is made up of a number of named sub-actions.
	 *
	 * @param Schema The Action Schema
	 * @param ElementNames The names of the sub-actions.
	 * @param Elements The corresponding sub-actions. Must be the same size as ElementNames.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	static UE_API FLearningAgentsActionSchemaElement SpecifyStructActionFromArrayViews(ULearningAgentsActionSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionSchemaElement> Elements, const bool bActive = true, const FName Tag = TEXT("StructAction"));

	/**
	 * Specifies a new exclusive union action. This represents an action which is an exclusive choice from a number of named sub-actions, sampled
	 * from a Categorical distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Elements The sub-actions.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a uniform distribution over options. Should sum to one.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyExclusiveUnionAction(ULearningAgentsActionSchema* Schema, const TMap<FName, FLearningAgentsActionSchemaElement>& Elements, const TMap<FName, float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("ExclusiveUnionAction"));

	/**
	 * Specifies a new exclusive union action. This represents an action which is an exclusive choice from a number of named sub-actions, sampled
	 * from a Categorical distribution.
	 *
	 * @param Schema The Action Schema
	 * @param ElementNames The names of the sub-actions.
	 * @param Elements The corresponding sub-actions. Must be the same size as ElementNames.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a uniform distribution over options. Should sum to one.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyExclusiveUnionActionFromArrays(ULearningAgentsActionSchema* Schema, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionSchemaElement>& Elements, const TArray<float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("ExclusiveUnionAction"));
	
	/**
	 * Specifies a new exclusive union action. This represents an action which is an exclusive choice from a number of named sub-actions, sampled
	 * from a Categorical distribution.
	 *
	 * @param Schema The Action Schema
	 * @param ElementNames The names of the sub-actions.
	 * @param Elements The corresponding sub-actions. Must be the same size as ElementNames.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a uniform distribution over options. Should sum to one.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	static UE_API FLearningAgentsActionSchemaElement SpecifyExclusiveUnionActionFromArrayViews(ULearningAgentsActionSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionSchemaElement> Elements, const TArrayView<const float> PriorProbabilities = {}, const bool bActive = true, const FName Tag = TEXT("ExclusiveUnionAction"));

	/**
	 * Specifies a new inclusive union action. This represents an action which is an inclusive choice from a number of named sub-actions, sampled
	 * from a Bernoulli distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Elements The sub-actions.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a probability of 0.5 for each option.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyInclusiveUnionAction(ULearningAgentsActionSchema* Schema, const TMap<FName, FLearningAgentsActionSchemaElement>& Elements, const TMap<FName, float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionAction"));

	/**
	 * Specifies a new inclusive union action. This represents an action which is an inclusive choice from a number of named sub-actions, sampled
	 * from a Bernoulli distribution.
	 *
	 * @param Schema The Action Schema
	 * @param ElementNames The names of the sub-actions.
	 * @param Elements The corresponding sub-actions. Must be the same size as ElementNames.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a probability of 0.5 for each option.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyInclusiveUnionActionFromArrays(ULearningAgentsActionSchema* Schema, const TArray<FName> ElementNames, const TArray<FLearningAgentsActionSchemaElement>& Elements, const TArray<float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionAction"));
	
	/**
	 * Specifies a new inclusive union action. This represents an action which is an inclusive choice from a number of named sub-actions, sampled
	 * from a Bernoulli distribution.
	 *
	 * @param Schema The Action Schema
	 * @param ElementNames The names of the sub-actions.
	 * @param Elements The corresponding sub-actions. Must be the same size as ElementNames.
	 * @param PriorProbabilities The prior probabilities of each option. Can be left empty to use a probability of 0.5 for each option.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	static UE_API FLearningAgentsActionSchemaElement SpecifyInclusiveUnionActionFromArrayViews(ULearningAgentsActionSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionSchemaElement> Elements, const TArrayView<const float> PriorProbabilities = {}, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionAction"));

	/**
	 * Specifies a new static array action. This represents an action which is a fixed sized array of some sub-action.
	 *
	 * @param Schema The Action Schema
	 * @param Element The sub-action.
	 * @param Num The number of elements in the array.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionSchemaElement SpecifyStaticArrayAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsActionSchemaElement Element, const int32 Num, const bool bActive = true, const FName Tag = TEXT("StaticArrayAction"));

	/**
	 * Specifies a new pair action. This represents an action which is made up of a key and value sub-actions.
	 *
	 * @param Schema The Action Schema
	 * @param Key The key sub-action.
	 * @param Value The value sub-action.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionSchemaElement SpecifyPairAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsActionSchemaElement Key, const FLearningAgentsActionSchemaElement Value, const bool bActive = true, const FName Tag = TEXT("PairAction"));

	/**
	 * Specifies a new enum action. This represents an action which is an exclusive choice from entries of an Enum, sampled from a Categorical distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Enum The Enum type.
	 * @param PriorProbabilities The prior probabilities of each enum element. Can be left empty to use a uniform distribution over elements. Should sum to one.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyEnumAction(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TMap<uint8, float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("EnumAction"));

	/**
	 * Specifies a new enum action. This represents an action which is an exclusive choice from entries of an Enum, sampled from a Categorical distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Enum The Enum type.
	 * @param PriorProbabilities The prior probabilities of each enum element. Can be left empty to use a uniform distribution over elements. Should sum to one.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyEnumActionFromArray(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TArray<float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("EnumAction"));
	
	/**
	 * Specifies a new enum action. This represents an action which is an exclusive choice from entries of an Enum, sampled from a Categorical distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Enum The Enum type.
	 * @param PriorProbabilities The prior probabilities of each enum element. Can be left empty to use a uniform distribution over elements. Should sum to one.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	static UE_API FLearningAgentsActionSchemaElement SpecifyEnumActionFromArrayView(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TArrayView<const float> PriorProbabilities = {}, const bool bActive = true, const FName Tag = TEXT("EnumAction"));

	/**
	 * Specifies a new bitmask action. This represents an action which is an inclusive choice from entries of an Enum, sampled from a Bernoulli 
	 * distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Enum The Enum type.
	 * @param PriorProbabilities The prior probabilities of each enum element. Can be left empty to use a probability of 0.5 for each element.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyBitmaskAction(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TMap<uint8, float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("BitmaskAction"));

	/**
	 * Specifies a new bitmask action. This represents an action which is an inclusive choice from entries of an Enum, sampled from a Bernoulli
	 * distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Enum The Enum type.
	 * @param PriorProbabilities The prior probabilities of each enum element. Can be left empty to use a probability of 0.5 for each element.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "PriorProbabilities"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyBitmaskActionFromArray(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TArray<float>& PriorProbabilities, const bool bActive = true, const FName Tag = TEXT("BitmaskAction"));
	
	/**
	 * Specifies a new bitmask action. This represents an action which is an inclusive choice from entries of an Enum, sampled from a Bernoulli
	 * distribution.
	 *
	 * @param Schema The Action Schema
	 * @param Enum The Enum type.
	 * @param PriorProbabilities The prior probabilities of each enum element. Can be left empty to use a probability of 0.5 for each element.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	static UE_API FLearningAgentsActionSchemaElement SpecifyBitmaskActionFromArrayView(ULearningAgentsActionSchema* Schema, const UEnum* Enum, const TArrayView<const float> PriorProbabilities = {}, const bool bActive = true, const FName Tag = TEXT("BitmaskAction"));

	/**
	 * Specifies a new optional action. This represents an action which may or may not be generated.
	 *
	 * @param Schema The Action Schema
	 * @param Element The sub-action.
	 * @param PriorProbabilities The prior probability of sampling this action.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionSchemaElement SpecifyOptionalAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsActionSchemaElement Element, const float PriorProbability = 0.5f, const bool bActive = true, const FName Tag = TEXT("OptionalAction"));

	/**
	 * Specifies a new either action. This represents an action which is either action A or action B.
	 *
	 * @param Schema The Action Schema
	 * @param A The sub-action A.
	 * @param B The sub-action B.
	 * @param PriorProbabilityOfA The prior probability of sampling action A over action B.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionSchemaElement SpecifyEitherAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsActionSchemaElement A, const FLearningAgentsActionSchemaElement B, const float PriorProbabilityOfA = 0.5f, const bool bActive = true, const FName Tag = TEXT("EitherAction"));

	/**
	 * Specifies a new encoding action. This represents an action which will be a decoding of another sub-action using a small neural network.
	 *
	 * @param Schema The Action Schema
	 * @param Element The sub-action.
	 * @param EncodingSize The encoding size used to decode this sub-action.
	 * @param HiddenLayerNum The number of hidden layers used to decode this sub-action.
	 * @param ActivationFunction The activation function used to decode this sub-action.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionSchemaElement SpecifyEncodingAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsActionSchemaElement Element, const int32 EncodingSize = 128, const int32 HiddenLayerNum = 1, const ELearningAgentsActivationFunction ActivationFunction = ELearningAgentsActivationFunction::ELU, const bool bActive = true, const FName Tag = TEXT("EncodingAction"));

	/**
	 * Specifies a new bool action. This represents an action which is either true or false.
	 *
	 * @param Schema The Action Schema
	 * @param PriorProbability The prior probability of this action being true.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsActionSchemaElement SpecifyBoolAction(ULearningAgentsActionSchema* Schema, const float PriorProbability = 0.5f, const bool bActive = true, const FName Tag = TEXT("BoolAction"));

	/**
	 * Specifies a new float action. This represents an action which is a single float sampled from a Gaussian distribution. It can be used as a
	 * catch-all for situations where a type-specific action does not exist.
	 *
	 * @param Schema The Action Schema
	 * @param NormalizationSettings The normalization used for this action.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyFloatAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsFloatActionNormalizationSettings& NormalizationSettings = FLearningAgentsFloatActionNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("FloatAction"));

	/**
	 * Specifies a new location action. This represents an action which is a location sampled from a Gaussian distribution.
	 *
	 * @param Schema The Action Schema
	 * @param NormalizationSettings The normalization used for this action in cm.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings", NormalizationSettings = "(Scales=(X=100,Y=100,Z=100))"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyLocationAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsVectorActionNormalizationSettings& NormalizationSettings = FLearningAgentsVectorActionNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("LocationAction"));

	/**
	 * Specifies a new rotation action. This represents an action which is a rotation sampled from a Gaussian distribution in the angle-axis space.
	 *
	 * @param Schema The Action Schema
	 * @param NormalizationSettings The NormalizationSettings used for this action in degrees.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings", NormalizationSettings = "(Scales=(X=90,Y=90,Z=90))"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyRotationAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsVectorActionNormalizationSettings& NormalizationSettings = FLearningAgentsVectorActionNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("RotationAction"));

	/**
	 * Specifies a new scale action. This represents an action which is a scale sampled from a Gaussian distribution in the log space.
	 *
	 * @param Schema The Action Schema
	 * @param ScaleScale The scale used for this action.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyScaleAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsVectorActionNormalizationSettings& NormalizationSettings = FLearningAgentsVectorActionNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("ScaleAction"));

	/**
	 * Specifies a new transform action.
	 *
	 * @param Schema The Action Schema
	 * @param LocationNormalization The normalization used for the Location part of the transform in this action in cm.
	 * @param RotationNormalization The normalization used for the Rotation part of the transform in this action in degrees.
	 * @param ScaleNormalization The normalization used for the Scale part of the transform in this action.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4, AutoCreateRefTerm = "LocationNormalization, RotationNormalization, ScaleNormalization", LocationNormalization = "(Scales=(X=100,Y=100,Z=100))", RotationNormalization = "(Scales=(X=90,Y=90,Z=90))"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyTransformAction(
		ULearningAgentsActionSchema* Schema, 
		const FLearningAgentsVectorActionNormalizationSettings& LocationNormalization = FLearningAgentsVectorActionNormalizationSettings(),
		const FLearningAgentsVectorActionNormalizationSettings& RotationNormalization = FLearningAgentsVectorActionNormalizationSettings(),
		const FLearningAgentsVectorActionNormalizationSettings& ScaleNormalization = FLearningAgentsVectorActionNormalizationSettings(),
		const bool bActive = true, 
		const FName Tag = TEXT("TransformAction"));

	/**
	 * Specifies a new angle action. This represents an action which is an angle sampled from a Gaussian distribution centered around zero.
	 *
	 * @param Schema The Action Schema
	 * @param NormalizationSettings The normalization used for this action in degrees.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings", NormalizationSettings = "(Scale=90)"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyAngleAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsFloatActionNormalizationSettings& NormalizationSettings = FLearningAgentsFloatActionNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("AngleAction"));

	/**
	 * Specifies a new velocity action. This represents an action which is a velocity sampled from a Gaussian distribution.
	 *
	 * @param Schema The Action Schema
	 * @param NormalizationSettings The normalization used for this action in cm/s.
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings", NormalizationSettings = "(Scales=(X=200,Y=200,Z=200))"))
	static UE_API FLearningAgentsActionSchemaElement SpecifyVelocityAction(ULearningAgentsActionSchema* Schema, const FLearningAgentsVectorActionNormalizationSettings& NormalizationSettings = FLearningAgentsVectorActionNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("VelocityAction"));

	/**
	 * Specifies a new direction action. This represents an action which is a direction sampled from a Gaussian distribution and normalized.
	 *
	 * @param Schema The Action Schema
	 * @param bActive Whether to enable the specified action as a valid network output. If set to inactive, this action will be considered an empty action. This flag allows easy toggles of specified actions.
	 * @param Tag The tag of this new action. Used during action object validation and debugging.
	 * @return The newly created action schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsActionSchemaElement SpecifyDirectionAction(ULearningAgentsActionSchema* Schema, const bool bActive = true, const FName Tag = TEXT("DirectionAction"));

public:

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsActionObjectElement MakeNullAction(ULearningAgentsActionObject* Object, const FName Tag = TEXT("NullAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeContinuousAction(
		ULearningAgentsActionObject* Object,
		const TArray<float>& Values,
		const bool bActive = true,
		const FName Tag = TEXT("ContinuousAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	static UE_API FLearningAgentsActionObjectElement MakeContinuousActionFromArrayView(
		ULearningAgentsActionObject* Object,
		const TArrayView<const float> Values,
		const bool bActive = true,
		const FName Tag = TEXT("ContinuousAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeExclusiveDiscreteAction(
		ULearningAgentsActionObject* Object,
		const int32 Index,
		const bool bActive = true, 
		const FName Tag = TEXT("DiscreteExclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeNamedExclusiveDiscreteAction(
		ULearningAgentsActionObject* Object,
		const FName Name,
		const bool bActive = true, 
		const FName Tag = TEXT("NamedDiscreteExclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeInclusiveDiscreteAction(
		ULearningAgentsActionObject* Object,
		const TArray<int32>& Indices,
		const bool bActive = true, 
		const FName Tag = TEXT("DiscreteInclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	static UE_API FLearningAgentsActionObjectElement MakeInclusiveDiscreteActionFromArrayView(
		ULearningAgentsActionObject* Object,
		const TArrayView<const int32> Indices,
		const bool bActive = true,
		const FName Tag = TEXT("DiscreteInclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeNamedInclusiveDiscreteAction(
		ULearningAgentsActionObject* Object,
		const TArray<FName>& Names,
		const bool bActive = true, 
		const FName Tag = TEXT("NamedDiscreteInclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	static UE_API FLearningAgentsActionObjectElement MakeNamedInclusiveDiscreteActionFromArrayView(
		ULearningAgentsActionObject* Object,
		const TArrayView<const FName> Names,
		const bool bActive = true, 
		const FName Tag = TEXT("NamedDiscreteInclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionObjectElement MakeStructAction(ULearningAgentsActionObject* Object, const TMap<FName, FLearningAgentsActionObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("StructAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionObjectElement MakeStructActionFromArrays(ULearningAgentsActionObject* Object, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("StructAction"));
	static UE_API FLearningAgentsActionObjectElement MakeStructActionFromArrayViews(ULearningAgentsActionObject* Object, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionObjectElement> Elements, const bool bActive = true, const FName Tag = TEXT("StructAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionObjectElement MakeExclusiveUnionAction(ULearningAgentsActionObject* Object, const FName ElementName, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ExclusiveUnionAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionObjectElement MakeInclusiveUnionAction(ULearningAgentsActionObject* Object, const TMap<FName, FLearningAgentsActionObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionObjectElement MakeInclusiveUnionActionFromArrays(ULearningAgentsActionObject* Object, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionAction"));
	static UE_API FLearningAgentsActionObjectElement MakeInclusiveUnionActionFromArrayViews(ULearningAgentsActionObject* Object, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionObjectElement> Elements, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionObjectElement MakeStaticArrayAction(ULearningAgentsActionObject* Object, const TArray<FLearningAgentsActionObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("StaticArrayAction"));
	static UE_API FLearningAgentsActionObjectElement MakeStaticArrayActionFromArrayView(ULearningAgentsActionObject* Object, const TArrayView<const FLearningAgentsActionObjectElement> Elements, const bool bActive = true, const FName Tag = TEXT("StaticArrayAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionObjectElement MakePairAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Key, const FLearningAgentsActionObjectElement Value, const bool bActive = true, const FName Tag = TEXT("PairAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeEnumAction(
		ULearningAgentsActionObject* Object,
		const UEnum* Enum,
		const uint8 EnumValue,
		const bool bActive = true, 
		const FName Tag = TEXT("EnumAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeBitmaskAction(
		ULearningAgentsActionObject* Object,
		const UEnum* Enum,
		const int32 BitmaskValue,
		const bool bActive = true, 
		const FName Tag = TEXT("BitmaskAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionObjectElement MakeOptionalAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const ELearningAgentsOptionalAction Option, const bool bActive = true, const FName Tag = TEXT("OptionalAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsActionObjectElement MakeOptionalNullAction(ULearningAgentsActionObject* Object, const bool bActive = true, const FName Tag = TEXT("OptionalAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionObjectElement MakeOptionalValidAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("OptionalAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionObjectElement MakeEitherAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const ELearningAgentsEitherAction Either, const bool bActive = true, const FName Tag = TEXT("EitherAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, DisplayName = "Make Either A Action"))
	static UE_API FLearningAgentsActionObjectElement MakeEitherAAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement A, const bool bActive = true, const FName Tag = TEXT("EitherAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, DisplayName = "Make Either B Action"))
	static UE_API FLearningAgentsActionObjectElement MakeEitherBAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement B, const bool bActive = true, const FName Tag = TEXT("EitherAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionObjectElement MakeEncodingAction(ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("EncodingAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeBoolAction(
		ULearningAgentsActionObject* Object,
		const bool bValue,
		const bool bActive = true, 
		const FName Tag = TEXT("BoolAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeFloatAction(
		ULearningAgentsActionObject* Object,
		const float Value,
		const bool bActive = true,
		const FName Tag = TEXT("FloatAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeLocationAction(
		ULearningAgentsActionObject* Object,
		const FVector Location,
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true,
		const FName Tag = TEXT("LocationAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeRotationAction(
		ULearningAgentsActionObject* Object,
		const FRotator Rotation,
		const FRotator RelativeRotation = FRotator::ZeroRotator,
		const bool bActive = true,
		const FName Tag = TEXT("RotationAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeRotationActionFromQuat(
		ULearningAgentsActionObject* Object,
		const FQuat Rotation,
		const FQuat RelativeRotation,
		const bool bActive = true,
		const FName Tag = TEXT("RotationAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeScaleAction(
		ULearningAgentsActionObject* Object,
		const FVector Scale,
		const FVector RelativeScale = FVector(1, 1, 1),
		const bool bActive = true,
		const FName Tag = TEXT("ScaleAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeTransformAction(
		ULearningAgentsActionObject* Object,
		const FTransform Transform,
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true, 
		const FName Tag = TEXT("TransformAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeAngleAction(
		ULearningAgentsActionObject* Object,
		const float Angle,
		const float RelativeAngle = 0.0f,
		const bool bActive = true,
		const FName Tag = TEXT("AngleAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeAngleActionRadians(
		ULearningAgentsActionObject* Object,
		const float Angle,
		const float RelativeAngle = 0.0f,
		const bool bActive = true,
		const FName Tag = TEXT("AngleAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeVelocityAction(
		ULearningAgentsActionObject* Object,
		const FVector Velocity,
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true,
		const FName Tag = TEXT("VelocityAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsActionObjectElement MakeDirectionAction(
		ULearningAgentsActionObject* Object,
		const FVector Direction,
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true,
		const FName Tag = TEXT("DirectionAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

public:

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsActionModifierElement MakeNullActionModifier(ULearningAgentsActionModifier* Modifier, const FName Tag = TEXT("NullAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionModifierElement MakeContinuousActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<bool>& Masked, const TArray<float>& MaskedValues, const FName Tag = TEXT("ContinuousAction"));
	static UE_API FLearningAgentsActionModifierElement MakeContinuousActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const bool> Masked, const TArrayView<const float> MaskedValues, const FName Tag = TEXT("ContinuousAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionModifierElement MakeExclusiveDiscreteActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<int32>& MaskedIndices, const FName Tag = TEXT("DiscreteExclusiveAction"));
	static UE_API FLearningAgentsActionModifierElement MakeExclusiveDiscreteActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const int32> MaskedIndices, const FName Tag = TEXT("DiscreteExclusiveAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionModifierElement MakeNamedExclusiveDiscreteActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<FName>& MaskedNames, const FName Tag = TEXT("NamedDiscreteExclusiveAction"));
	static UE_API FLearningAgentsActionModifierElement MakeNamedExclusiveDiscreteActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const FName> MaskedNames, const FName Tag = TEXT("NamedDiscreteExclusiveAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionModifierElement MakeInclusiveDiscreteActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<int32>& MaskedIndices, const FName Tag = TEXT("DiscreteInclusiveAction"));
	static UE_API FLearningAgentsActionModifierElement MakeInclusiveDiscreteActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const int32> MaskedIndices, const FName Tag = TEXT("DiscreteInclusiveAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionModifierElement MakeNamedInclusiveDiscreteActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<FName>& MaskedNames, const FName Tag = TEXT("NamedDiscreteInclusiveAction"));
	static UE_API FLearningAgentsActionModifierElement MakeNamedInclusiveDiscreteActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const FName> MaskedNames, const FName Tag = TEXT("NamedDiscreteInclusiveAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionModifierElement MakeStructActionModifier(ULearningAgentsActionModifier* Modifier, const TMap<FName, FLearningAgentsActionModifierElement>& Elements, const FName Tag = TEXT("StructAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionModifierElement MakeStructActionModifierFromArrays(ULearningAgentsActionModifier* Modifier, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionModifierElement>& Elements, const FName Tag = TEXT("StructAction"));
	static UE_API FLearningAgentsActionModifierElement MakeStructActionModifierFromArrayViews(ULearningAgentsActionModifier* Modifier, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionModifierElement> Elements, const FName Tag = TEXT("StructAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionModifierElement MakeExclusiveUnionActionModifier(ULearningAgentsActionModifier* Modifier, const TMap<FName, FLearningAgentsActionModifierElement>& Elements, const TArray<FName>& MaskedElements, const FName Tag = TEXT("ExclusiveUnionAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4))
	static UE_API FLearningAgentsActionModifierElement MakeExclusiveUnionActionModifierFromArrays(ULearningAgentsActionModifier* Modifier, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionModifierElement>& Elements, const TArray<FName>& MaskedElements, const FName Tag = TEXT("ExclusiveUnionAction"));
	static UE_API FLearningAgentsActionModifierElement MakeExclusiveUnionActionModifierFromArrayViews(ULearningAgentsActionModifier* Modifier, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionModifierElement> Elements, const TArrayView<const FName> MaskedElements, const FName Tag = TEXT("ExclusiveUnionAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionModifierElement MakeInclusiveUnionActionModifier(ULearningAgentsActionModifier* Modifier, const TMap<FName, FLearningAgentsActionModifierElement>& Elements, const TArray<FName>& MaskedElements, const FName Tag = TEXT("InclusiveUnionAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4))
	static UE_API FLearningAgentsActionModifierElement MakeInclusiveUnionActionModifierFromArrays(ULearningAgentsActionModifier* Modifier, const TArray<FName>& ElementNames, const TArray<FLearningAgentsActionModifierElement>& Elements, const TArray<FName>& MaskedElements, const FName Tag = TEXT("InclusiveUnionAction"));
	static UE_API FLearningAgentsActionModifierElement MakeInclusiveUnionActionModifierFromArrayViews(ULearningAgentsActionModifier* Modifier, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsActionModifierElement> Elements, const TArrayView<const FName> MaskedElements, const FName Tag = TEXT("InclusiveUnionAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionModifierElement MakeStaticArrayActionModifier(ULearningAgentsActionModifier* Modifier, const TArray<FLearningAgentsActionModifierElement>& Elements, const FName Tag = TEXT("StaticArrayAction"));
	static UE_API FLearningAgentsActionModifierElement MakeStaticArrayActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const TArrayView<const FLearningAgentsActionModifierElement> Elements, const FName Tag = TEXT("StaticArrayAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionModifierElement MakePairActionModifier(ULearningAgentsActionModifier* Modifier, const FLearningAgentsActionModifierElement Key, const FLearningAgentsActionModifierElement Value, const FName Tag = TEXT("PairAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionModifierElement MakeEnumActionModifier(ULearningAgentsActionModifier* Modifier, const UEnum* Enum, const TArray<uint8>& EnumMaskedValues, const FName Tag = TEXT("EnumAction"));
	static UE_API FLearningAgentsActionModifierElement MakeEnumActionModifierFromArrayView(ULearningAgentsActionModifier* Modifier, const UEnum* Enum, const TArrayView<const uint8> EnumMaskedValues, const FName Tag = TEXT("EnumAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionModifierElement MakeBitmaskActionModifier(ULearningAgentsActionModifier* Modifier, const UEnum* Enum, const int32 MaskedBitmask, const FName Tag = TEXT("BitmaskAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4))
	static UE_API FLearningAgentsActionModifierElement MakeOptionalActionModifier(ULearningAgentsActionModifier* Modifier, const FLearningAgentsActionModifierElement Element, const bool bAllowOnlyValid, const bool bAllowOnlyNull, const FName Tag = TEXT("OptionalAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 5))
	static UE_API FLearningAgentsActionModifierElement MakeEitherActionModifier(ULearningAgentsActionModifier* Modifier, const FLearningAgentsActionModifierElement A, const FLearningAgentsActionModifierElement B, const bool bAllowOnlyA, const bool bAllowOnlyB, const FName Tag = TEXT("EitherAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionModifierElement MakeEncodingActionModifier(ULearningAgentsActionModifier* Modifier, const FLearningAgentsActionModifierElement Element, const FName Tag = TEXT("EncodingAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsActionModifierElement MakeBoolActionModifier(ULearningAgentsActionModifier* Modifier, const bool bValue, const FName Tag = TEXT("BoolAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsActionModifierElement MakeFloatActionModifier(ULearningAgentsActionModifier* Modifier, const float MaskedValue, const bool bMasked, const FName Tag = TEXT("FloatAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 6))
	static UE_API FLearningAgentsActionModifierElement MakeLocationActionModifier(ULearningAgentsActionModifier* Modifier, const FVector MaskedLocation, const bool bMaskedX, const bool bMaskedY, const bool bMaskedZ, const FTransform RelativeTransform = FTransform(), const FName Tag = TEXT("LocationAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 6))
	static UE_API FLearningAgentsActionModifierElement MakeScaleActionModifier(ULearningAgentsActionModifier* Modifier, const FVector MaskedScale, const bool bMaskedX, const bool bMaskedY, const bool bMaskedZ, const FVector RelativeScale = FVector(1, 1, 1), const FName Tag = TEXT("ScaleAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4))
	static UE_API FLearningAgentsActionModifierElement MakeAngleActionModifier(ULearningAgentsActionModifier* Modifier, const float MaskedAngle, const bool bMask, const float RelativeAngle = 0.0f, const FName Tag = TEXT("AngleAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4))
	static UE_API FLearningAgentsActionModifierElement MakeAngleActionModifierRadians(ULearningAgentsActionModifier* Modifier, const float MaskedAngle, const bool bMask, const float RelativeAngle = 0.0f, const FName Tag = TEXT("AngleAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 6))
	static UE_API FLearningAgentsActionModifierElement MakeVelocityActionModifier(ULearningAgentsActionModifier* Modifier, const FVector MaskedVelocity, const bool bMaskedX, const bool bMaskedY, const bool bMaskedZ, const FTransform RelativeTransform = FTransform(), const FName Tag = TEXT("VelocityAction"));

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 6))
	static UE_API FLearningAgentsActionModifierElement MakeDirectionActionModifier(ULearningAgentsActionModifier* Modifier, const FVector MaskedDirection, const bool bMaskedX, const bool bMaskedY, const bool bMaskedZ, const FTransform RelativeTransform = FTransform(), const FName Tag = TEXT("DirectionAction"));

public:

	/**
	 * Get a null action.
	 *
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 2, ReturnDisplayName = "Success"))
	static UE_API bool GetNullAction(const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const FName Tag = TEXT("NullAction"));

	/**
	 * Get the number of values in a continuous action.
	 *
	 * @param OutNum The output number of values.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetContinuousActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ContinuousAction"));

	/**
	 * Get the values of a continuous action.
	 *
	 * @param OutValues The output values.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetContinuousAction(
		TArray<float>& OutValues,
		const ULearningAgentsActionObject* Object,
		const FLearningAgentsActionObjectElement Element,
		const bool bActive = true,
		const FName Tag = TEXT("ContinuousAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the values of a continuous action. The OutValues ArrayView must be the correct size.
	 *
	 * @param OutValues The output values.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	static UE_API bool GetContinuousActionToArrayView(
		TArrayView<float> OutValues,
		const ULearningAgentsActionObject* Object,
		const FLearningAgentsActionObjectElement Element,
		const bool bActive = true,
		const FName Tag = TEXT("ContinuousAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the index for an exclusive discrete action.
	 *
	 * @param OutIndex The output index.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetExclusiveDiscreteAction(
		int32& OutIndex, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const bool bActive = true, 
		const FName Tag = TEXT("DiscreteExclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the name for a named exclusive discrete action.
	 *
	 * @param OutName The output name.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetNamedExclusiveDiscreteAction(
		FName& OutName,
		const ULearningAgentsActionObject* Object,
		const FLearningAgentsActionObjectElement Element,
		const bool bActive = true, 
		const FName Tag = TEXT("NamedDiscreteExclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the number of indices for an inclusive discrete action.
	 *
	 * @param OutNum The output number of indices.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveDiscreteActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("DiscreteInclusiveAction"));

	/**
	 * Get the indices for an inclusive discrete action.
	 *
	 * @param OutIndices The output indices.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetInclusiveDiscreteAction(
		TArray<int32>& OutIndices, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const bool bActive = true, 
		const FName Tag = TEXT("DiscreteInclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());
	
	/**
	 * Get the indices for an inclusive discrete action. The OutIndices ArrayView must be the correct size.
	 *
	 * @param OutIndices The output indices.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	static UE_API bool GetInclusiveDiscreteActionToArrayView(
		TArrayView<int32> OutIndices, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const bool bActive = true, 
		const FName Tag = TEXT("DiscreteInclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the number of names for a named inclusive discrete action.
	 *
	 * @param OutNum The output number of names.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetNamedInclusiveDiscreteActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("NamedDiscreteInclusiveAction"));

	/**
	 * Get the names for a named inclusive discrete action.
	 *
	 * @param OutNames The output names.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetNamedInclusiveDiscreteAction(
		TArray<FName>& OutNames,
		const ULearningAgentsActionObject* Object,
		const FLearningAgentsActionObjectElement Element,
		const bool bActive = true, 
		const FName Tag = TEXT("NamedDiscreteInclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the names for a named inclusive discrete action. The OutNames ArrayView must be the correct size.
	 *
	 * @param OutNames The output names.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	static UE_API bool GetNamedInclusiveDiscreteActionToArrayView(
		TArrayView<FName> OutNames,
		const ULearningAgentsActionObject* Object,
		const FLearningAgentsActionObjectElement Element,
		const bool bActive = true, 
		const FName Tag = TEXT("NamedDiscreteInclusiveAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the number of sub-actions for a struct action.
	 *
	 * @param OutNum The output number of sub-actions.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStructActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StructAction"));

	/**
	 * Get the sub-actions for a struct action.
	 *
	 * @param OutElements The output sub-actions.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStructAction(TMap<FName, FLearningAgentsActionObjectElement>& OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StructAction"));

	/**
	 * Get the sub-action given its name for a struct action.
	 *
	 * @param OutElement The output sub-action.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param ElementName The Sub-action Name.
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element has the sub-action, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetStructActionElement(FLearningAgentsActionObjectElement& OutElement, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const FName ElementName, const bool bActive = true, const FName Tag = TEXT("StructAction"));

	/**
	 * Get the sub-actions for a struct action.
	 *
	 * @param OutElementNames The output sub-action names.
	 * @param OutElements The output sub-actions.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetStructActionToArrays(TArray<FName>& OutElementNames, TArray<FLearningAgentsActionObjectElement>& OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StructAction"));
	
	/**
	 * Get the sub-actions for a struct action. The OutElementNames and OutElements ArrayViews must be the correct size.
	 *
	 * @param OutElementNames The output sub-action names.
	 * @param OutElements The output sub-actions.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	static UE_API bool GetStructActionToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FLearningAgentsActionObjectElement> OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StructAction"));

	/**
	 * Get the chosen sub-action for an exclusive union action.
	 *
	 * @param OutElement The output sub-action.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetExclusiveUnionAction(FName& OutElementName, FLearningAgentsActionObjectElement& OutElement, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ExclusiveUnionAction"));

	/**
	 * Get the number of sub-actions for an inclusive union action.
	 *
	 * @param OutNum The output number of sub-actions.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveUnionActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionAction"));

	/**
	 * Get the chosen sub-actions for an inclusive union action.
	 *
	 * @param OutElements The output sub-actions.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveUnionAction(TMap<FName, FLearningAgentsActionObjectElement>& OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionAction"));

	/**
	 * Get the chosen sub-actions for an inclusive union action.
	 *
	 * @param OutElementNames The output sub-action names.
	 * @param OutElements The output sub-actions.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveUnionActionToArrays(TArray<FName>& OutElementNames, TArray<FLearningAgentsActionObjectElement>& OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionAction"));
	
	/**
	 * Get the chosen sub-actions for an inclusive union action. The OutElementNames and OutElements ArrayViews must be the correct size.
	 *
	 * @param OutElementNames The output sub-action names.
	 * @param OutElements The output sub-actions.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	static UE_API bool GetInclusiveUnionActionToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FLearningAgentsActionObjectElement> OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionAction"));

	/**
	 * Get the number of entries in a static array action.
	 *
	 * @param OutNum The output number of entries.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStaticArrayActionNum(int32& OutNum, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StaticArrayAction"));

	/**
	 * Get the entries of a static array action.
	 *
	 * @param OutElements The output sub-elements.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStaticArrayAction(TArray<FLearningAgentsActionObjectElement>& OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StaticArrayAction"));
	
	/**
	 * Get the entries of a static array action. The OutElements ArrayView must be the correct size.
	 *
	 * @param OutElements The output sub-elements.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	static UE_API bool GetStaticArrayActionToArrayView(TArrayView<FLearningAgentsActionObjectElement> OutElements, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StaticArrayAction"));

	/**
	 * Get the sub-actions of a pair action.
	 *
	 * @param OutKey The output key sub-element.
	 * @param OutValue The output value sub-element.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetPairAction(FLearningAgentsActionObjectElement& OutKey, FLearningAgentsActionObjectElement& OutValue, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("PairAction"));

	/**
	 * Get the enum value of an enum action.
	 *
	 * @param OutEnumValue The output enum value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param Enum The Enum
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetEnumAction(
		uint8& OutEnumValue, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const UEnum* Enum, 
		const bool bActive = true, 
		const FName Tag = TEXT("EnumAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the bitmask value of a bitmask action.
	 *
	 * @param OutBitmaskValue The output bitmask value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param Enum The Enum
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetBitmaskAction(
		int32& OutBitmaskValue, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const UEnum* Enum, 
		const bool bActive = true, 
		const FName Tag = TEXT("BitmaskAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the sub-action of an option action.
	 *
	 * @param OutOption The output optional specifier.
	 * @param OutElement The output sub-action.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ExpandEnumAsExecs = "OutOption", ReturnDisplayName = "Success"))
	static UE_API bool GetOptionalAction(ELearningAgentsOptionalAction& OutOption, FLearningAgentsActionObjectElement& OutElement, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("OptionalAction"));

	/**
	 * Get the sub-action of an either action.
	 *
	 * @param OutEither The output either specifier.
	 * @param OutElement The output sub-action.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ExpandEnumAsExecs = "OutEither", ReturnDisplayName = "Success"))
	static UE_API bool GetEitherAction(ELearningAgentsEitherAction& OutEither, FLearningAgentsActionObjectElement& OutElement, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("EitherAction"));

	/**
	 * Get the sub-action of an encoding action.
	 *
	 * @param OutElement The output sub-action.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetEncodingAction(FLearningAgentsActionObjectElement& OutElement, const ULearningAgentsActionObject* Object, const FLearningAgentsActionObjectElement Element, const bool bActive = true, const FName Tag = TEXT("EncodingAction"));

	/**
	 * Get the value for a bool action.
	 *
	 * @param bOutValue The output bool value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetBoolAction(
		bool& bOutValue, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const bool bActive = true, 
		const FName Tag = TEXT("BoolAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the value for a float action.
	 *
	 * @param OutValue The output float value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetFloatAction(
		float& OutValue, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const bool bActive = true, 
		const FName Tag = TEXT("FloatAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the value for a location action.
	 *
	 * @param OutLocation The output location value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param RelativeTransform The relative transform to transform the location by.
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetLocationAction(
		FVector& OutLocation, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const FTransform RelativeTransform = FTransform(), 
		const bool bActive = true, 
		const FName Tag = TEXT("LocationAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the value for a rotation action.
	 *
	 * @param OutRotation The output rotation value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param RelativeRotation The relative rotation to transform the rotation by.
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerRotationLocation A location for the visual logger to display the rotation in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetRotationAction(
		FRotator& OutRotation, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const FRotator RelativeRotation = FRotator::ZeroRotator, 
		const bool bActive = true, 
		const FName Tag = TEXT("RotationAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the value for a rotation action as a quaternion.
	 *
	 * @param OutRotation The output rotation value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param RelativeRotation The relative rotation to transform the rotation by.
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerRotationLocation A location for the visual logger to display the rotation in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetRotationActionAsQuat(
		FQuat& OutRotation, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const FQuat RelativeRotation, 
		const bool bActive = true, 
		const FName Tag = TEXT("RotationAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the value for a scale action.
	 *
	 * @param OutScale The output scale value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param RelativeScale The relative scale to transform the scale by.
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetScaleAction(
		FVector& OutScale, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const FVector RelativeScale = FVector(1, 1, 1), 
		const bool bActive = true, 
		const FName Tag = TEXT("ScaleAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the value for a transform action.
	 *
	 * @param OutTransform The output transform value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param RelativeTransform The relative transform.
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 6, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetTransformAction(
		FTransform& OutTransform, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const FTransform RelativeTransform = FTransform(), 
		const bool bActive = true, 
		const FName Tag = TEXT("TransformAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the value for an angle action. Returned angle is in degrees.
	 *
	 * @param OutAngle The output angle value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param RelativeAngle The relative angle to transform the angle by.
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerAngleLocation A location for the visual logger to display the angle in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetAngleAction(
		float& OutAngle, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const float RelativeAngle = 0.0f, 
		const bool bActive = true, 
		const FName Tag = TEXT("AngleAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the value for an angle action. Returned angle is in radians.
	 *
	 * @param OutAngle The output angle value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param RelativeAngle The relative angle to transform the angle by.
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerAngleLocation A location for the visual logger to display the angle in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetAngleActionRadians(
		float& OutAngle, 
		const ULearningAgentsActionObject* Object,
		const FLearningAgentsActionObjectElement Element, 
		const float RelativeAngle = 0.0f,
		const bool bActive = true, 
		const FName Tag = TEXT("AngleAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the value for a velocity action.
	 *
	 * @param OutVelocity The output velocity value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param RelativeTransform The relative transform to transform the velocity by.
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerVelocityLocation A location for the visual logger to display the velocity in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetVelocityAction(
		FVector& OutVelocity, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const FTransform RelativeTransform = FTransform(), 
		const bool bActive = true, 
		const FName Tag = TEXT("VelocityAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());

	/**
	 * Get the value for a direction action.
	 *
	 * @param OutDirection The output direction value.
	 * @param Object The Action Object
	 * @param Element The Action Object Element
	 * @param RelativeTransform The relative transform to transform the direction by.
	 * @param bActive Whether the action should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding action. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this action. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this action.
	 * @param VisualLoggerDirectionLocation A location for the visual logger to display the direction in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerArrowLength The length of the arrow to display to represent the direction.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return true if the provided Element is the correct type, otherwise false.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success", AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API bool GetDirectionAction(
		FVector& OutDirection, 
		const ULearningAgentsActionObject* Object, 
		const FLearningAgentsActionObjectElement Element, 
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true, 
		const FName Tag = TEXT("DirectionAction"),
		const FLearningAgentsVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsVisualLoggerSettings());
};




#undef UE_API