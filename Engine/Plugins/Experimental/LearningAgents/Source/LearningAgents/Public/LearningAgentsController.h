// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsObservations.h"
#include "LearningAgentsActions.h"

#include "LearningArray.h"

#include "LearningAgentsController.generated.h"

#define UE_API LEARNINGAGENTS_API

/**
 * A controller is an object that can be used to construct actions from observations - essentially a hand-made Policy. This can be useful for making 
 * a learning agents system that uses some other existing behavior, e.g. we may want to gather demonstrations from a human or AI behavior tree 
 * controlling our agent(s) for imitation learning purposes.
 */
UCLASS(MinimalAPI, Abstract, HideDropdown, BlueprintType, Blueprintable)
class ULearningAgentsController : public ULearningAgentsManagerListener
{
	GENERATED_BODY()

public:

	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsController();
	UE_API ULearningAgentsController(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsController();

	/**
	 * Constructs a new controller for the given agent interactor.
	 * 
	 * @param InManager			The input Manager
	 * @param InInteractor		The input Interactor component
	 * @param Class				The controller class
	 * @param Name				The controller name
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (DeterminesOutputType = "Class"))
	static UE_API ULearningAgentsController* MakeController(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor, 
		TSubclassOf<ULearningAgentsController> Class,
		const FName Name = TEXT("Controller"));

	/** Initializes this object to be used with the given agent interactor. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void SetupController(UPARAM(ref) ULearningAgentsManager*& InManager, UPARAM(ref) ULearningAgentsInteractor*& InInteractor);

	/**
	 * This callback should be overridden by the Controller and produces an Action Object Element from an Observation Object Element.
	 *
	 * @param OutActionObjectElement		Output Action Object Element.
	 * @param InActionObject				Action object used to construct the output Action Object Element.
	 * @param InObservationObject			Input Observation Object.
	 * @param InObservationObjectElement	Input Observation Object Element.
	 * @param AgentId						Agent id associated with the observation.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void EvaluateAgentController(
		FLearningAgentsActionObjectElement& OutActionObjectElement,
		ULearningAgentsActionObject* InActionObject,
		const ULearningAgentsObservationObject* InObservationObject,
		const FLearningAgentsObservationObjectElement& InObservationObjectElement,
		const int32 AgentId);

	/**
	 * This callback can be overridden by the Controller and produces an array of Action Object Elements, from an array of Observation 
	 * Object Elements. By default this will call EvaluateAgentController for each agent.
	 * 
	 * @param OutActionObjectElements		Output Action Object Elements. This should be the same size as the input AgentIds and 
	 *                                      InObservationObjectElements arrays.
	 * @param InActionObject				Action object used to construct output elements.
	 * @param InObservationObject			Input Observation Object.
	 * @param InObservationObjectElements	Input Observation Object Elements.
	 * @param AgentIds						Agent ids associated with each observation.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void EvaluateAgentControllers(
		TArray<FLearningAgentsActionObjectElement>& OutActionObjectElements, 
		ULearningAgentsActionObject* InActionObject, 
		const ULearningAgentsObservationObject* InObservationObject,
		const TArray<FLearningAgentsObservationObjectElement>& InObservationObjectElements,
		const TArray<int32>& AgentIds);

	/**
	 * Call this function when it is time to evaluate the controller and produce the actions for the agents. This should be called after 
	 * GatherObservations but before PerformActions. This will call this controller's EvaluateAgentController event.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void EvaluateController();

	/**
	 * Calls GatherObservations, followed by EvaluateController, followed by PerformActions
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void RunController();

protected:

	/**
	 * Gets the agent interactor associated with this component.
	 * 
	 * @param AgentClass The class to cast the agent interactor to (in blueprint).
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (DeterminesOutputType = "InteractorClass"))
	UE_API ULearningAgentsInteractor* GetInteractor(const TSubclassOf<ULearningAgentsInteractor> InteractorClass) const;

protected:

	/** The agent interactor this controller is associated with. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsInteractor> Interactor;
};

#undef UE_API
