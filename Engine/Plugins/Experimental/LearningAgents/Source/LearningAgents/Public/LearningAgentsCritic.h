// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningAgentsManagerListener.h"

#include "LearningAgentsNeuralNetwork.h" // Included for ELearningAgentsActivationFunction
#include "LearningArray.h"
#include "UObject/ObjectPtr.h"

#include "LearningAgentsCritic.generated.h"

#define UE_API LEARNINGAGENTS_API

namespace UE::Learning
{
	struct FNeuralNetworkCritic;
}

class ULearningAgentsNeuralNetwork;
class ULearningAgentsInteractor;
class ULearningAgentsPolicy;

/** The configurable settings for a ULearningAgentsCritic. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsCriticSettings
{
	GENERATED_BODY()

public:

	/** Number of hidden layers for critic network. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0"))
	int32 HiddenLayerNum = 1;

	/** Number of neurons in each hidden layer of the critic network */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 HiddenLayerSize = 128;

	/** Activation function to use on hidden layers of the critic network */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	ELearningAgentsActivationFunction ActivationFunction = ELearningAgentsActivationFunction::ELU;
};

/** A critic used for training the policy. Can be used at inference time to estimate the discounted returns.  */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, meta = (BlueprintSpawnableComponent))
class ULearningAgentsCritic : public ULearningAgentsManagerListener
{
	GENERATED_BODY()

public:

	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsCritic();
	UE_API ULearningAgentsCritic(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsCritic();

	/**
	 * Constructs a Critic to be used with the given agent interactor and critic settings.
	 * 
	 * @param InManager						The input Manager
	 * @param InInteractor					The input Interactor object
	 * @param InPolicy						The input Policy object
	 * @param Class							The critic class
	 * @param Name							The critic name
	 * @param CriticNeuralNetworkAsset		Optional Network Asset to use. If not provided, asset is empty, or
	 *										bReinitializeNetwork is set then a new neural network object will be created
	 *										according to the given CriticSettings.
	 * @param bReinitializeCriticNetwork	If to reinitialize the critic network
	 * @param CriticSettings				The critic settings to use on creation of a new critic network
	 * @param Seed							Random seed to use for initializing the critic weights
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (Class = "/Script/LearningAgents.LearningAgentsCritic", DeterminesOutputType = "Class", AutoCreateRefTerm = "CriticSettings"))
	static UE_API ULearningAgentsCritic* MakeCritic(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		UPARAM(ref) ULearningAgentsPolicy*& InPolicy,
		TSubclassOf<ULearningAgentsCritic> Class,
		const FName Name = TEXT("Critic"),
		ULearningAgentsNeuralNetwork* CriticNeuralNetworkAsset = nullptr,
		const bool bReinitializeCriticNetwork = true,
		const FLearningAgentsCriticSettings& CriticSettings = FLearningAgentsCriticSettings(),
		const int32 Seed = 1234);

	/**
	 * Initializes a critic to be used with the given agent interactor and critic settings.
	 * 
	 * @param InManager						The input Manager
	 * @param InInteractor					The input Interactor object
	 * @param InPolicy						The input Policy object
	 * @param CriticNeuralNetworkAsset		Optional Network Asset to use. If not provided, asset is empty, or
	 *										bReinitializeNetwork is set then a new neural network object will be created
	 *										according to the given CriticSettings.
	 * @param bReinitializeCriticNetwork	If to reinitialize the critic network
	 * @param CriticSettings				The critic settings to use on creation of a new critic network
	 * @param Seed							Random seed to use for initializing the critic weights
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "CriticSettings"))
	UE_API void SetupCritic(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		UPARAM(ref) ULearningAgentsPolicy*& InPolicy,
		ULearningAgentsNeuralNetwork* CriticNeuralNetworkAsset = nullptr,
		const bool bReinitializeCriticNetwork = true,
		const FLearningAgentsCriticSettings& CriticSettings = FLearningAgentsCriticSettings(),
		const int32 Seed = 1234);

public:

	//~ Begin ULearningAgentsManagerListener Interface
	UE_API virtual void OnAgentsAdded_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsReset_Implementation(const TArray<int32>& AgentIds) override;
	//~ End ULearningAgentsManagerListener Interface

// ----- Blueprint public interface -----
public:

	/**
	 * Calling this function will run the underlying neural network on the previously buffered observations to populate
	 * the output value buffer. This should be called after the corresponding agent interactor's EncodeObservations.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void EvaluateCritic();

	/**
	 * Gets an estimate of the average discounted return expected by an agent according to the critic. I.E. the total
	 * sum of future rewards, scaled by the discount factor that was used during training. This value can be useful if 
	 * you want to make some decision based on how well the agent thinks they are doing at achieving their task. This 
	 * should be called only after EvaluateCritic.
	 * 
	 * @param AgentId	The AgentId to look-up the estimated discounted return for
	 * @returns			The estimated average discounted return according to the critic
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", Meta=(AgentId = -1))
	UE_API float GetEstimatedDiscountedReturn(const int32 AgentId) const;

	/** Gets the current Network Asset being used */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API ULearningAgentsNeuralNetwork* GetCriticNetworkAsset();

// ----- Non-blueprint public interface -----
public:

	/** Get a reference to this critic's critic function object. */
	UE_API UE::Learning::FNeuralNetworkCritic& GetCriticObject();

// ----- Private Data -----
private:

	/** The agent interactor this critic is associated with. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsInteractor> Interactor;

	/** The policy this critic is associated with. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsPolicy> Policy;

	/** The underlying neural network. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsNeuralNetwork> CriticNetwork;

	/** Internal Critic Function Object */
	TSharedPtr<UE::Learning::FNeuralNetworkCritic> CriticObject;

	/** Buffer to store the computed returns for each agent */
	TLearningArray<1, float, TInlineAllocator<32>> Returns;

	/** Number of times the returns have been computed for all agents */
	TLearningArray<1, uint64, TInlineAllocator<32>> ReturnsIteration;

	/** Temp buffers used to record the set of agents that are valid for evaluation */
	TArray<int32> ValidAgentIds;
	UE::Learning::FIndexSet ValidAgentSet;
};

#undef UE_API
