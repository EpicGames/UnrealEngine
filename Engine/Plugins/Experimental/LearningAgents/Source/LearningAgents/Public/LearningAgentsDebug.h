// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VisualLogger/VisualLogger.h"

#ifndef UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
#define UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG ENABLE_VISUAL_LOG
#endif

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG

#define UE_LEARNING_AGENTS_VLOG_STRING(Owner, Category, Verbosity, Location, Color, Format, ...) \
	UE_VLOG_LOCATION(Owner, Category, Verbosity, Location, 0.0f, Color, Format, ##__VA_ARGS__)

#define UE_LEARNING_AGENTS_VLOG_LOCATION(Owner, Category, Verbosity, Location, Radius, Color, Format, ...) \
	UE_VLOG_LOCATION(Owner, Category, Verbosity, Location, Radius, Color, Format, ##__VA_ARGS__)

#define UE_LEARNING_AGENTS_VLOG_ARROW(Owner, Category, Verbosity, Start, End, Color, Format, ...) \
	UE_VLOG_ARROW(Owner, Category, Verbosity, Start, End, Color, Format, ##__VA_ARGS__)

#define UE_LEARNING_AGENTS_VLOG_SEGMENT(Owner, Category, Verbosity, Start, End, Color, Format, ...) \
	UE_VLOG_SEGMENT(Owner, Category, Verbosity, Start, End, Color, Format, ##__VA_ARGS__)

#define UE_LEARNING_AGENTS_VLOG_CIRCLE(Owner, Category, Verbosity, Center, UpAxis, Radius, Color, Format, ...) \
	UE_VLOG_CIRCLE(Owner, Category, Verbosity, Center, UpAxis, Radius, Color, Format, ##__VA_ARGS__)

#define UE_LEARNING_AGENTS_VLOG_OBOX(Owner, Category, Verbosity, Box, Matrix, Color, Format, ...) \
	UE_VLOG_OBOX(Owner, Category, Verbosity, Box, Matrix, Color, Format, ##__VA_ARGS__)

#define UE_LEARNING_AGENTS_VLOG_MATRIX(Owner, Category, Verbosity, Matrix, Color, Format, ...) \
	{ \
		UE_LEARNING_AGENTS_VLOG_SEGMENT(Owner, Category, Verbosity, Matrix.TransformPosition(FVector::ZeroVector), Matrix.TransformPosition(15.0f * FVector::ForwardVector), FColor::Red, TEXT("")); \
		UE_LEARNING_AGENTS_VLOG_SEGMENT(Owner, Category, Verbosity, Matrix.TransformPosition(FVector::ZeroVector), Matrix.TransformPosition(15.0f * FVector::RightVector), FColor::Green, TEXT("")); \
		UE_LEARNING_AGENTS_VLOG_SEGMENT(Owner, Category, Verbosity, Matrix.TransformPosition(FVector::ZeroVector), Matrix.TransformPosition(15.0f * FVector::UpVector), FColor::Blue, TEXT("")); \
		UE_LEARNING_AGENTS_VLOG_STRING(Owner, Category, Verbosity, Matrix.TransformPosition(FVector::ZeroVector) + FVector(0.0f, 0.0f, 20.0f), Color, Format, ##__VA_ARGS__); \
	}

#define UE_LEARNING_AGENTS_VLOG_TRANSFORM(Owner, Category, Verbosity, Location, Rotation, Color, Format, ...) \
	{ \
		UE_LEARNING_AGENTS_VLOG_SEGMENT(Owner, Category, Verbosity, Location, Location + 15.0f * Rotation.RotateVector(FVector::ForwardVector), FColor::Red, TEXT("")); \
		UE_LEARNING_AGENTS_VLOG_SEGMENT(Owner, Category, Verbosity, Location, Location + 15.0f * Rotation.RotateVector(FVector::RightVector), FColor::Green, TEXT("")); \
		UE_LEARNING_AGENTS_VLOG_SEGMENT(Owner, Category, Verbosity, Location, Location + 15.0f * Rotation.RotateVector(FVector::UpVector), FColor::Blue, TEXT("")); \
		UE_LEARNING_AGENTS_VLOG_OBOX(Owner, Category, Verbosity, FBox(5.0f * FVector(-1, -1, -1), 5.0f * FVector(1, 1, 1)), FTransform(Rotation, Location, FVector::OneVector).ToMatrixNoScale(), Color, TEXT("")); \
		UE_LEARNING_AGENTS_VLOG_STRING(Owner, Category, Verbosity, Location + FVector(0.0f, 0.0f, 20.0f), Color, Format, ##__VA_ARGS__); \
	}

#define UE_LEARNING_AGENTS_VLOG_ANGLE_RADIANS(Owner, Category, Verbosity, Angle, RelativeAngle, Location, Radius, Color, Format, ...) \
	{ \
		UE_LEARNING_AGENTS_VLOG_CIRCLE(Owner, Category, Verbosity, Location, FVector::UpVector, Radius, Color, TEXT("")); \
		UE_LEARNING_AGENTS_VLOG_SEGMENT(Owner, Category, Verbosity, Location, Location + Radius * FVector(FMath::Sin(RelativeAngle), FMath::Cos(RelativeAngle), 0.0f), Color, TEXT("")); \
		UE_LEARNING_AGENTS_VLOG_SEGMENT(Owner, Category, Verbosity, Location, Location + Radius * FVector(FMath::Sin(Angle), FMath::Cos(Angle), 0.0f), Color, TEXT("")); \
		UE_LEARNING_AGENTS_VLOG_LOCATION(Owner, Category, Verbosity, Location + Radius * FVector(FMath::Sin(Angle), FMath::Cos(Angle), 0.0f), static_cast<uint16>(Radius / 20.0f), Color, Format, ##__VA_ARGS__); \
	}

#define UE_LEARNING_AGENTS_VLOG_ANGLE_DEGREES(Owner, Category, Verbosity, Angle, RelativeAngle, Location, Radius, Color, Format, ...) \
	UE_LEARNING_AGENTS_VLOG_ANGLE_RADIANS(Owner, Category, Verbosity, FMath::DegreesToRadians(Angle), FMath::DegreesToRadians(RelativeAngle), Location, Radius, Color, Format)

#endif
