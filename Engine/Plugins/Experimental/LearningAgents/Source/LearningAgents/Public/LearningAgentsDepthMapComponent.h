// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "CollisionQueryParams.h"
#include "LearningAgentsDepthMapComponent.generated.h"

#define UE_API LEARNINGAGENTS_API

USTRUCT(BlueprintType, Category = "LearningAgents")
struct FDepthMapConfig
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	float HorizontalFOV = 90.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	float AspectRatio = 1.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	int32 Width = 42; 

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	int32 Height = 42; 

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	float MaxDistance = 1500.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	FVector FrustumOffset = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	bool bInvertEncoding = false;
};

UCLASS(BlueprintType, Blueprintable, ClassGroup = (LearningAgents), meta = (BlueprintSpawnableComponent), config=Engine)
class ULearningAgentsDepthMapComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UE_API ULearningAgentsDepthMapComponent();

	UPROPERTY(Config, BlueprintReadWrite, EditDefaultsOnly, Category = "LearningAgents")
	bool bDrawDepthFrustum = false;

	UPROPERTY(Config, BlueprintReadWrite, EditDefaultsOnly, Category = "LearningAgents")
	FDepthMapConfig DepthMapConfig;

	// If specified as a positive integer, each tick will only update a batch of the depth rays in order to improve performance.
	UPROPERTY(Config, BlueprintReadWrite, EditDefaultsOnly, Category = "LearningAgents")
	int32 DepthRaysBatchSize = 0;

	UPROPERTY(Config, BlueprintReadWrite, EditDefaultsOnly, Category = "LearningAgents")
	TEnumAsByte<ECollisionChannel> CollisionChannel = ECollisionChannel::ECC_WorldStatic;

	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API const TArray<float>& GetDepthMapFlatArray() const;

	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

protected:
	virtual void BeginPlay() override;

private:
	// Set as class variable for perf
	FCollisionObjectQueryParams CollisionObjectQueryParams;

	// Set as class variable for perf
	FCollisionQueryParams CollisionQueryParams;

	int32 BatchCastingCounter = 0;
	
	TArray<float> DepthMapFlatArray;

	TArray<FVector> DepthMapDirections;	

	void GenerateDepthMapDirections();

	float CastDepthRay(const FVector& RayStartWorld, const FVector& RayEndWorld) const;
	
	void BatchUpdateDepthMap();

	float EncodeDepthValue(const float DepthValue) const;

	void DebugDrawCornerRays();
};

#undef UE_API
