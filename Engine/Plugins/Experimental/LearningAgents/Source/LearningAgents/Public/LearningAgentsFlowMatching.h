// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsNeuralNetwork.h"
#include "LearningArray.h"
#include "LearningAction.h"
#include "UObject/ObjectPtr.h"

#include "LearningAgentsFlowMatching.generated.h"

#define UE_API LEARNINGAGENTS_API

namespace UE::Learning
{
	struct FNeuralNetworkFunction;
}

namespace UE::NNE::RuntimeBasic
{
	class FModelBuilderElement;
	class FModelBuilder;
}

class ULearningAgentsNeuralNetwork;
class ULearningAgentsInteractor;
class FJsonObject;

/** The configurable settings for a ULearningAgentsFlowMatching. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsFlowMatchingSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 ActionChunkSize = 1;

	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0"))
	int32 HiddenLayerNum = 1;

	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 HiddenLayerSize = 128;

	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	ELearningAgentsActivationFunction ActivationFunction = ELearningAgentsActivationFunction::ELU;

	/** Number of ODE integration steps. More steps is usually better quality but slower inference. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1", ClampMax = "100", UIMax = "100"))
	int32 ODEStepsNum = 8;
};

UCLASS(MinimalAPI, BlueprintType, Blueprintable, meta = (BlueprintSpawnableComponent))
class ULearningAgentsFlowMatching : public ULearningAgentsManagerListener
{
	GENERATED_BODY()

public:

	UE_API ULearningAgentsFlowMatching();
	UE_API ULearningAgentsFlowMatching(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsFlowMatching();

	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (Class = "/Script/LearningAgents.LearningAgentsFlowMatching", DeterminesOutputType = "Class", AutoCreateRefTerm = "FlowMatchingSettings"))
	static UE_API ULearningAgentsFlowMatching* MakeFlowMatchingModel(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		TSubclassOf<ULearningAgentsFlowMatching> Class,
		const FName Name = TEXT("FlowMatchingModel"),
		ULearningAgentsNeuralNetwork* EncoderNeuralNetworkAsset = nullptr,
		ULearningAgentsNeuralNetwork* DenoiserNeuralNetworkAsset = nullptr,
		ULearningAgentsNeuralNetwork* DecoderNeuralNetworkAsset = nullptr,
		const bool bReinitializeEncoderNetwork = true,
		const bool bReinitializeDenoiserNetwork = true,
		const bool bReinitializeDecoderNetwork = true,
		const bool bSkipCompatibilityChecks = false,
		const FLearningAgentsFlowMatchingSettings& FlowMatchingSettings = FLearningAgentsFlowMatchingSettings(),
		const int32 Seed = 1234);

	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "InFlowMatchingSettings"))
	UE_API void SetupFlowMatchingModel(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		ULearningAgentsNeuralNetwork* EncoderNeuralNetworkAsset = nullptr,
		ULearningAgentsNeuralNetwork* DenoiserNeuralNetworkAsset = nullptr,
		ULearningAgentsNeuralNetwork* DecoderNeuralNetworkAsset = nullptr,
		const bool bReinitializeEncoderNetwork = true,
		const bool bReinitializeDenoiserNetwork = true,
		const bool bReinitializeDecoderNetwork = true,
		const bool bSkipCompatibilityChecks = false,
		const FLearningAgentsFlowMatchingSettings& InFlowMatchingSettings = FLearningAgentsFlowMatchingSettings(),
		const int32 Seed = 1234);

	UE_API TSharedRef<FJsonObject> AsJsonConfig() const;

public:

	UE_API virtual void OnAgentsAdded_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsReset_Implementation(const TArray<int32>& AgentIds) override;

public:

	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void RunInference();

	UE_API void RunInference(const int32 ODEStepsNum);

	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API ULearningAgentsNeuralNetwork* GetEncoderNetworkAsset();

	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API ULearningAgentsNeuralNetwork* GetDenoiserNetworkAsset();

	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API ULearningAgentsNeuralNetwork* GetDecoderNetworkAsset();

public:

	UE_API UE::Learning::FNeuralNetworkFunction& GetEncoderObject();
	UE_API UE::Learning::FNeuralNetworkFunction& GetDenoiserObject();
	UE_API UE::Learning::FNeuralNetworkFunction& GetDecoderObject();

private:

	void PerformODEStep(TLearningArray<2, float>& CurrentState, const float Time, const float DeltaTime);
	void EvaluateVelocity(TLearningArray<2, float>& OutVelocity, const TLearningArray<2, float>& CurrentState, const float Time);
	void InitializeNoiseState(TLearningArray<2, float>& OutInitialState);

	void GenerateDenormalizerNetworkFileDataFromSchema(TArray<uint8>& OutFileData, uint32& OutInputSize, uint32& OutOutputSize, const UE::Learning::Action::FSchema& Schema, const UE::Learning::Action::FSchemaElement SchemaElement, const UE::Learning::Action::FNetworkSettings& NetworkSettings, const uint32 Seed);
	void MakeDenormalizerNetworkModelBuilderElementFromSchema(UE::NNE::RuntimeBasic::FModelBuilderElement& OutElement, UE::NNE::RuntimeBasic::FModelBuilder& Builder, const UE::Learning::Action::FSchema& Schema, const UE::Learning::Action::FSchemaElement SchemaElement, const UE::Learning::Action::FNetworkSettings& NetworkSettings);
	
private:

	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsInteractor> Interactor;

	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsNeuralNetwork> EncoderNetwork;

	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsNeuralNetwork> DenoiserNetwork;

	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsNeuralNetwork> DecoderNetwork;

	uint32 GlobalSeed = 0;

	TLearningArray<1, uint32> Seeds;

	TLearningArray<2, float> ObservationVectorsEncoded;

	TLearningArray<2, float> ODEState;

	TLearningArray<2, float> VelocityBuffer;

	TSharedPtr<UE::Learning::FNeuralNetworkFunction> EncoderObject;

	TSharedPtr<UE::Learning::FNeuralNetworkFunction> DenoiserObject;

	TSharedPtr<UE::Learning::FNeuralNetworkFunction> DecoderObject;

	TLearningArray<2, float> DenoiserInputBuffer;

	TArray<int32> ValidAgentIds;
	UE::Learning::FIndexSet ValidAgentSet;

	FLearningAgentsFlowMatchingSettings FlowMatchingSettings;

	int32 ActionDimension = 0;
};

#undef UE_API
