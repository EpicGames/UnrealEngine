// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsObservations.h"
#include "LearningAgentsActions.h"

#include "LearningArray.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include "LearningAgentsInteractor.generated.h"

#define UE_API LEARNINGAGENTS_API

class ULearningAgentsNeuralNetwork;

/**
 * ULearningAgentsInteractor defines how agents interact with the environment through their observations and actions.
 *
 * To use this class, you need to implement `SpecifyAgentObservation` and `SpecifyAgentAction`, which will define 
 * the structure of inputs and outputs to your policy. You also need to implement `GatherAgentObservation` and 
 * `PerformAgentAction` which will dictate how those observations are gathered, and actions actuated in your
 * environment.
 */
UCLASS(MinimalAPI, Abstract, HideDropdown, BlueprintType, Blueprintable)
class ULearningAgentsInteractor : public ULearningAgentsManagerListener
{
	GENERATED_BODY()

public:
	
	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsInteractor();
	UE_API ULearningAgentsInteractor(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsInteractor();

	/**
	 * Constructs an Interactor.
	 *
	 * @param InManager						The input Manager
	 * @param Class							The interactor class
	 * @param Name							The interactor name
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (DeterminesOutputType = "Class"))
	static UE_API ULearningAgentsInteractor* MakeInteractor(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		TSubclassOf<ULearningAgentsInteractor> Class,
		const FName Name = TEXT("Interactor"));

	/**
	 * Initializes an Interactor.
	 *
	 * @param InManager						The input Manager
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void SetupInteractor(UPARAM(ref) ULearningAgentsManager*& InManager);

public:

	//~ Begin ULearningAgentsManagerListener Interface
	UE_API virtual void OnAgentsAdded_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsReset_Implementation(const TArray<int32>& AgentIds) override;
	//~ End ULearningAgentsManagerListener Interface

// ----- Observations -----
public:

	/**
	 * This callback should be overridden by the Interactor and specifies the structure of the observations using the Observation Schema.
	 * 
	 * @param OutObservationSchemaElement		Output Schema Element
	 * @param InObservationSchema				Observation Schema
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void SpecifyAgentObservation(FLearningAgentsObservationSchemaElement& OutObservationSchemaElement, ULearningAgentsObservationSchema* InObservationSchema);

	/**
	 * This callback should be overridden by the Interactor and gathers the observations for a single agent. The structure of the Observation Elements 
	 * output by this function should match that defined by the Schema.
	 *
	 * @param OutObservationObjectElement		Output Observation Element.
	 * @param InObservationObject				Observation Object.
	 * @param AgentId							The Agent Id to gather observations for.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void GatherAgentObservation(FLearningAgentsObservationObjectElement& OutObservationObjectElement, ULearningAgentsObservationObject* InObservationObject, const int32 AgentId);


	/**
	 * This callback can be overridden by the Interactor and gathers all the observations for the given agents. The structure of the Observation 
	 * Elements output by this function should match that defined by the Schema. The default implementation calls GatherAgentObservation on each agent.
	 *
	 * @param OutObservationObjectElements		Output Observation Elements. This should be the same size as AgentIds.
	 * @param InObservationObject				Observation Object.
	 * @param AgentIds							Set of Agent Ids to gather observations for.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void GatherAgentObservations(TArray<FLearningAgentsObservationObjectElement>& OutObservationObjectElements, ULearningAgentsObservationObject* InObservationObject, const TArray<int32>& AgentIds);


// ----- Actions -----
public:

	/**
	 * This callback should be overridden by the Interactor and specifies the structure of the actions using the Action Schema.
	 *
	 * @param OutActionSchemaElement			Output Schema Element
	 * @param InActionSchema					Action Schema
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void SpecifyAgentAction(FLearningAgentsActionSchemaElement& OutActionSchemaElement, ULearningAgentsActionSchema* InActionSchema);

	/**
	 * This callback should be overridden by the Interactor and performs the action for the given agent in the world. The 
	 * structure of the Action Elements given as input to this function will match that defined by the Schema.
	 *
	 * @param InActionObject					Action Object.
	 * @param InActionObjectElement				Input Actions Element.
	 * @param AgentId							Agent Id to perform actions for.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void PerformAgentAction(const ULearningAgentsActionObject* InActionObject, const FLearningAgentsActionObjectElement& InActionObjectElement, const int32 AgentId);

	/**
	 * This callback can be overridden by the Interactor and performs all the actions for the given agents in the world. 
	 * The structure of the Action Elements given as input to this function will match that defined by the Schema. The default implementation calls 
	 * PerformAgentAction on each agent.
	 *
	 * @param InActionObject					Action Object.
	 * @param InActionObjectElements			Input Actions Element. This will be the same size as AgentIds.
	 * @param AgentIds							Set of Agent Ids to perform actions for.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void PerformAgentActions(const ULearningAgentsActionObject* InActionObject, const TArray<FLearningAgentsActionObjectElement>& InActionObjectElements, const TArray<int32>& AgentIds);

	/**
	 * This callback can be optionally overridden by the Interactor to create an action modifier for a single agent. The structure of the Action 
	 * Modifier Elements output by this function should match that of the actions defined by the Schema.
	 *
	 * @param OutActionModifierElement			Output Action Modifier Element.
	 * @param InActionModifier					Action Modifier Object.
	 * @param InObservationObject				Input Observation Object.
	 * @param InObservationObjectElement		Input Observation Object Element.
	 * @param AgentId							The Agent Id to make action modifiers for.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void MakeAgentActionModifier(
		FLearningAgentsActionModifierElement& OutActionModifierElement, 
		ULearningAgentsActionModifier* InActionModifier, 
		const ULearningAgentsObservationObject* InObservationObject,
		const FLearningAgentsObservationObjectElement& InObservationObjectElement,
		const int32 AgentId);

	/**
	 * This callback can be optionally overridden by the Interactor to create all the action modifier for the given agents. The structure of the 
	 * Action Modifier Elements output by this function should match that of the actions defined by the Schema. The default implementation calls 
	 * MakeAgentActionModifier on each agent.
	 *
	 * @param OutActionModifierElements			Output Action Modifier Elements. This should be the same size as AgentIds.
	 * @param InActionModifier					Action Modifier Object.
	 * @param InObservationObject				Input Observation Object.
	 * @param InObservationObjectElements		Input Observation Object Elements.
	 * @param AgentIds							Set of Agent Ids to make action modifiers for.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void MakeAgentActionModifiers(
		TArray<FLearningAgentsActionModifierElement>& OutActionModifierElements, 
		ULearningAgentsActionModifier* InActionModifier, 
		const ULearningAgentsObservationObject* InObservationObject,
		const TArray<FLearningAgentsObservationObjectElement>& InObservationObjectElements,
		const TArray<int32>& AgentIds);


// ----- Blueprint public interface -----
public:

	/** Gathers all the observations for all agents. This will call GatherAgentObservations. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void GatherObservations();

	/** Makes all the action modifiers for all agents. This will call MakeAgentActionModifiers. Should be called even when Action Modifiers are not used. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void MakeActionModifiers();

	/** Performs all the actions for all agents. This will call PerformAgentActions. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void PerformActions();

	/**
	 * Get the current buffered observation vector for the given agent.
	 * 
	 * @param OutObservationVector				Output Observation Vector
	 * @param OutObservationCompatibilityHash	Output Compatibility Hash used to identify which schema this vector is compatible with.
	 * @param AgentId							Agent Id to look up the observation vector for.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", Meta = (AgentId = "-1"))
	UE_API void GetObservationVector(TArray<float>& OutObservationVector, int32& OutObservationCompatibilityHash, const int32 AgentId);

	/**
	 * Get the current buffered action modifier vector for the given agent.
	 *
	 * @param OutActionModifierVector			Output Action Modifier Vector
	 * @param OutActionCompatibilityHash		Output Compatibility Hash used to identify which schema this vector is compatible with.
	 * @param AgentId							Agent Id to look up the observation vector for.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", Meta = (AgentId = "-1"))
	UE_API void GetActionModifierVector(TArray<float>& OutActionModifierVector, int32& OutActionCompatibilityHash, const int32 AgentId);

	/**
	 * Get the current buffered action vector for the given agent.
	 *
	 * @param OutActionVector					Output Action Vector
	 * @param OutActionCompatibilityHash		Output Compatibility Hash used to identify which schema this vector is compatible with.
	 * @param AgentId							Agent Id to look up the action vector for.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", Meta = (AgentId = "-1"))
	UE_API void GetActionVector(TArray<float>& OutActionVector, int32& OutActionCompatibilityHash, const int32 AgentId);

	/**
	 * Sets the current buffered observation vector for the given agent.
	 *
	 * @param ObservationVector					Observation Vector
	 * @param InObservationCompatibilityHash	Compatibility Hash used to identify which schema this vector is compatible with.
	 * @param AgentId							Agent Id to set the observation vector for.
	 * @param bIncrementIteration				If to increment the iteration number used to keep track of associated actions and observations.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", Meta = (AgentId = "-1"))
	UE_API void SetObservationVector(const TArray<float>& ObservationVector, const int32 InObservationCompatibilityHash, const int32 AgentId, bool bIncrementIteration = true);

	/**
	 * Sets the current buffered action modifier vector for the given agent.
	 *
	 * @param ActionModifierVector				Action Modifier Vector
	 * @param InActionCompatibilityHash			Compatibility Hash used to identify which schema this vector is compatible with.
	 * @param AgentId							Agent Id to set the observation vector for.
	 * @param bIncrementIteration				If to increment the iteration number used to keep track of associated actions and observations.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", Meta = (AgentId = "-1"))
	UE_API void SetActionModifierVector(const TArray<float>& ActionModifierVector, const int32 InActionCompatibilityHash, const int32 AgentId, bool bIncrementIteration = true);

	/**
	 * Sets the current buffered action vector for the given agent.
	 *
	 * @param ActionVector						Action Vector
	 * @param InActionCompatibilityHash			Compatibility Hash used to identify which schema this vector is compatible with.
	 * @param AgentId							Agent Id to set the observation vector for.
	 * @param bIncrementIteration				If to increment the iteration number used to keep track of associated actions and observations.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", Meta = (AgentId = "-1"))
	UE_API void SetActionVector(const TArray<float>& ActionVector, const int32 InActionCompatibilityHash, const int32 AgentId, bool bIncrementIteration = true);

	/**
	 * Returns true if GatherObservations or SetObservationVector has been called and the observation vector already set for the given agent.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API bool HasObservationVector(const int32 AgentId) const;

	/**
	 * Returns true if MakeActionModifiers or SetActionModifierVector has been called and the action modifier vector already set for the given agent.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API bool HasActionModifierVector(const int32 AgentId) const;

	/**
	 * Returns true if DecodeAndSampleActions on the policy or SetActionVector has been called and the action vector already set for the given agent.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API bool HasActionVector(const int32 AgentId) const;

	/** Gets the size of the observation vector used by this interactor. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetObservationVectorSize() const;

	/** Gets the size of the encoded observation vector used by this interactor. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetObservationEncodedVectorSize() const;

	/** Gets the size of the action vector used by this interactor. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetActionVectorSize() const;

	/** Gets the size of the action distribution vector used by this interactor. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetActionDistributionVectorSize() const;

	/** Gets the size of the action modifier vector used by this interactor. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetActionModifierVectorSize() const;

	/** Gets the size of the encoded action vector used by this interactor. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetActionEncodedVectorSize() const;

// ----- Non-blueprint public interface -----
public:

	/** Encode Observations for a specific set of agents */
	UE_API void GatherObservations(const UE::Learning::FIndexSet AgentSet, bool bIncrementIteration = true);

	/** Make Action Modifiers for a specific set of agents */
	UE_API void MakeActionModifiers(const UE::Learning::FIndexSet AgentSet, bool bIncrementIteration = true);

	/** Perform Actions for a specific set of agents */
	UE_API void PerformActions(const UE::Learning::FIndexSet AgentSet);

	/** Gets the observation schema object */
	UE_API const ULearningAgentsObservationSchema* GetObservationSchema() const;

	/** Gets the observation schema element */
	UE_API const FLearningAgentsObservationSchemaElement GetObservationSchemaElement() const;

	/** Gets the action schema object */
	UE_API const ULearningAgentsActionSchema* GetActionSchema() const;

	/** Gets the action schema element */
	UE_API const FLearningAgentsActionSchemaElement GetActionSchemaElement() const;

	/** Gets the observation vectors as a const array view. */
	UE_API TLearningArrayView<2, const float> GetObservationVectorsArrayView() const;

	/** Gets the observation iteration value for the given agent id. */
	UE_API uint64 GetObservationIteration(const int32 AgentId) const;

	/** Gets the action modifier vectors as a const array view. */
	UE_API TLearningArrayView<2, const float> GetActionModifierVectorsArrayView() const;

	/** Gets the action modifier iteration value for the given agent id. */
	UE_API uint64 GetActionModifierIteration(const int32 AgentId) const;

	/** Gets the action vectors as a const array view. */
	UE_API TLearningArrayView<2, const float> GetActionVectorsArrayView() const;

	/** Gets the action iteration value for the given agent id. */
	UE_API uint64 GetActionIteration(const int32 AgentId) const;

	/** Gets the observation object. */
	UE_API const ULearningAgentsObservationObject* GetObservationObject() const;

	/** Gets the observation object elements. */
	UE_API const TArray<FLearningAgentsObservationObjectElement>& GetObservationObjectElements() const;

	/** Gets the action modifier. */
	UE_API const ULearningAgentsActionModifier* GetActionModifier() const;

	/** Gets the action modifier elements. */
	UE_API const TArray<FLearningAgentsActionModifierElement>& GetActionModifierElements() const;

	/** Gets the action object. */
	UE_API ULearningAgentsActionObject* GetActionObject();

	/** Gets the action object elements. */
	UE_API TArray<FLearningAgentsActionObjectElement>& GetActionObjectElements();

	/** Gets the action vectors as a mutable array view. */
	UE_API TLearningArrayView<2, float> GetActionVectorsArrayView();

	/** Gets the action vector iterations as a mutable array view. */
	UE_API TLearningArrayView<1, uint64> GetActionVectorIterationArrayView();

private:

	/** Observation Schema used by this interactor */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsObservationSchema> ObservationSchema;

	/** Observation Schema Element used by this interactor */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	FLearningAgentsObservationSchemaElement ObservationSchemaElement;

	/** Action Schema used by this interactor */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsActionSchema> ActionSchema;

	/** Action Schema Element used by this interactor */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	FLearningAgentsActionSchemaElement ActionSchemaElement;

	/** Observation Object used by this interactor */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsObservationObject> ObservationObject;

	/** Observation Object Elements used by this interactor */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TArray<FLearningAgentsObservationObjectElement> ObservationObjectElements;

	/** Action Modifier used by this interactor */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsActionModifier> ActionModifier;

	/** Action Modifier Elements used by this interactor */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TArray<FLearningAgentsActionModifierElement> ActionModifierElements;

	/** Action Object used by this interactor */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsActionObject> ActionObject;

	/** Action Object Elements used by this interactor */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TArray<FLearningAgentsActionObjectElement> ActionObjectElements;

// ----- Private Data -----
private:

	/** Buffer of Observation Vectors for each agent */
	TLearningArray<2, float> ObservationVectors;

	/** Buffer of Action Modifier Vectors for each agent */
	TLearningArray<2, float> ActionModifierVectors;

	/** Buffer of Action Vectors for each agent */
	TLearningArray<2, float> ActionVectors;

	/** Compatibility Hash for Observation Schema */
	int32 ObservationCompatibilityHash = 0;

	/** Compatibility Hash for Action Schema */
	int32 ActionCompatibilityHash = 0;

	/** Number of times observation vector has been set for all agents */
	TLearningArray<1, uint64, TInlineAllocator<32>> ObservationVectorIteration;

	/** Number of times action modifier vector has been set for all agents */
	TLearningArray<1, uint64, TInlineAllocator<32>> ActionModifierVectorIteration;

	/** Number of times action vector has been set for all agents */
	TLearningArray<1, uint64, TInlineAllocator<32>> ActionVectorIteration;

	/** Temp buffers used to record the set of agents that are valid for encoding/decoding */
	TArray<int32> ValidAgentIds;
	UE::Learning::FIndexSet ValidAgentSet;

};

#undef UE_API
