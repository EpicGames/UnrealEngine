// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Templates/SharedPointer.h"

#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "Components/ActorComponent.h"

#include "LearningAgentsManager.generated.h"

#define UE_API LEARNINGAGENTS_API

class ULearningAgentsManagerListener;

/**
 * The agent manager is responsible for tracking which game objects are agents. It's the central class around which
 * most of Learning Agents is built.
 *
 * If you have multiple different types of objects you want controlled by Learning Agents, you should consider creating
 * one agent manager per object type, rather than trying to share an agent manager.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, meta = (BlueprintSpawnableComponent))
class ULearningAgentsManager : public UActorComponent
{
	GENERATED_BODY()

public:

	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsManager();
	UE_API ULearningAgentsManager(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsManager();

	UE_API virtual void OnRegister() override;

	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

// ----- Agent Management -----
public:

	/** Returns the maximum number of agents that this manager is configured to handle. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetMaxAgentNum() const;
	
	/** Sets the maximum number of agents that this manager is configured to handle. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	UE_API void SetMaxAgentNum(const int32 InMaxAgentNum);

public:

	/**
	 * Adds the given object as an agent to the manager.
	 * @param Agent The object to be added.
	 * @return The agent's newly assigned id.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API int32 AddAgent(UObject* Agent);

	/**
	 * Adds the given objects as an agents to the manager.
	 * @param OutAgentIds The output newly assigned agent ids
	 * @param InAgents The objects to be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void AddAgents(TArray<int32>& OutAgentIds, const TArray<UObject*>& InAgents);

	/**
	 * Schedules a task to add the given object as an agent to the manager, preventing synchronous mutations of agent arrays on the game thread.
	 * @param Agent The object to be added.
	 * @return The agent's newly assigned id.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void RequestAddAgent(UObject* Agent);

public:

	/**
	 * Removes the agent with the given id from the manager.
	 * @param AgentId The id of the agent to remove.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API void RemoveAgent(const int32 AgentId);

	/**
	 * Removes the agents with the given ids from the manager.
	 * @param AgentIds The ids of the agents to remove.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void RemoveAgents(const TArray<int32>& AgentIds);

	/** Removes all agents from the manager. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void RemoveAllAgents();

	/**
	 * Schedules a task to remove the agent with the given ID from the manager, preventing synchronous mutations of agent arrays on the game thread.
	 * @param AgentId The id of the agent to remove.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API void RequestRemoveAgent(const int32 AgentId);

public:

	/**
	 * Resets the agent with the given id on the manager. Used to tell components to reset any state associated with this agent.
	 * @param AgentId The id of the agent to reset.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API void ResetAgent(const int32 AgentId);

	/**
	 * Resets the agents with the given ids on the manager. Used to tell components to reset any state associated with this agent.
	 * @param AgentIds The ids of the agents to reset.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void ResetAgents(const TArray<int32>& AgentIds);

	/** Resets all the agents on the manager. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void ResetAllAgents();

public:

	/**
	 * Gets the agent with the given id. Calling this from blueprint with the appropriate AgentClass will automatically
	 * cast the object to the given type. If not in a blueprint, you should use one of the other GetAgent overloads.
	 * 
	 * @param AgentId The id of the agent to get.
	 * @param AgentClass The class to cast the agent object to (in blueprint).
	 * @return The agent object.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1", DeterminesOutputType = "AgentClass"))
	UE_API UObject* GetAgent(const int32 AgentId, const TSubclassOf<UObject> AgentClass) const;

	/**
	 * Gets the agents associated with a set of ids. Calling this from blueprint with the appropriate AgentClass will 
	 * automatically cast the object to the given type.
	 * 
	 * @param AgentIds The ids of the agents to get.
	 * @param AgentClass The class to cast the agent objects to (in blueprint).
	 * @param OutAgents The output array of agent objects.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (DeterminesOutputType = "AgentClass", DynamicOutputParam = "OutAgents"))
	UE_API void GetAgents(TArray<UObject*>& OutAgents, const TArray<int32>& AgentIds, const TSubclassOf<UObject> AgentClass) const;

	/**
	 * Gets all added agents. Calling this from blueprint with the appropriate AgentClass will automatically
	 * cast the object to the given type.
	 * 
	 * @param AgentClass The class to cast the agent objects to (in blueprint).
	 * @param OutAgents The output array of agent objects.
	 * @param OutAgentIds The output array of agent ids.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (DeterminesOutputType = "AgentClass", DynamicOutputParam = "OutAgents"))
	UE_API void GetAllAgents(TArray<UObject*>& OutAgents, TArray<int32>& OutAgentIds, const TSubclassOf<UObject> AgentClass) const;

	/**
	 * Gets the agent id associated with a given agent.
	 * 
	 * @param Agent The agent object.
	 * @return The agent id.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetAgentId(UObject* Agent) const;

	/**
	 * Gets the agent ids associated with a set of agents.
	 * 
	 * @param OutAgentIds The ids of the agents.
	 * @param InAgents The agent objects.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents")
	UE_API void GetAgentIds(TArray<int32>& OutAgentIds, const TArray<UObject*>& InAgents) const;

	/**
	 * Gets the number of agents added
	 * 
	 * @return The number of agents added.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetAgentNum() const;

public:

	/** Returns true if the given object is an agent used by the manager; Otherwise, false. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API bool HasAgentObject(UObject* Agent) const;

	/** Returns true if the given id is an agent used by the manager; Otherwise, false. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API bool HasAgent(const int32 AgentId) const;

public:

	/** Adds a listener to be tracked by this manager. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void AddListener(ULearningAgentsManagerListener* Listener);

	/** Removes a listener from being tracked by this manager. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void RemoveListener(ULearningAgentsManagerListener* Listener);


// ----- Non-blueprint public interface -----
public:

	/** Gets the agent corresponding to the given id. */
	UE_API const UObject* GetAgent(const int32 AgentId) const;

	/** Gets the agent corresponding to the given id. */
	UE_API UObject* GetAgent(const int32 AgentId);

	/** Gets the set of agent ids currently added */
	UE_API const TArray<int32>& GetAllAgentIds() const;

	/** Gets the set of agent ids currently added as an FIndexSet */
	UE_API UE::Learning::FIndexSet GetAllAgentSet() const;

	/** Get a const array view of this manager's agent objects. */
	UE_API TConstArrayView<TObjectPtr<UObject>> GetAgents() const;

protected:

	/** Maximum number of agents. Used to preallocate internal buffers. Setting this higher will allow more agents but use up more memory. */
	UPROPERTY(EditDefaultsOnly, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 MaxAgentNum = 1;

private:

	/** The list of current agents. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TArray<TObjectPtr<UObject>> Agents;

	/** The list of current listeners. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TArray<TObjectPtr<ULearningAgentsManagerListener>> Listeners;

private:

	/** Update the agent sets to keep them in sync with the id lists. */
	UE_API void UpdateAgentSets();

	/** Array of agent ids to be passed to events such as ULearningAgentsManagerListener::OnAgentAdded. */
	TArray<int32> OnEventAgentIds;

	/** Array of agent ids currently in use and associated with each agent object. */
	TArray<int32> OccupiedAgentIds;
	UE::Learning::FIndexSet OccupiedAgentSet;

	/* Array of agent ids currently free and available for assignment. */
	TArray<int32> VacantAgentIds;
	UE::Learning::FIndexSet VacantAgentSet;
};

#undef UE_API
