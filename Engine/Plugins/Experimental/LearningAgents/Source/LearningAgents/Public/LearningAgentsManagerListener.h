// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#include "Containers/Array.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "Templates/SubclassOf.h"

#include "LearningAgentsManagerListener.generated.h"

#define UE_API LEARNINGAGENTS_API

class ULearningAgentsManager;

/** Dummy class used for visual logging */
UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsVisualLoggerObject : public UObject
{
	GENERATED_BODY()
};

/**
 * Base class for objects which can be added to a ULearningAgentsManager to receive callbacks whenever agents are added, remove or reset.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable)
class ULearningAgentsManagerListener : public UObject
{
	GENERATED_BODY()

public:

	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsManagerListener(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	UE_API ULearningAgentsManagerListener(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsManagerListener();

	/**
	 * Called whenever agents are added to the manager.
	 *
	 * @param AgentIds		Array of agent ids which have been added
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents")
	UE_API void OnAgentsAdded(const TArray<int32>& AgentIds);

	/**
	 * Called whenever agents are removed from the manager.
	 *
	 * @param AgentIds		Array of agent ids which have been removed
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents")
	UE_API void OnAgentsRemoved(const TArray<int32>& AgentIds);

	/**
	 * Called whenever agents are reset on the manager.
	 *
	 * @param AgentIds		Array of agent ids which have been reset
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents")
	UE_API void OnAgentsReset(const TArray<int32>& AgentIds);

	/**
	 * Called whenever the manager object is ticked.
	 .
	 * @param AgentIds		Array of agent ids which have been ticked
	 * @param DeltaTime		The delta time associated with the tick
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents")
	UE_API void OnAgentsManagerTick(const TArray<int32>& AgentIds, const float DeltaTime);

	/** Returns true if this object has been setup. Otherwise, false. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API bool IsSetup() const;

// ----- Blueprint Convenience Functions -----
protected:

	/**
	 * Gets the agent with the given id from the manager. Calling this from blueprint with the appropriate AgentClass
	 * will automatically cast the object to the given type. If not in a blueprint, you should call the manager's
	 * GetAgent methods directly.
	 * 
	 * @param AgentId			The id of the agent to get.
	 * @param AgentClass		The class to cast the agent object to (in blueprint).
	 * @return					The agent object.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1", DeterminesOutputType = "AgentClass"))
	UE_API UObject* GetAgent(const int32 AgentId, const TSubclassOf<UObject> AgentClass) const;

	/**
	 * Gets the agents associated with a set of ids from the manager. Calling this from blueprint with the appropriate 
	 * AgentClass will automatically cast the object to the given type. If not in a blueprint, you should call the 
	 * manager's GetAgents method directly.
	 * 
	 * @param AgentIds			The ids of the agents to get.
	 * @param AgentClass		The class to cast the agent objects to (in blueprint).
	 * @param OutAgents			The output array of agent objects.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (DeterminesOutputType = "AgentClass", DynamicOutputParam = "OutAgents"))
	UE_API void GetAgents(const TArray<int32>& AgentIds, const TSubclassOf<UObject> AgentClass, TArray<UObject*>& OutAgents) const;

	/**
	 * Gets all added agents from the manager. Calling this from blueprint with the appropriate AgentClass will 
	 * automatically cast the object to the given type.
	 * 
	 * @param AgentClass		The class to cast the agent objects to (in blueprint).
	 * @param OutAgents			The output array of agent objects.
	 * @param OutAgentIds		The output array of agent ids.
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (DeterminesOutputType = "AgentClass", DynamicOutputParam = "OutAgents"))
	UE_API void GetAllAgents(TArray<UObject*>& OutAgents, TArray<int32>& OutAgentIds, const TSubclassOf<UObject> AgentClass) const;

	/** Gets the agent manager associated with this object. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API ULearningAgentsManager* GetAgentManager() const;

// ----- Non-blueprint public interface -----
public:

	/** Gets the agent corresponding to the given id. */
	UE_API const UObject* GetAgent(const int32 AgentId) const;

	/** Gets the agent corresponding to the given id. */
	UE_API UObject* GetAgent(const int32 AgentId);

	/** Checks if the manager has an agent with the given agent id */
	UE_API bool HasAgent(const int32 AgentId) const;

	/** Checks if the object has an associated agent manager. */
	UE_API bool HasAgentManager() const;

	/** Either gets or adds a visual logger object for the given name */
	UE_API const ULearningAgentsVisualLoggerObject* GetOrAddVisualLoggerObject(const FName Name);

protected:

	/** True if this object has been setup. Otherwise, false. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	bool bIsSetup = false;

	/** The manager this object is associated with. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsManager> Manager;

	/** The visual logger objects associated with this listener. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TMap<FName, TObjectPtr<const ULearningAgentsVisualLoggerObject>> VisualLoggerObjects;
};

#undef UE_API
