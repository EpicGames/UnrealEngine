// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "LearningLog.h"
#include "NNERuntimeBasicCpuBuilder.h"
#include "LearningAgentsNeuralNetwork.generated.h"

#define UE_API LEARNINGAGENTS_API

/** Activation functions for neural networks. */
UENUM(BlueprintType, Category = "LearningAgents")
enum class ELearningAgentsActivationFunction : uint8
{
	/** ReLU Activation - Fast to train and evaluate but occasionally causes gradient collapse and untrainable networks. */
	ReLU	UMETA(DisplayName = "ReLU"),

	/** ELU Activation - Generally performs better than ReLU and is not prone to gradient collapse but slower to evaluate. */
	ELU		UMETA(DisplayName = "ELU"),

	/** TanH Activation - Smooth activation function that is slower to train and evaluate but sometimes more stable for certain tasks. */
	TanH	UMETA(DisplayName = "TanH"),

	/** GELU Activation - Gaussian Error Linear Unit. */
	GELU	UMETA(DisplayName = "GELU"),
};

namespace UE::Learning::Agents::NeuralNetwork
{
	inline UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction GetBuilderActivationFunction(const ELearningAgentsActivationFunction ActivationFunction)
	{
		switch (ActivationFunction)
		{
		case ELearningAgentsActivationFunction::ReLU: return UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ReLU;
		case ELearningAgentsActivationFunction::ELU: return UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ELU;
		case ELearningAgentsActivationFunction::TanH: return UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction::TanH;
		case ELearningAgentsActivationFunction::GELU: return UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction::GELU;
		default:
			UE_LOGF(LogLearning, Error, "Unknown Activation Function");
			return UE::NNE::RuntimeBasic::FModelBuilder::EActivationFunction::ReLU;
		}
	}
}

/** Normalization options for neural network inputs & outputs. */
UENUM(BlueprintType)
enum class ELearningAgentsNormalization : uint8
{
	/** 
	 * Uses user specified scales and biases to normalize/denormalize network dataflow at the cpp schema layer.
	 * <IMPORTANT> Hand picked scales and offsets values will only be used if normalization is set to "Manual."
	 */
	Manual = 0,

	/**
	 * Uses dataset derived standard deviation and mean as scales and biases to normalize/denormalize network dataflow.
	 * If a feature has multiple dimensions, a single scale and bias will be computed for that feature.
	 * <IMPORTANT> Hand picked scales and offsets values will NOT be used if normalization is set to "AutoShared."
	 */
	AutoShared = 1,

	/**
	 * Uses dataset derived standard deviation and mean as scales and biases to normalize/denormalize network dataflow.
	 * If a feature has multiple dimensions, a scale and bias will be computed for each dimension.
	 * <IMPORTANT> Hand picked scales and offsets values will NOT be used if normalization is set to "AutoPerDimension."
	 */
	AutoPerDimension = 2,
};

class ULearningNeuralNetworkData;

/** A neural network data asset. */
UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsNeuralNetwork : public UDataAsset
{
	GENERATED_BODY()

public:
	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsNeuralNetwork();
	UE_API ULearningAgentsNeuralNetwork(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsNeuralNetwork();

	/** Resets this network asset to be empty. */
	UFUNCTION(CallInEditor, Category = "LearningAgents")
	UE_API void ResetNetwork();

	/**
	 * Load this network from a snapshot.
	 * 
	 * @param File The snapshot file.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (RelativePath))
	UE_API void LoadNetworkFromSnapshot(const FFilePath& File);

	/**
	 * Save this network into a snapshot.
	 * 
	 * @param File The snapshot file.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (RelativePath))
	UE_API void SaveNetworkToSnapshot(const FFilePath& File);

	/**
	 * Copy another asset into this network.
	 * 
	 * @param NeuralNetworkAsset The asset to load from.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void LoadNetworkFromAsset(ULearningAgentsNeuralNetwork* NeuralNetworkAsset);

	/**
	 * Copy this network into another asset.
	 * 
	 * @param NeuralNetworkAsset The asset to save to.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void SaveNetworkToAsset(ULearningAgentsNeuralNetwork* NeuralNetworkAsset);

public:

	/** Marks this asset as modified even during PIE */
	UE_API void ForceMarkDirty();

public:

	/** The internal Neural Network Data */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "LearningAgents");
	TObjectPtr<ULearningNeuralNetworkData> NeuralNetworkData;
};

#undef UE_API
