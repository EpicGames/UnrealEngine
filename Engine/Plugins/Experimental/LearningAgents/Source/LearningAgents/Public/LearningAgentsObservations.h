// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"
#include "LearningObservation.h"

#include "LearningAgentsNeuralNetwork.h" // Included for ELearningAgentsActivationFunction

#include "Engine/EngineTypes.h"
#include "GameFramework/OnlineReplStructs.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "LearningAgentsObservations.generated.h"

#define UE_API LEARNINGAGENTS_API

class ULearningAgentsManagerListener;
class ULearningAgentsObservationSchema;
class ULearningAgentsObservationObject;
struct FLearningAgentsObservationSchemaElement;
struct FLearningAgentsObservationObjectElement;

class USplineComponent;

/** An element of an Observation Schema */
USTRUCT(BlueprintType)
struct FLearningAgentsObservationSchemaElement
{
	GENERATED_BODY()

	UE::Learning::Observation::FSchemaElement SchemaElement;
};

/** An element of an Observation Object */
USTRUCT(BlueprintType)
struct FLearningAgentsObservationObjectElement
{
	GENERATED_BODY()

	UE::Learning::Observation::FObjectElement ObjectElement;
};

/** Settings for normalizing observations */
USTRUCT(BlueprintType)
struct FLearningAgentsObservationNormalizationSettings
{
	GENERATED_BODY()

	/** The normalization method to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	ELearningAgentsNormalization NormalizationOption = ELearningAgentsNormalization::Manual;

	/** Per-element standard deviation values used for normalization. If empty, no scaling is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	TArray<float> Scales;

	/** Per-element mean values used for normalization. If empty, no bias is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	TArray<float> Offsets;
};

/** Settings for normalizing observations */
USTRUCT(BlueprintType)
struct FLearningAgentsFloatObservationNormalizationSettings
{
	GENERATED_BODY()

	/** The normalization method to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	ELearningAgentsNormalization NormalizationOption = ELearningAgentsNormalization::Manual;

	/** Per-element standard deviation values used for normalization. If empty, no scaling is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	float Scale = 1.0f;

	/** Per-element mean values used for normalization. If empty, no bias is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	float Offset = 0.0f;
};

/** Settings for normalizing observations */
USTRUCT(BlueprintType)
struct FLearningAgentsVectorObservationNormalizationSettings
{
	GENERATED_BODY()

	/** The normalization method to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	ELearningAgentsNormalization NormalizationOption = ELearningAgentsNormalization::Manual;

	/** Per-element standard deviation values used for normalization. If empty, no scaling is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	FVector Scales = FVector::OneVector;

	/** Per-element mean values used for normalization. If empty, no bias is applied. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (EditCondition = "NormalizationOption == ELearningAgentsNormalization::Manual"))
	FVector Offsets = FVector::ZeroVector;
};

/** Settings for visual logger output */
USTRUCT(BlueprintType)
struct FLearningAgentsObservationVisualLoggerSettings
{
	GENERATED_BODY()

	/** Whether visual logging is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	bool bEnabled = false;

	/** The listener object which is making this observation. This must be set to use logging. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsManagerListener> Listener = nullptr;

	/** The agent id associated with this observation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	int32 AgentId = -1;

	/** An additional location for visualizing a specific observation element (e.g., rotation arrow origin, velocity arrow start). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	FVector DebugLocation = FVector::ZeroVector;

	/** A location for the visual logger text information in the world. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	FVector Location = FVector::ZeroVector;

	/** The color for the visual logger display. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	FLinearColor Color = FLinearColor::Red;

	/** The length of debug arrows to display for direction/velocity observations. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	float ArrowLength = 100.0f;
};

/** Comparison and Hashing operators for Observation Elements */

LEARNINGAGENTS_API bool operator==(const FLearningAgentsObservationSchemaElement& Lhs, const FLearningAgentsObservationSchemaElement& Rhs);
LEARNINGAGENTS_API bool operator==(const FLearningAgentsObservationObjectElement& Lhs, const FLearningAgentsObservationObjectElement& Rhs);

LEARNINGAGENTS_API uint32 GetTypeHash(const FLearningAgentsObservationSchemaElement& Element);
LEARNINGAGENTS_API uint32 GetTypeHash(const FLearningAgentsObservationObjectElement& Element);

template<>
struct TStructOpsTypeTraits<FLearningAgentsObservationSchemaElement> : public TStructOpsTypeTraitsBase2<FLearningAgentsObservationSchemaElement>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};

template<>
struct TStructOpsTypeTraits<FLearningAgentsObservationObjectElement> : public TStructOpsTypeTraitsBase2<FLearningAgentsObservationObjectElement>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};


/**
 * Observation Schema
 *
 * This object is used to construct a schema describing some structure of observations.
 */
UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsObservationSchema : public UObject
{
	GENERATED_BODY()

public:

	UE::Learning::Observation::FSchema ObservationSchema;
};

/**
 * Observation Object
 *
 * This object is used to construct or get the values of observations.
 */
UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsObservationObject : public UObject
{
	GENERATED_BODY()

public:

	UE::Learning::Observation::FObject ObservationObject;
};

/** Enum Type representing either observation A or observation B */
UENUM(BlueprintType)
enum class ELearningAgentsEitherObservation : uint8
{
	A,
	B,
};

/** Enum Type representing either a Null observation or some Valid observation */
UENUM(BlueprintType)
enum class ELearningAgentsOptionalObservation : uint8
{
	Null,
	Valid,
};

/** Padding Modes for Convolutions. */
UENUM(BlueprintType, Category = "LearningAgents")
enum class ELearningAgentsPaddingMode : uint8
{
	Zeros = 0	UMETA(DisplayName = "Zeros"),
	Circular = 1	UMETA(DisplayName = "Circular"),
};

/** A wrapper of Conv1d observation parameters */
USTRUCT(BlueprintType)
struct FConv1dObservationParams
{
    GENERATED_BODY()

    /** Length of the 1D input. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 InputLength = 0;

    /** Number of input channels. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 InChannels = 0;

    /** Number of output channels. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 OutChannels = 0;

    /** Size of the convolution kernel. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 KernelSize = 0;

    /** Amount of padding to add. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 Padding = 0;

    /** The padding mode. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    ELearningAgentsPaddingMode PaddingMode = ELearningAgentsPaddingMode::Circular;

    /** Activation applied after the convolution. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    ELearningAgentsActivationFunction ActivationFunction = ELearningAgentsActivationFunction::ReLU;

	FConv1dObservationParams() = default;

	FConv1dObservationParams(
		int32 InInputLength,
		int32 InInChannels,
		int32 InOutChannels,
		int32 InKernelSize,
		int32 InPadding,
		ELearningAgentsPaddingMode InPaddingMode,
		ELearningAgentsActivationFunction InActivationFunction = ELearningAgentsActivationFunction::ReLU
	):
		InputLength(InInputLength),
		InChannels(InInChannels),
		OutChannels(InOutChannels),
		KernelSize(InKernelSize),
		Padding(InPadding),
		PaddingMode(InPaddingMode),
		ActivationFunction(InActivationFunction)
	{}
};

/** A wrapper of Conv2d observation parameters */
USTRUCT(BlueprintType)
struct FConv2dObservationParams
{
    GENERATED_BODY()

    /** Height of the 2D input. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 InputHeight = 0;

    /** Width of the 2D input. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 InputWidth = 0;

    /** Number of input channels. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 InChannels = 0;

    /** Number of output channels. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 OutChannels = 0;

    /** Size of the convolution kernel. (3 -> (3x3), 5 -> (5x5) etc) */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 KernelSize = 0;

    /** Stride of the convolution. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 Stride = 0;

    /** Amount of zero‑padding on each side. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    int32 Padding = 0;

    /** Activation applied after the convolution. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="LearningAgents")
    ELearningAgentsActivationFunction ActivationFunction = ELearningAgentsActivationFunction::ReLU;

	FConv2dObservationParams() = default;

	FConv2dObservationParams(
		int32 InInputHeight,
		int32 InInputWidth,
		int32 InInChannels,
		int32 InOutChannels,
		int32 InKernelSize,
		int32 InStride,
		int32 InPadding,
		ELearningAgentsActivationFunction InActivationFunction = ELearningAgentsActivationFunction::ReLU
	):
		InputHeight(InInputHeight),
		InputWidth(InInputWidth),
		InChannels(InInChannels),
		OutChannels(InOutChannels),
		KernelSize(InKernelSize),
		Stride(InStride),
		Padding(InPadding),
		ActivationFunction(InActivationFunction)
	{}
};

UCLASS(MinimalAPI)
class ULearningAgentsObservations : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Project a transform onto the ground plane, leaving just rotation around the vertical axis */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	static UE_API FTransform ProjectTransformOntoGroundPlane(const FTransform Transform, const FVector LocalForwardVector = FVector::ForwardVector, const float GroundPlaneHeight = 0.0f);


	/** Find an Enum type by Name. This can be used to find Enum types defined in C++. This call can be expensive so the result should be cached. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	static UE_API UEnum* FindEnumByName(const FString& Name);

	/**
	 * Validates that the given observation object matches the schema. Will log errors on objects that don't match.
	 *
	 * @param Schema				Observation Schema
	 * @param SchemaElement			Observation Schema Element
	 * @param Object				Observation Object
	 * @param ObjectElement			Observation Object Element
	 * @returns						true if the object matches the schema
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents")
	static UE_API bool ValidateObservationObjectMatchesSchema(
		const ULearningAgentsObservationSchema* Schema,
		const FLearningAgentsObservationSchemaElement SchemaElement,
		const ULearningAgentsObservationObject* Object,
		const FLearningAgentsObservationObjectElement ObjectElement);

	/**
	 * Logs an Observation Object Element. Useful for debugging.
	 *
	 * @param Object				Observation Object
	 * @param ObjectElement			Observation Object Element
	 */
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents")
	static UE_API void LogObservation(const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element);

public:

	/**
	 * Specifies a new null observation. This represents an empty observation and can be useful when an observation is needed which has no value.
	 *
	 * @param Schema The Observation Schema
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyNullObservation(ULearningAgentsObservationSchema* Schema, const FName Tag = TEXT("NullObservation"));

	/**
	 * Specifies a new continuous observation. This represents an observation made up of several float values.
	 *
	 * @param Schema The Observation Schema
	 * @param Size The number of float values in the observation.
	 * @param Scale The scale used to normalize this observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @param NormalizationSettings Optional normalization settings containing per-element Scale, Bias, and whether to apply normalization.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "NormalizationSettings"))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyContinuousObservation(ULearningAgentsObservationSchema* Schema, const int32 Size, const FLearningAgentsObservationNormalizationSettings& NormalizationSettings = FLearningAgentsObservationNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("ContinuousObservation"));

	/**
	 * Specifies a new exclusive discrete observation. This represents a discrete observation which is an exclusive selection from multiple choices.
	 *
	 * @param Schema The Observation Schema
	 * @param Size The number of discrete options in the observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyExclusiveDiscreteObservation(ULearningAgentsObservationSchema* Schema, const int32 Size, const bool bActive = true, const FName Tag = TEXT("ExclusiveDiscreteObservation"));

	/**
	 * Specifies a new inclusive discrete observation. This represents a discrete observation which is an inclusive selection from multiple choices.
	 *
	 * @param Schema The Observation Schema
	 * @param Size The number of discrete options in the observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyInclusiveDiscreteObservation(ULearningAgentsObservationSchema* Schema, const int32 Size, const bool bActive = true, const FName Tag = TEXT("InclusiveDiscreteObservation"));

	/**
	 * Specifies a new named exclusive discrete observation. This represents a discrete observation which is an exclusive selection from multiple choices.
	 *
	 * @param Schema The Observation Schema
	 * @param ElementNames The names of the discrete options in the observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyNamedExclusiveDiscreteObservation(ULearningAgentsObservationSchema* Schema, const TArray<FName>& ElementNames, const bool bActive = true, const FName Tag = TEXT("NamedExclusiveDiscreteObservation"));

	/**
	 * Specifies a new named exclusive discrete observation. This represents a discrete observation which is an exclusive selection from multiple choices.
	 *
	 * @param Schema The Observation Schema
	 * @param ElementNames The names of the discrete options in the observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	static UE_API FLearningAgentsObservationSchemaElement SpecifyNamedExclusiveDiscreteObservationFromArrayView(ULearningAgentsObservationSchema* Schema, const TArrayView<const FName> ElementNames, const bool bActive = true, const FName Tag = TEXT("NamedExclusiveDiscreteObservation"));

	/**
	 * Specifies a new named inclusive discrete observation. This represents a discrete observation which is an inclusive selection from multiple choices.
	 *
	 * @param Schema The Observation Schema
	 * @param ElementNames The names of the discrete options in the observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyNamedInclusiveDiscreteObservation(ULearningAgentsObservationSchema* Schema, const TArray<FName>& ElementNames, const bool bActive = true, const FName Tag = TEXT("NamedInclusiveDiscreteObservation"));

	/**
	 * Specifies a new named inclusive discrete observation. This represents a discrete observation which is an inclusive selection from multiple choices.
	 *
	 * @param Schema The Observation Schema
	 * @param ElementNames The names of the discrete options in the observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	static UE_API FLearningAgentsObservationSchemaElement SpecifyNamedInclusiveDiscreteObservationFromArrayView(ULearningAgentsObservationSchema* Schema, const TArrayView<const FName> ElementNames, const bool bActive = true, const FName Tag = TEXT("NamedInclusiveDiscreteObservation"));

	/**
	 * Specifies a new count observation. This represents a count of something such as the size of, or index into, an array.
	 *
	 * @param Schema The Observation Schema
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyCountObservation(ULearningAgentsObservationSchema* Schema, const bool bActive = true, const FName Tag = TEXT("CountObservation"));

	/**
	 * Specifies a new struct observation. This represents a group of named sub-observations.
	 *
	 * @param Schema The Observation Schema
	 * @param Elements The sub-observations that make up this struct.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyStructObservation(ULearningAgentsObservationSchema* Schema, const TMap<FName, FLearningAgentsObservationSchemaElement>& Elements, const bool bActive = true, const FName Tag = TEXT("StructObservation"));

	/**
	 * Specifies a new struct observation. This represents a group of named sub-observations.
	 *
	 * @param Schema The Observation Schema
	 * @param ElementNames The names of the sub-observations that make up this struct.
	 * @param Elements The corresponding sub-observations that make up this struct. Must be the same size as ElementNames.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyStructObservationFromArrays(ULearningAgentsObservationSchema* Schema, const TArray<FName>& ElementNames, const TArray<FLearningAgentsObservationSchemaElement>& Elements, const bool bActive = true, const FName Tag = TEXT("StructObservation"));
	
	/**
	 * Specifies a new struct observation. This represents a group of named sub-observations.
	 *
	 * @param Schema The Observation Schema
	 * @param ElementNames The names of the sub-observations that make up this struct.
	 * @param Elements The corresponding sub-observations that make up this struct. Must be the same size as ElementNames.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	static UE_API FLearningAgentsObservationSchemaElement SpecifyStructObservationFromArrayViews(ULearningAgentsObservationSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsObservationSchemaElement> Elements, const bool bActive = true, const FName Tag = TEXT("StructObservation"));

	/**
	 * Specifies a new exclusive union observation. This represents an observation which is exclusively chosen from a set of named sub-observations.
	 * In other words, when this observation is created, you only need to provide one observation from the given sub-observations.
	 *
	 * @param Schema The Observation Schema
	 * @param Elements The sub-observations that make up this union.
	 * @param EncodingSize The encoding size used to encode each sub-observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyExclusiveUnionObservation(ULearningAgentsObservationSchema* Schema, const TMap<FName, FLearningAgentsObservationSchemaElement>& Elements, const int32 EncodingSize = 128, const bool bActive = true, const FName Tag = TEXT("ExclusiveUnionObservation"));

	/**
	 * Specifies a new exclusive union observation. This represents an observation which is exclusively chosen from a set of named sub-observations.
	 * In other words, when this observation is created, you only need to provide one observation from the given sub-observations.
	 *
	 * @param Schema The Observation Schema
	 * @param ElementNames The names of the sub-observations that make up this union.
	 * @param Elements The corresponding sub-observations that make up this union. Must be the same size as ElementNames.
	 * @param EncodingSize The encoding size used to encode each sub-observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyExclusiveUnionObservationFromArrays(ULearningAgentsObservationSchema* Schema, const TArray<FName>& ElementNames, const TArray<FLearningAgentsObservationSchemaElement>& Elements, const int32 EncodingSize = 128, const bool bActive = true, const FName Tag = TEXT("ExclusiveUnionObservation"));
	
	/**
	 * Specifies a new exclusive union observation. This represents an observation which is exclusively chosen from a set of named sub-observations.
	 * In other words, when this observation is created, you only need to provide one observation from the given sub-observations.
	 *
	 * @param Schema The Observation Schema
	 * @param ElementNames The names of the sub-observations that make up this union.
	 * @param Elements The corresponding sub-observations that make up this union. Must be the same size as ElementNames.
	 * @param EncodingSize The encoding size used to encode each sub-observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	static UE_API FLearningAgentsObservationSchemaElement SpecifyExclusiveUnionObservationFromArrayViews(ULearningAgentsObservationSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsObservationSchemaElement> Elements, const int32 EncodingSize = 128, const bool bActive = true, const FName Tag = TEXT("ExclusiveUnionObservation"));

	/**
	 * Specifies a new inclusive union observation. This represents an observation which is inclusively chosen from a set of named sub-observations.
	 * In other words, when this observation is created, you can provide any combination of observations from the given sub-observations. Internally
	 * this observation uses Attention so can be slower to evaluate and more difficult to train than other observation types. For this reason it 
	 * should be used sparingly.
	 *
	 * @param Schema The Observation Schema
	 * @param Elements The sub-observations that make up this union.
	 * @param AttentionEncodingSize The encoding size used by the attention mechanism.
	 * @param AttentionHeadNum The number of heads used by the attention mechanism.
	 * @param ValueEncodingSize The output encoding size used by the attention mechanism.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyInclusiveUnionObservation(ULearningAgentsObservationSchema* Schema, const TMap<FName, FLearningAgentsObservationSchemaElement>& Elements, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionObservation"));

	/**
	 * Specifies a new inclusive union observation. This represents an observation which is inclusively chosen from a set of named sub-observations.
	 * In other words, when this observation is created, you can provide any combination of observations from the given sub-observations. Internally
	 * this observation uses Attention so can be slower to evaluate and more difficult to train than other observation types. For this reason it
	 * should be used sparingly.
	 *
	 * @param Schema The Observation Schema
	 * @param ElementNames The names of the sub-observations that make up this union.
	 * @param Elements The corresponding sub-observations that make up this union. Must be the same size as ElementNames.
	 * @param AttentionEncodingSize The encoding size used by the attention mechanism.
	 * @param AttentionHeadNum The number of heads used by the attention mechanism.
	 * @param ValueEncodingSize The output encoding size used by the attention mechanism.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyInclusiveUnionObservationFromArrays(ULearningAgentsObservationSchema* Schema, const TArray<FName>& ElementNames, const TArray<FLearningAgentsObservationSchemaElement>& Elements, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionObservation"));
	
	/**
	 * Specifies a new inclusive union observation. This represents an observation which is inclusively chosen from a set of named sub-observations.
	 * In other words, when this observation is created, you can provide any combination of observations from the given sub-observations. Internally
	 * this observation uses Attention so can be slower to evaluate and more difficult to train than other observation types. For this reason it
	 * should be used sparingly.
	 *
	 * @param Schema The Observation Schema
	 * @param ElementNames The names of the sub-observations that make up this union.
	 * @param Elements The corresponding sub-observations that make up this union. Must be the same size as ElementNames.
	 * @param AttentionEncodingSize The encoding size used by the attention mechanism.
	 * @param AttentionHeadNum The number of heads used by the attention mechanism.
	 * @param ValueEncodingSize The output encoding size used by the attention mechanism.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	static UE_API FLearningAgentsObservationSchemaElement SpecifyInclusiveUnionObservationFromArrayViews(ULearningAgentsObservationSchema* Schema, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsObservationSchemaElement> Elements, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionObservation"));

	/**
	 * Specifies a new static array observation. This represents an observation made up of a fixed-size array of some other observation.
	 *
	 * @param Schema The Observation Schema
	 * @param Element The sub-observation that represents elements of this array.
	 * @param Num The number of elements in the fixed size array.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyStaticArrayObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const int32 Num, const bool bActive = true, const FName Tag = TEXT("StaticArrayObservation"));

	/**
	 * Specifies a new set observation. This represents an observation made up of a Set of some other observation. This Set can be variable in size 
	 * (up to some fixed maximum size) and elements are considered unordered. Internally this observation uses Attention so can be slower to evaluate 
	 * and more difficult to train than other observation types. For this reason it should be used sparingly.
	 *
	 * @param Schema The Observation Schema
	 * @param Element The sub-observation that represents elements of this array.
	 * @param MaxNum The maximum number of elements that can be included in the set.
	 * @param AttentionEncodingSize The encoding size used by the attention mechanism.
	 * @param AttentionHeadNum The number of heads used by the attention mechanism.
	 * @param ValueEncodingSize The output encoding size used by the attention mechanism.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationSchemaElement SpecifySetObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const int32 MaxNum, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const bool bActive = true, const FName Tag = TEXT("SetObservation"));

	/**
	 * Specifies a new pair observation. This represents an observation made up of two sub-observations.
	 *
	 * @param Schema The Observation Schema
	 * @param Key The first sub-observation.
	 * @param Value The second sub-observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyPairObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Key, const FLearningAgentsObservationSchemaElement Value, const bool bActive = true, const FName Tag = TEXT("PairObservation"));

	/**
	 * Specifies a new array observation. This represents an observation made up of an Array of some other observation. This Array can be variable in 
	 * size (up to some fixed maximum size) and the order of elements is taken into consideration. Internally this observation uses Attention so can 
	 * be slower to evaluate and more difficult to train than other observation types. For this reason it should be used sparingly.
	 *
	 * @param Schema The Observation Schema
	 * @param Element The sub-observation that represents elements of this array.
	 * @param MaxNum The maximum number of elements that can be included in the array.
	 * @param AttentionEncodingSize The encoding size used by the attention mechanism.
	 * @param AttentionHeadNum The number of heads used by the attention mechanism.
	 * @param ValueEncodingSize The output encoding size used by the attention mechanism.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyArrayObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const int32 MaxNum, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const bool bActive = true, const FName Tag = TEXT("ArrayObservation"));

	/**
	 * Specifies a new map observation. This represents an observation made up of a Map of some other key and pair observations. This Map can be 
	 * variable in size (up to some fixed maximum size) and elements are considered unordered. Internally this observation uses Attention so can
	 * be slower to evaluate and more difficult to train than other observation types. For this reason it should be used sparingly.
	 *
	 * @param Schema The Observation Schema
	 * @param KeyElement The sub-observation that represents keys in this map.
	 * @param ValueElement The sub-observation that represents values in this map.
	 * @param MaxNum The maximum number of elements that can be included in the map.
	 * @param AttentionEncodingSize The encoding size used by the attention mechanism.
	 * @param AttentionHeadNum The number of heads used by the attention mechanism.
	 * @param ValueEncodingSize The output encoding size used by the attention mechanism.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyMapObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement KeyElement, const FLearningAgentsObservationSchemaElement ValueElement, const int32 MaxNum, const int32 AttentionEncodingSize = 32, const int32 AttentionHeadNum = 4, const int32 ValueEncodingSize = 32, const bool bActive = true, const FName Tag = TEXT("MapObservation"));

	/**
	 * Specifies a new enum observation. This represents an exclusive choice from elements of the given Enum. To use this with an Enum defined in C++
	 * use the FindEnumByName convenience function.
	 *
	 * @param Schema The Observation Schema
	 * @param Enum The enum type to use.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyEnumObservation(ULearningAgentsObservationSchema* Schema, const UEnum* Enum, const bool bActive = true, const FName Tag = TEXT("EnumObservation"));

	/**
	 * Specifies a new bitmask observation. This represents an inclusive choice from elements of the given Enum. To use this with an Enum defined in
	 * C++ use the FindEnumByName convenience function.
	 *
	 * @param Schema The Observation Schema
	 * @param Enum The enum type to use.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyBitmaskObservation(ULearningAgentsObservationSchema* Schema, const UEnum* Enum, const bool bActive = true, const FName Tag = TEXT("BitmaskObservation"));

	/**
	 * Specifies a new optional observation. This represents an observation which may or may not be provided.
	 *
	 * @param Schema The Observation Schema
	 * @param Elements The sub-observation that may or may not be provided.
	 * @param EncodingSize The encoding size used to encode this sub-observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyOptionalObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const int32 EncodingSize = 128, const bool bActive = true, const FName Tag = TEXT("OptionalObservation"));

	/**
	 * Specifies a new either observation. This represents an observation which will be either sub-observation A or sub-observation B.
	 *
	 * @param Schema The Observation Schema
	 * @param A The first sub-observation.
	 * @param A The second sub-observation.
	 * @param EncodingSize The encoding size used to encode each sub-observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyEitherObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement A, const FLearningAgentsObservationSchemaElement B, const int32 EncodingSize = 128, const bool bActive = true, const FName Tag = TEXT("EitherObservation"));

	/**
	 * Specifies a new encoding observation. This represents an observation which will be an encoding of another sub-observation using a small neural
	 * network.
	 *
	 * @param Schema The Observation Schema
	 * @param Element The sub-observation to be encoded.
	 * @param EncodingSize The encoding size used to encode this sub-observation.
	 * @param HiddenLayerNum The number of hidden layers used to encode this sub-observation.
	 * @param ActivationFunction The activation function used to encode this sub-observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyEncodingObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const int32 EncodingSize = 128, const int32 HiddenLayerNum = 1, const ELearningAgentsActivationFunction ActivationFunction = ELearningAgentsActivationFunction::ELU, const bool bActive = true, const FName Tag = TEXT("EncodingObservation"));

	/**
	 * Specifies a new bool observation. A true or false observation.
	 *
	 * @param Schema The Observation Schema
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyBoolObservation(ULearningAgentsObservationSchema* Schema, const bool bActive = true, const FName Tag = TEXT("BoolObservation"));

	/**
	 * Specifies a new 1d convolution observation of another observation (Flat buffer input, Channel x Length).
	 *
	 * @param Schema The Observation Schema
	 * @param Element The sub-observation to be convolved.
	 * @param FConv1dObservationParams The convolution parameters.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 9))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyConv1dObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const FConv1dObservationParams& Conv1dParams, const bool bActive = true, const FName Tag = TEXT("Conv1dObservation"));

	/**
	 * Specifies a new 2d convolution observation of another observation (Flat buffer input, Channels x Input Height x Input Width), Zeros Padding.
	 *
	 * @param Schema The Observation Schema
	 * @param Element The sub-observation to be convolved.
	 * @param FConv2dObservationParams The convolution parameters.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	 UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 10))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyConv2dObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsObservationSchemaElement Element, const FConv2dObservationParams& Conv2dParams, const bool bActive = true, const FName Tag = TEXT("Conv2dObservation"));

	/**
	 * Specifies a new float observation. A simple observation which can be used as a catch-all for situations where a
	 * type-specific observation does not exist.
	 *
	 * @param Schema The Observation Schema
	 * @param NormalizationSettings The normalization used to normalize this observation.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings"))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyFloatObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsFloatObservationNormalizationSettings& NormalizationSettings = FLearningAgentsFloatObservationNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("FloatObservation"));

	/**
	 * Specifies a new location observation. Allows an agent to observe the location of some entity.
	 *
	 * @param Schema The Observation Schema
	 * @param NormalizationSettings The normalization used to normalize this observation in cm.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings", NormalizationSettings = "(Scales=(X=100,Y=100,Z=100))"))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyLocationObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsVectorObservationNormalizationSettings& NormalizationSettings = FLearningAgentsVectorObservationNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("LocationObservation"));

	/**
	 * Specifies a new rotation observation. Allows an agent to observe the rotation of some entity. Rotations are encoded as two columns of the
	 * rotation matrix to ensure there is no discontinuity in the encoding.
	 *
	 * @param Schema The Observation Schema
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyRotationObservation(ULearningAgentsObservationSchema* Schema, const bool bActive = true, const FName Tag = TEXT("RotationObservation"));

	/**
	 * Specifies a new scale observation. Allows an agent to observe the scale of some entity. Negative scales are not supported by this observation
	 * type.
	 *
	 * @param Schema The Observation Schema
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyScaleObservation(ULearningAgentsObservationSchema* Schema, const bool bActive = true, const FName Tag = TEXT("ScaleObservation"));

	/**
	 * Specifies a new transform observation. Allows an agent to observe the transform of some entity.
	 *
	 * @param Schema The Observation Schema
	 * @param NormalizationSettings The normalization used to normalize the Location part of this transform observation in cm.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings", NormalizationSettings = "(Scales=(X=100,Y=100,Z=100))"))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyTransformObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsVectorObservationNormalizationSettings& NormalizationSettings = FLearningAgentsVectorObservationNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("TransformObservation"));

	/**
	 * Specifies a new angle observation. This will be encoded as a 2-dimension Cartesian vector so that 0 and 350 are close to each other in the
	 * encoded space.
	 *
	 * @param Schema The Observation Schema
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.SpecifyAngleObservation
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyAngleObservation(ULearningAgentsObservationSchema* Schema, const bool bActive = true, const FName Tag = TEXT("AngleObservation"));

	/**
	 * Specifies a new velocity observation. Allows an agent to observe the velocity of some entity.
	 *
	 * @param Schema The Observation Schema
	 * @param VelocityScale The scale used to normalize this observation in cm/s.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings", NormalizationSettings = "(Scales=(X=200,Y=200,Z=200))"))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyVelocityObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsVectorObservationNormalizationSettings& NormalizationSettings = FLearningAgentsVectorObservationNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("VelocityObservation"));

	/**
	 * Specifies a new direction observation. Allows an agent to observe the direction of some entity.
	 *
	 * @param Schema The Observation Schema
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyDirectionObservation(ULearningAgentsObservationSchema* Schema, const bool bActive = true, const FName Tag = TEXT("DirectionObservation"));

	/**
	 * Specifies a new location along spline observation. This observes the location of the spline at the given distance along that spline.
	 *
	 * @param Schema The Observation Schema
	 * @param LocationScale The scale used to normalize this observation in cm.
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @param NormalizationSettings Optional normalization settings containing per-element Scales, Biases, and whether to apply normalization.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "NormalizationSettings", NormalizationSettings = "(Scales=(X=100,Y=100,Z=100))"))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyLocationAlongSplineObservation(ULearningAgentsObservationSchema* Schema, const FLearningAgentsVectorObservationNormalizationSettings& NormalizationSettings = FLearningAgentsVectorObservationNormalizationSettings(), const bool bActive = true, const FName Tag = TEXT("LocationAlongSplineObservation"));

	/**
	 * Specifies a new proportion along spline observation. This observes the proportion along a spline at the given distance. For looped splines
	 * this will be treated effectively like an angle between 0 and 360 degrees and encoded appropriately so that 0 and 350 are close to each other in the
	 * encoded space, while for non-looped splines this will be treated as a value between 0 and 1.
	 *
	 * @param Schema The Observation Schema
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyProportionAlongSplineObservation(ULearningAgentsObservationSchema* Schema, const bool bActive = true, const FName Tag = TEXT("ProportionAlongSplineObservation"));

	/**
	 * Specifies a new direction along spline observation. This observes the direction of the spline at the given distance along that spline.
	 *
	 * @param Schema The Observation Schema
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyDirectionAlongSplineObservation(ULearningAgentsObservationSchema* Schema, const bool bActive = true, const FName Tag = TEXT("DirectionAlongSplineObservation"));

	/**
	 * Specifies a new proportion along ray observation. This observes how far a you can travel along a ray before collision. Rays that can travel
	 * the full distance are encoded as zero, while rays that collide instantly are encoded as one.
	 *
	 * @param Schema The Observation Schema
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyProportionAlongRayObservation(ULearningAgentsObservationSchema* Schema, const bool bActive = true, const FName Tag = TEXT("ProportionAlongRayObservation"));
	
	/**
	 * Specifies a new LiDAR observation. This observes how far a you can travel along a ray before collision. Rays that can travel
	 * the full distance are encoded as one, while rays that collide instantly are encoded as zero.
	 *
	 * @param Schema The Observation Schema
	 * @param bActive Whether to enable the observation as a valid network input. If set to inactive, this observation will be considered an empty observation. This flag allows easy toggles of specified observation.
	 * @param Tag The tag of this new observation. Used during observation object validation and debugging.
	 * @return The newly created observation schema element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationSchemaElement SpecifyLidarObservation(ULearningAgentsObservationSchema* Schema, const bool bActive = true, const FName Tag = TEXT("LidarObservation"));

public:

	/**
	 * Make a new null observation.
	 *
	 * @param Object The Observation Object
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationObjectElement MakeNullObservation(ULearningAgentsObservationObject* Object, const FName Tag = TEXT("NullObservation"));

	/**
	 * Make a new continuous observation. The size of Values must match the Size given during Specify.
	 *
	 * @param Object The Observation Object
	 * @param Values The observation values.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeContinuousObservation(
		ULearningAgentsObservationObject* Object,
		const TArray<float>& Values,
		const bool bActive = true,
		const FName Tag = TEXT("ContinuousObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new continuous observation. The size of Values must match the Size given during Specify.
	 *
	 * @param Object The Observation Object
	 * @param Values The observation values.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	static UE_API FLearningAgentsObservationObjectElement MakeContinuousObservationFromArrayView(
		ULearningAgentsObservationObject* Object,
		const TArrayView<const float> Values,
		const bool bActive = true,
		const FName Tag = TEXT("ContinuousObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new exclusive discrete observation.
	 *
	 * @param Object The Observation Object
	 * @param DiscreteIndex The index of the discrete observation.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeExclusiveDiscreteObservation(
		ULearningAgentsObservationObject* Object, 
		const int32 DiscreteIndex, 
		const bool bActive = true, 
		const FName Tag = TEXT("ExclusiveDiscreteObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new inclusive discrete observation.
	 *
	 * @param Object The Observation Object
	 * @param DiscreteIndices The indices of the discrete observations.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeInclusiveDiscreteObservation(
		ULearningAgentsObservationObject* Object, 
		const TArray<int32>& DiscreteIndices, 
		const bool bActive = true, 
		const FName Tag = TEXT("InclusiveDiscreteObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());
	
	/**
	 * Make a new inclusive discrete observation.
	 *
	 * @param Object The Observation Object
	 * @param DiscreteIndices The indices of the discrete observations.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	static UE_API FLearningAgentsObservationObjectElement MakeInclusiveDiscreteObservationFromArrayView(
		ULearningAgentsObservationObject* Object, 
		const TArrayView<const int32> DiscreteIndices, 
		const bool bActive = true, 
		const FName Tag = TEXT("InclusiveDiscreteObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new named exclusive discrete observation.
	 *
	 * @param Object The Observation Object
	 * @param ElementName The name of the discrete observation.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeNamedExclusiveDiscreteObservation(
		ULearningAgentsObservationObject* Object,
		const FName ElementName,
		const bool bActive = true, 
		const FName Tag = TEXT("NamedExclusiveDiscreteObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new named inclusive discrete observation.
	 *
	 * @param Object The Observation Object
	 * @param ElementNames The names of the discrete observations.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeNamedInclusiveDiscreteObservation(
		ULearningAgentsObservationObject* Object,
		const TArray<FName>& ElementNames,
		const bool bActive = true, 
		const FName Tag = TEXT("NamedInclusiveDiscreteObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new named inclusive discrete observation.
	 *
	 * @param Object The Observation Object
	 * @param ElementNames The names of the discrete observations.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	static UE_API FLearningAgentsObservationObjectElement MakeNamedInclusiveDiscreteObservationFromArrayView(
		ULearningAgentsObservationObject* Object,
		const TArrayView<const FName> ElementNames,
		const bool bActive = true, 
		const FName Tag = TEXT("NamedInclusiveDiscreteObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new count observation.
	 *
	 * @param Object The Observation Object
	 * @param Num The number of items. Must be less than or equal to MaxNum.
	 * @param MaxNum The maximum number of items possible.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeCountObservation(
		ULearningAgentsObservationObject* Object, 
		const int32 Num, 
		const int32 MaxNum, 
		const bool bActive = true, 
		const FName Tag = TEXT("CountObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new struct observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The named sub-observations. Must match what was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationObjectElement MakeStructObservation(ULearningAgentsObservationObject* Object, const TMap<FName, FLearningAgentsObservationObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("StructObservation"));

	/**
	 * Make a new struct observation.
	 *
	 * @param Object The Observation Object
	 * @param ElementNames The names of the sub-observations. Must match what was given during Specify.
	 * @param Elements The corresponding sub-observations. Must match what was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationObjectElement MakeStructObservationFromArrays(ULearningAgentsObservationObject* Object, const TArray<FName>& ElementNames, const TArray<FLearningAgentsObservationObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("StructObservation"));
	
	/**
	 * Make a new struct observation.
	 *
	 * @param Object The Observation Object
	 * @param ElementNames The names of the sub-observations. Must match what was given during Specify.
	 * @param Elements The corresponding sub-observations. Must match what was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	static UE_API FLearningAgentsObservationObjectElement MakeStructObservationFromArrayViews(ULearningAgentsObservationObject* Object, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsObservationObjectElement> Elements, const bool bActive = true, const FName Tag = TEXT("StructObservation"));

	/**
	 * Make a new exclusive union observation.
	 *
	 * @param Object The Observation Object
	 * @param ElementName The name of the chosen sub-observation.
	 * @param Element The corresponding chosen sub-observation.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationObjectElement MakeExclusiveUnionObservation(ULearningAgentsObservationObject* Object, const FName ElementName, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ExclusiveUnionObservation"));

	/**
	 * Make a new inclusive union observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The chosen sub-observations.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "Elements"))
	static UE_API FLearningAgentsObservationObjectElement MakeInclusiveUnionObservation(ULearningAgentsObservationObject* Object, const TMap<FName, FLearningAgentsObservationObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionObservation"));

	/**
	 * Make a new inclusive union observation.
	 *
	 * @param Object The Observation Object
	 * @param ElementNames The names of the chosen sub-observations.
	 * @param Elements The corresponding chosen sub-observations.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "ElementNames,Elements"))
	static UE_API FLearningAgentsObservationObjectElement MakeInclusiveUnionObservationFromArrays(ULearningAgentsObservationObject* Object, const TArray<FName>& ElementNames, const TArray<FLearningAgentsObservationObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionObservation"));
	
	/**
	 * Make a new inclusive union observation.
	 *
	 * @param Object The Observation Object
	 * @param ElementNames The names of the chosen sub-observations.
	 * @param Elements The corresponding chosen sub-observations.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	static UE_API FLearningAgentsObservationObjectElement MakeInclusiveUnionObservationFromArrayViews(ULearningAgentsObservationObject* Object, const TArrayView<const FName> ElementNames, const TArrayView<const FLearningAgentsObservationObjectElement> Elements, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionObservation"));

	/**
	 * Make a new static array observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The sub-observations. The number of elements here must match what was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationObjectElement MakeStaticArrayObservation(ULearningAgentsObservationObject* Object, const TArray<FLearningAgentsObservationObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("StaticArrayObservation"));

	/**
	 * Make a new static array observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The sub-observations. The number of elements here must match what was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	static UE_API FLearningAgentsObservationObjectElement MakeStaticArrayObservationFromArrayView(ULearningAgentsObservationObject* Object, const TArrayView<const FLearningAgentsObservationObjectElement> Elements, const bool bActive = true, const FName Tag = TEXT("StaticArrayObservation"));

	/**
	 * Make a new set observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The sub-observations. The number of elements here must be less than or equal to the maximum that was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "Elements"))
	static UE_API FLearningAgentsObservationObjectElement MakeSetObservation(ULearningAgentsObservationObject* Object, const TSet<FLearningAgentsObservationObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("SetObservation"));

	/**
	 * Make a new set observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The sub-observations. The number of elements here must be less than or equal to the maximum that was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "Elements"))
	static UE_API FLearningAgentsObservationObjectElement MakeSetObservationFromArray(ULearningAgentsObservationObject* Object, const TArray<FLearningAgentsObservationObjectElement>& Elements, const bool bActive = true, const FName Tag = TEXT("SetObservation"));

	/**
	 * Make a new set observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The sub-observations. The number of elements here must be less than or equal to the maximum that was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	static UE_API FLearningAgentsObservationObjectElement MakeSetObservationFromArrayView(ULearningAgentsObservationObject* Object, const TArrayView<const FLearningAgentsObservationObjectElement> Elements, const bool bActive = true, const FName Tag = TEXT("SetObservation"));

	/**
	 * Make a new pair observation.
	 *
	 * @param Object The Observation Object
	 * @param Key The key sub-observation.
	 * @param Key The value sub-observation.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationObjectElement MakePairObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Key, const FLearningAgentsObservationObjectElement Value, const bool bActive = true, const FName Tag = TEXT("PairObservation"));

	/**
	 * Make a new array observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The sub-observations. The number of elements here must be less than or equal to the maximum that was given during Specify.
	 * @param MaxNum The maximum number of elements possible for this observation. Must match what was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "Elements"))
	static UE_API FLearningAgentsObservationObjectElement MakeArrayObservation(ULearningAgentsObservationObject* Object, const TArray<FLearningAgentsObservationObjectElement>& Elements, const int32 MaxNum, const bool bActive = true, const FName Tag = TEXT("ArrayObservation"));

	/**
	 * Make a new array observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The sub-observations. The number of elements here must be less than or equal to the maximum that was given during Specify.
	 * @param MaxNum The maximum number of elements possible for this observation. Must match what was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	static UE_API FLearningAgentsObservationObjectElement MakeArrayObservationFromArrayView(ULearningAgentsObservationObject* Object, const TArrayView<const FLearningAgentsObservationObjectElement> Elements, const int32 MaxNum, const bool bActive = true, const FName Tag = TEXT("ArrayObservation"));

	/**
	 * Make a new map observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The sub-observations. The number of elements here must be less than or equal to the maximum that was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "Map"))
	static UE_API FLearningAgentsObservationObjectElement MakeMapObservation(ULearningAgentsObservationObject* Object, const TMap<FLearningAgentsObservationObjectElement, FLearningAgentsObservationObjectElement>& Map, const bool bActive = true, const FName Tag = TEXT("MapObservation"));

	/**
	 * Make a new map observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The sub-observations. The number of elements here must be less than or equal to the maximum that was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "Keys,Values"))
	static UE_API FLearningAgentsObservationObjectElement MakeMapObservationFromArrays(ULearningAgentsObservationObject* Object, const TArray<FLearningAgentsObservationObjectElement>& Keys, const TArray<FLearningAgentsObservationObjectElement>& Values, const bool bActive = true, const FName Tag = TEXT("MapObservation"));
	
	/**
	 * Make a new map observation.
	 *
	 * @param Object The Observation Object
	 * @param Elements The sub-observations. The number of elements here must be less than or equal to the maximum that was given during Specify.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	static UE_API FLearningAgentsObservationObjectElement MakeMapObservationFromArrayViews(ULearningAgentsObservationObject* Object, const TArrayView<const FLearningAgentsObservationObjectElement> Keys, const TArrayView<const FLearningAgentsObservationObjectElement> Values, const bool bActive = true, const FName Tag = TEXT("MapObservation"));

	/**
	 * Make a new enum observation.
	 *
	 * @param Object The Observation Object
	 * @param Enum The enum type for this observation. Must match what was given during Specify.
	 * @param EnumValue The enum value.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeEnumObservation(
		ULearningAgentsObservationObject* Object, 
		const UEnum* Enum, 
		const uint8 EnumValue, 
		const bool bActive = true, 
		const FName Tag = TEXT("EnumObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new bitmask observation.
	 *
	 * @param Object The Observation Object
	 * @param Enum The enum type for this observation. Must match what was given during Specify.
	 * @param BitmaskValue The bitmask value.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeBitmaskObservation(
		ULearningAgentsObservationObject* Object, 
		const UEnum* Enum, 
		const int32 BitmaskValue, 
		const bool bActive = true, 
		const FName Tag = TEXT("BitmaskObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new optional observation.
	 *
	 * @param Object The Observation Object
	 * @param Element The sub-observation given.
	 * @param Option The indicator as to if this is observation should be used.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationObjectElement MakeOptionalObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const ELearningAgentsOptionalObservation Option, const bool bActive = true, const FName Tag = TEXT("OptionalObservation"));

	/**
	 * Make a new null optional observation. Use this to provide a null optional observation.
	 *
	 * @param Object The Observation Object
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1))
	static UE_API FLearningAgentsObservationObjectElement MakeOptionalNullObservation(ULearningAgentsObservationObject* Object, const bool bActive = true, const FName Tag = TEXT("OptionalObservation"));

	/**
	 * Make a new valid optional observation. Use this to provide a valid optional observation.
	 *
	 * @param Object The Observation Object
	 * @param Element The sub-observation given.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationObjectElement MakeOptionalValidObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("OptionalObservation"));

	/**
	 * Make a new either observation.
	 *
	 * @param Object The Observation Object
	 * @param Element The sub-observation given.
	 * @param Option The indicator as to if this is observation A or B.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3))
	static UE_API FLearningAgentsObservationObjectElement MakeEitherObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const ELearningAgentsEitherObservation Either, const bool bActive = true, const FName Tag = TEXT("EitherObservation"));

	/**
	 * Make a new either A observation. Use this to provide option A.
	 *
	 * @param Object The Observation Object
	 * @param Element The sub-observation given.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, DisplayName = "Make Either A Observation"))
	static UE_API FLearningAgentsObservationObjectElement MakeEitherAObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement A, const bool bActive = true, const FName Tag = TEXT("EitherObservation"));

	/**
	 * Make a new either B observation. Use this to provide option B.
	 *
	 * @param Object The Observation Object
	 * @param Element The sub-observation given.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, DisplayName = "Make Either B Observation"))
	static UE_API FLearningAgentsObservationObjectElement MakeEitherBObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement B, const bool bActive = true, const FName Tag = TEXT("EitherObservation"));

	/**
	 * Make a new encoding observation. This must be used in conjunction with SpecifyEncodingObservation.
	 *
	 * @param Object The Observation Object
	 * @param Element The Observation Element to be encoded.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2))
	static UE_API FLearningAgentsObservationObjectElement MakeEncodingObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("EncodingObservation"));

	/**
	 * Make a new bool observation.
	 *
	 * @param Object The Observation Object
	 * @param Value The new value of this observation.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeBoolObservation(
		ULearningAgentsObservationObject* Object,
		const bool bValue,
		const bool bActive = true, 
		const FName Tag = TEXT("BoolObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new 1d convolution observation.
	 *
	 * @param Object The Observation Object
	 * @param Element The Observation Element to be encoded.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeConv1dObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("Conv1dObservation"));

	/**
	 * Make a new 2d convolution observation.
	 *
	 * @param Object The Observation Object
	 * @param Element The Observation Element to be encoded.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeConv2dObservation(ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("Conv2dObservation"));

	/**
	 * Make a new float observation.
	 *
	 * @param Object The Observation Object
	 * @param Value The new value of this observation.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeFloatObservation(
		ULearningAgentsObservationObject* Object,
		const float Value,
		const bool bActive = true,
		const FName Tag = TEXT("FloatObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new location observation.
	 *
	 * @param Object The Observation Object
	 * @param Location The location of interest to the agent.
	 * @param RelativeTransform The transform the provided location should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeLocationObservation(
		ULearningAgentsObservationObject* Object,
		const FVector Location,
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true,
		const FName Tag = TEXT("LocationObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new rotation observation.
	 *
	 * @param Object The Observation Object
	 * @param Rotation The rotation of interest to the agent.
	 * @param RelativeRotation The rotation the provided rotation should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerRotationLocation A location for the visual logger to display the rotation in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeRotationObservation(
		ULearningAgentsObservationObject* Object,
		const FRotator Rotation,
		const FRotator RelativeRotation = FRotator::ZeroRotator,
		const bool bActive = true, 
		const FName Tag = TEXT("RotationObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new rotation observation from a quaternion.
	 *
	 * @param Object The Observation Object
	 * @param Rotation The rotation of interest to the agent.
	 * @param RelativeRotation The rotation the provided rotation should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerRotationLocation A location for the visual logger to display the rotation in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeRotationObservationFromQuat(
		ULearningAgentsObservationObject* Object,
		const FQuat Rotation,
		const FQuat RelativeRotation,
		const bool bActive = true, 
		const FName Tag = TEXT("RotationObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new scale observation.
	 *
	 * @param Object The Observation Object
	 * @param Scale The scale of interest to the agent.
	 * @param RelativeScale The scale the provided scale should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerScaleLocation A location for the visual logger to display the scale in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeScaleObservation(
		ULearningAgentsObservationObject* Object,
		const FVector Scale,
		const FVector RelativeScale = FVector(1, 1, 1),
		const bool bActive = true, 
		const FName Tag = TEXT("ScaleObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new transform observation.
	 *
	 * @param Object The Observation Object
	 * @param Transform The transform of interest to the agent.
	 * @param RelativeTransform The transform the provided transform should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeTransformObservation(
		ULearningAgentsObservationObject* Object,
		const FTransform Transform,
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true, 
		const FName Tag = TEXT("TransformObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new angle observation. Angles should be given in degrees.
	 *
	 * @param Object The Observation Object
	 * @param Angle The angle of interest to the agent.
	 * @param RelativeAngle The angle the provided angle should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeAngleObservation(
		ULearningAgentsObservationObject* Object,
		const float Angle,
		const float RelativeAngle = 0.0f,
		const bool bActive = true,
		const FName Tag = TEXT("AngleObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new angle observation. Angles should be given in radians.
	 *
	 * @param Object The Observation Object
	 * @param Angle The angle of interest to the agent.
	 * @param RelativeAngle The angle the provided angle should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeAngleObservationRadians(
		ULearningAgentsObservationObject* Object, 
		const float Angle, 
		const float RelativeAngle = 0.0f, 
		const bool bActive = true, 
		const FName Tag = TEXT("AngleObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new velocity observation.
	 *
	 * @param Object The Observation Object
	 * @param Velocity The velocity of interest to the agent.
	 * @param RelativeTransform The transform the provided velocity should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerVelocityLocation A location for the visual logger to display the velocity in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeVelocityObservation(
		ULearningAgentsObservationObject* Object,
		const FVector Velocity,
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true,
		const FName Tag = TEXT("VelocityObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new direction observation.
	 *
	 * @param Object The Observation Object
	 * @param Direction The direction of interest to the agent.
	 * @param RelativeTransform The transform the provided direction should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerDirectionLocation A location for the visual logger to display the direction in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerArrowLength The length of the arrow to display to represent the direction.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeDirectionObservation(
		ULearningAgentsObservationObject* Object,
		const FVector Direction,
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true,
		const FName Tag = TEXT("DirectionObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new location along spline observation.
	 *
	 * @param Object The Observation Object
	 * @param SplineComponent The spline to observe.
	 * @param DistanceAlongSpline The distance along that spline.
	 * @param RelativeTransform The transform the provided location should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeLocationAlongSplineObservation(
		ULearningAgentsObservationObject* Object,
		const USplineComponent* SplineComponent,
		const float DistanceAlongSpline,
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true,
		const FName Tag = TEXT("LocationAlongSplineObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new proportion along spline observation.
	 *
	 * @param Object The Observation Object
	 * @param SplineComponent The spline to observe.
	 * @param DistanceAlongSpline The distance along that spline.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeProportionAlongSplineObservation(
		ULearningAgentsObservationObject* Object, 
		const USplineComponent* SplineComponent, 
		const float DistanceAlongSpline, 
		const bool bActive = true, 
		const FName Tag = TEXT("ProportionAlongSplineObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new direction along spline observation.
	 *
	 * @param Object The Observation Object
	 * @param SplineComponent The spline to observe.
	 * @param DistanceAlongSpline The distance along that spline.
	 * @param RelativeTransform The transform the provided direction should be encoded relative to.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerArrowLength The length of the arrow to display to represent the direction.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeDirectionAlongSplineObservation(
		ULearningAgentsObservationObject* Object,
		const USplineComponent* SplineComponent,
		const float DistanceAlongSpline,
		const FTransform RelativeTransform = FTransform(),
		const bool bActive = true, 
		const FName Tag = TEXT("DirectionAlongSplineObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new proportion along ray observation.
	 *
	 * @param Object The Observation Object
	 * @param RayStart The local ray start location.
	 * @param RayEnd The local ray end location.
	 * @param RayTransform The transform to use to transform the local ray starts and ends into the world space.
	 * @param CollisionChannel The collision channel to collide against.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 5, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeProportionAlongRayObservation(
		ULearningAgentsObservationObject* Object, 
		const FVector RayStart, 
		const FVector RayEnd, 
		const FTransform RayTransform = FTransform(), 
		const ECollisionChannel CollisionChannel = ECollisionChannel::ECC_WorldStatic, 
		const bool bActive = true, 
		const FName Tag = TEXT("ProportionAlongRayObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

	/**
	 * Make a new LiDAR observation.
	 *
	 * @param Object The Observation Object
	 * @param RayStart The local ray start location.
	 * @param RayEnd The local ray end location.
	 * @param RayTransform The transform to use to transform the local ray starts and ends into the world space.
	 * @param CollisionChannel The collision channel to collide against.
	 * @param ActorToIgnore The actor to ignore in the raycast.
	 * @param bActive Whether the observation should be considered active. Must match the value given during Specify.
	 * @param Tag The tag of the corresponding observation. Must match the tag given during Specify.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this observation. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this observation.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The newly created observation object element.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 6, AutoCreateRefTerm = "VisualLoggerSettings"))
	static UE_API FLearningAgentsObservationObjectElement MakeLidarObservation(
		ULearningAgentsObservationObject* Object,
		const FVector RayStart,
		const FVector RayEnd,
		const FTransform RayTransform = FTransform(),
		const ECollisionChannel CollisionChannel = ECollisionChannel::ECC_WorldStatic,
		const AActor* ActorToIgnore = nullptr,
		const bool bActive = true, 
		const FName Tag = TEXT("LidarObservation"),
		const FLearningAgentsObservationVisualLoggerSettings& VisualLoggerSettings = FLearningAgentsObservationVisualLoggerSettings());

public:

	// Get Basic Observations

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 2, ReturnDisplayName = "Success"))
	static UE_API bool GetNullObservation(const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FName Tag = TEXT("NullObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetContinuousObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ContinuousObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetContinuousObservation(TArray<float>& OutValues, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ContinuousObservation"));
	static UE_API bool GetContinuousObservationToArrayView(TArrayView<float> OutValues, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ContinuousObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetExclusiveDiscreteObservation(int32& OutIndex, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ExclusiveDiscreteObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveDiscreteObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveDiscreteObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveDiscreteObservation(TArray<int32>& OutIndices, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveDiscreteObservation"));
	static UE_API bool GetInclusiveDiscreteObservationToArrayView(TArrayView<int32> OutIndices, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveDiscreteObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetNamedExclusiveDiscreteObservation(FName& OutName, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ExclusiveDiscreteObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetNamedInclusiveDiscreteObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveDiscreteObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetNamedInclusiveDiscreteObservation(TArray<FName>& OutNames, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveDiscreteObservation"));
	static UE_API bool GetNamedInclusiveDiscreteObservationToArrayView(TArrayView<FName> OutNames, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveDiscreteObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetCountObservation(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const int32 MaxNum, const bool bActive = true, const FName Tag = TEXT("CountObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStructObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StructObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStructObservation(TMap<FName, FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StructObservation"));
	
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetStructObservationElement(FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FName ElementName, const bool bActive = true, const FName Tag = TEXT("StructObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetStructObservationToArrays(TArray<FName>& OutElementNames, TArray<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StructObservation"));
	static UE_API bool GetStructObservationToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FLearningAgentsObservationObjectElement> OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StructObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetExclusiveUnionObservation(FName& OutElementName, FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ExclusiveUnionObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveUnionObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveUnionObservation(TMap<FName, FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetInclusiveUnionObservationToArrays(TArray<FName>& OutElementNames, TArray<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionObservation"));
	static UE_API bool GetInclusiveUnionObservationToArrayViews(TArrayView<FName> OutElementNames, TArrayView<FLearningAgentsObservationObjectElement> OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("InclusiveUnionObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStaticArrayObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StaticArrayObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetStaticArrayObservation(TArray<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StaticArrayObservation"));
	static UE_API bool GetStaticArrayObservationToArrayView(TArrayView<FLearningAgentsObservationObjectElement> OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("StaticArrayObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetSetObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("SetObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetSetObservation(TSet<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("SetObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetSetObservationToArray(TArray<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("SetObservation"));
	static UE_API bool GetSetObservationToArrayView(TArrayView<FLearningAgentsObservationObjectElement> OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("SetObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetPairObservation(FLearningAgentsObservationObjectElement& OutKey, FLearningAgentsObservationObjectElement& OutValue, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("PairObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetArrayObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ArrayObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetArrayObservation(TArray<FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const int32 MaxNum, const bool bActive = true, const FName Tag = TEXT("ArrayObservation"));
	static UE_API bool GetArrayObservationToArrayView(TArrayView<FLearningAgentsObservationObjectElement> OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const int32 MaxNum, const bool bActive = true, const FName Tag = TEXT("ArrayObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetMapObservationNum(int32& OutNum, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("MapObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetMapObservation(TMap<FLearningAgentsObservationObjectElement, FLearningAgentsObservationObjectElement>& OutElements, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("MapObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetMapObservationToArrays(TArray<FLearningAgentsObservationObjectElement>& OutKeys, TArray<FLearningAgentsObservationObjectElement>& OutValues, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("MapObservation"));
	static UE_API bool GetMapObservationToArrayViews(TArrayView<FLearningAgentsObservationObjectElement> OutKeys, TArrayView<FLearningAgentsObservationObjectElement> OutValues, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("MapObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetEnumObservation(uint8& OutEnumValue, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const UEnum* Enum, const bool bActive = true, const FName Tag = TEXT("EnumObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetBitmaskObservation(int32& OutBitmaskValue, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const UEnum* Enum, const bool bActive = true, const FName Tag = TEXT("BitmaskObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ExpandEnumAsExecs = "OutOption", ReturnDisplayName = "Success"))
	static UE_API bool GetOptionalObservation(ELearningAgentsOptionalObservation& OutOption, FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("OptionalObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ExpandEnumAsExecs = "OutEither", ReturnDisplayName = "Success"))
	static UE_API bool GetEitherObservation(ELearningAgentsEitherObservation& OutEither, FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("EitherObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetEncodingObservation(FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("EncodingObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetBoolObservation(bool& bOutValue, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("BoolObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetFloatObservation(float& OutValue, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("FloatObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetLocationObservation(FVector& OutLocation, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform = FTransform(), const bool bActive = true, const FName Tag = TEXT("LocationObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetRotationObservation(FRotator& OutRotation, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FRotator RelativeRotation = FRotator::ZeroRotator, const bool bActive = true, const FName Tag = TEXT("RotationObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetRotationObservationAsQuat(FQuat& OutRotation, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FQuat RelativeRotation, const bool bActive = true, const FName Tag = TEXT("RotationObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetScaleObservation(FVector& OutScale, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FVector RelativeScale = FVector(1, 1, 1), const bool bActive = true, const FName Tag = TEXT("ScaleObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetTransformObservation(FTransform& OutTransform, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform = FTransform(), const bool bActive = true, const FName Tag = TEXT("TransformObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetAngleObservation(float& OutAngle, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const float RelativeAngle = 0.0f, const bool bActive = true, const FName Tag = TEXT("AngleObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetAngleObservationRadians(float& OutAngle, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const float RelativeAngle = 0.0f, const bool bActive = true, const FName Tag = TEXT("AngleObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetVelocityObservation(FVector& OutVelocity, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform = FTransform(), const bool bActive = true, const FName Tag = TEXT("VelocityObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetDirectionObservation(FVector& OutDirection, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform = FTransform(), const bool bActive = true, const FName Tag = TEXT("DirectionObservation"));

	// Get Spline Observations

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetLocationAlongSplineObservation(FVector& OutLocation, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform = FTransform(), const bool bActive = true, const FName Tag = TEXT("LocationAlongSplineObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 5, ReturnDisplayName = "Success"))
	static UE_API bool GetProportionAlongSplineObservation(bool& bOutIsClosedLoop, float& OutAngle, float& OutPropotion, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ProportionAlongSplineObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 4, ReturnDisplayName = "Success"))
	static UE_API bool GetDirectionAlongSplineObservation(FVector& OutDirection, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const FTransform RelativeTransform = FTransform(), const bool bActive = true, const FName Tag = TEXT("DirectionAlongSplineObservation"));

	// Get Ray Cast Observations

	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetProportionAlongRayObservation(float& OutProportion, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("ProportionAlongRayObservation"));

	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetConv1dObservation(FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("Conv1dObservation"));
	
	UFUNCTION(BlueprintPure = false, Category = "LearningAgents", meta = (AdvancedDisplay = 3, ReturnDisplayName = "Success"))
	static UE_API bool GetConv2dObservation(FLearningAgentsObservationObjectElement& OutElement, const ULearningAgentsObservationObject* Object, const FLearningAgentsObservationObjectElement Element, const bool bActive = true, const FName Tag = TEXT("Conv2dObservation"));
};


#undef UE_API
