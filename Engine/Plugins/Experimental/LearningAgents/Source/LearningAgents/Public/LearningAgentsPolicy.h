// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningAgentsManagerListener.h"

#include "LearningAgentsNeuralNetwork.h" // Included for ELearningAgentsActivationFunction
#include "LearningArray.h"
#include "UObject/ObjectPtr.h"

#include "LearningAgentsPolicy.generated.h"

#define UE_API LEARNINGAGENTS_API

namespace UE::Learning
{
	struct FNeuralNetworkPolicy;
	struct FNeuralNetworkFunction;
}

class ULearningAgentsNeuralNetwork;
class FJsonObject;

UENUM(BlueprintType)
enum class ELearningAgentsMemoryCell : uint8
{
	NoMemoryCell UMETA(DisplayName = "No Memory Cell"),
	LearningAgentsGRU UMETA(DisplayName = "Learning Agents GRU"),
	StandardGRU UMETA(DisplayName = "Standard GRU")
};

/** The configurable settings for a ULearningAgentsPolicy. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsPolicySettings
{
	GENERATED_BODY()

public:

	/** Number of hidden layers for policy network. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0"))
	int32 HiddenLayerNum = 1;

	/** Number of neurons in each hidden layer of the policy network */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 HiddenLayerSize = 128;

	/** The type of memory cell to use. Setting this to NoMemoryCell is equivalent to setting MemoryStateSize to 0. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	ELearningAgentsMemoryCell MemoryCell = ELearningAgentsMemoryCell::LearningAgentsGRU;

	/** Number of neurons in the memory state of the policy network */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0"))
	int32 MemoryStateSize = 64;

	/** Initial scale for encoded actions. Setting this <1 reduces the initial bias from the network random weights and tends to help training. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "1"))
	float InitialEncodedActionScale = 0.1f;

	/** Activation function to use on hidden layers of the policy network */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	ELearningAgentsActivationFunction ActivationFunction = ELearningAgentsActivationFunction::ELU;

	/** If true, policy can be evaluated on multiple threads to speed up large agent counts. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bUseParallelEvaluation = true;
};

/** A policy that maps from observations to actions. */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, meta = (BlueprintSpawnableComponent))
class ULearningAgentsPolicy : public ULearningAgentsManagerListener
{
	GENERATED_BODY()

public:

	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsPolicy();
	UE_API ULearningAgentsPolicy(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsPolicy();

	/**
	 * Constructs this object to be used with the given agent interactor and policy settings.
	 * 
	 * @param InManager						The input Manager
	 * @param InInteractor					The input Interactor component
	 * @param Class							The policy class
	 * @param Name							The policy name
	 * @param EncoderNeuralNetworkAsset		Optional Encoder Network Asset to use. If not provided, asset is empty, or
	 *										bReinitializeEncoderNetwork is set then a new neural network object will be
	 *										created.
	 * @param PolicyNeuralNetworkAsset		Optional Policy Network Asset to use. If not provided, asset is empty, or
	 *										bReinitializePolicyNetwork is set then a new neural network object will be 
	 *										created according to the given PolicySettings.
	 * @param DecoderNeuralNetworkAsset		Optional Decoder Network Asset to use. If not provided, asset is empty, or
	 *										bReinitializeDecoderNetwork is set then a new neural network object will be
	 *										created.
	 * @param bReinitializeEncoderNetwork	If to reinitialize the encoder network
	 * @param bReinitializePolicyNetwork	If to reinitialize the policy network
	 * @param bReinitializeDecoderNetwork	If to reinitialize the decoder network
	 * @param PolicySettings				The policy settings to use on creation of a new policy network
	 * @param Seed							Random seed to use for initializing network weights and policy sampling
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta=(Class = "/Script/LearningAgents.LearningAgentsPolicy", DeterminesOutputType = "Class", AutoCreateRefTerm = "PolicySettings"))
	static UE_API ULearningAgentsPolicy* MakePolicy(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		TSubclassOf<ULearningAgentsPolicy> Class,
		const FName Name = TEXT("Policy"),
		ULearningAgentsNeuralNetwork* EncoderNeuralNetworkAsset = nullptr,
		ULearningAgentsNeuralNetwork* PolicyNeuralNetworkAsset = nullptr,
		ULearningAgentsNeuralNetwork* DecoderNeuralNetworkAsset = nullptr,
		const bool bReinitializeEncoderNetwork = true,
		const bool bReinitializePolicyNetwork = true,
		const bool bReinitializeDecoderNetwork = true,
		const FLearningAgentsPolicySettings& PolicySettings = FLearningAgentsPolicySettings(),
		const int32 Seed = 1234);

	/**
	 * Initializes this object to be used with the given agent interactor and policy settings.
	 * 
	 * @param InManager						The input Manager
	 * @param InInteractor					The input Interactor component
	 * @param EncoderNeuralNetworkAsset		Optional Encoder Network Asset to use. If not provided, asset is empty, or
	 *										bReinitializeEncoderNetwork is set then a new neural network object will be
	 *										created.
	 * @param PolicyNeuralNetworkAsset		Optional Policy Network Asset to use. If not provided, asset is empty, or
	 *										bReinitializePolicyNetwork is set then a new neural network object will be
	 *										created according to the given PolicySettings.
	 * @param DecoderNeuralNetworkAsset		Optional Decoder Network Asset to use. If not provided, asset is empty, or
	 *										bReinitializeDecoderNetwork is set then a new neural network object will be
	 *										created.
	 * @param bReinitializeEncoderNetwork	If to reinitialize the encoder network
	 * @param bReinitializePolicyNetwork	If to reinitialize the policy network
	 * @param bReinitializeDecoderNetwork	If to reinitialize the decoder network
	 * @param PolicySettings				The policy settings to use on creation of a new policy network
	 * @param Seed							Random seed to use for initializing network weights and policy sampling
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "PolicySettings"))
	UE_API void SetupPolicy(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		ULearningAgentsNeuralNetwork* EncoderNeuralNetworkAsset = nullptr,
		ULearningAgentsNeuralNetwork* PolicyNeuralNetworkAsset = nullptr,
		ULearningAgentsNeuralNetwork* DecoderNeuralNetworkAsset = nullptr,
		const bool bReinitializeEncoderNetwork = true,
		const bool bReinitializePolicyNetwork = true,
		const bool bReinitializeDecoderNetwork = true,
		const FLearningAgentsPolicySettings& InPolicySettings = FLearningAgentsPolicySettings(),
		const int32 Seed = 1234);

	UE_API TSharedRef<FJsonObject> AsJsonConfig() const;

public:

	//~ Begin ULearningAgentsManagerListener Interface
	UE_API virtual void OnAgentsAdded_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsReset_Implementation(const TArray<int32>& AgentIds) override;
	//~ End ULearningAgentsManagerListener Interface


// ----- Blueprint public interface -----
public:

	/**
	 * Encodes the buffered observation vectors using the Encoder network. This should be called after GatherObservations and before EvaluatePolicy.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void EncodeObservations();

	/**
	 * Calling this function will run the underlying neural network on the previously encoded observations to populate
	 * the encoded actions. This should be called after EncodeObservations and before DecodeAndSampleActions.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void EvaluatePolicy();

	/**
	 * Decodes and samples action vectors using the Decoder network. This should be called after both MakeActionModifiers and EvaluatePolicy, but 
	 * before PerformActions.
	 * 
	 * @param ActionNoiseScale		Scale of the action noise to use during sampling. Set this to zero to always sample the mean (expected) action.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void DecodeAndSampleActions(const float ActionNoiseScale = 1.0f);

	/**
	 * Calls GatherObservations, EncodeObservations, EvaluatePolicy, DecodeAndSampleActions, PerformActions
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void RunInference(const float ActionNoiseScale = 1.0f);

	/**
	 * Gets the current memory state for a given agent as represented by an abstract vector learned by the policy.
	 *
	 * @param OutMemoryState	The output memory state of the agent
	 * @param AgentId			The AgentId to get the memory state of
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", Meta = (AgentId = -1))
	UE_API void GetMemoryState(TArray<float>& OutMemoryState, const int32 AgentId) const;

	/**
	 * Sets the current memory state for a given agent as represented by an abstract vector learned by the policy.
	 *
	 * @param AgentId			The AgentId to set the memory state of
	 * @param InMemoryState		The input memory state of the agent
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", Meta = (AgentId = -1))
	UE_API void SetMemoryState(const int32 AgentId, const TArray<float>& InMemoryState);

	/** Gets the size of the memory state */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetMemoryStateSize() const;

	/** Gets the current Encoder Network Asset being used */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API ULearningAgentsNeuralNetwork* GetEncoderNetworkAsset();

	/** Gets the current Policy Network Asset being used */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API ULearningAgentsNeuralNetwork* GetPolicyNetworkAsset();

	/** Gets the current Encoder Network Asset being used */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API ULearningAgentsNeuralNetwork* GetDecoderNetworkAsset();


// ----- Non-blueprint public interface -----
public:

	/** Get a reference to this policy's encoder function object. */
	UE_API UE::Learning::FNeuralNetworkFunction& GetEncoderObject();

	/** Get a reference to this policy's policy function object. */
	UE_API UE::Learning::FNeuralNetworkPolicy& GetPolicyObject();

	/** Get a reference to this policy's decoder function object. */
	UE_API UE::Learning::FNeuralNetworkFunction& GetDecoderObject();

	/** Get a const view to this policy's pre-evaluation memory state. */
	UE_API TLearningArrayView<2, const float> GetPreEvaluationMemoryState() const;

	/** Get a const view to this policy's memory state. */
	UE_API TLearningArrayView<2, const float> GetMemoryState() const;

	/** Get a const view to this policy's encoded observation vectors. */
	UE_API TLearningArrayView<2, const float> GetObservationVectorsEncoded() const;

	/** Returns true if this policy has encoded observations for the given agent id. Otherwise, false. */
	UE_API bool HasEncodedObservationsForAgent(const int32 AgentId) const;

// ----- Private Data -----
private:

	/** The agent interactor this policy is associated with. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsInteractor> Interactor;

	/** The underlying encoder neural network. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsNeuralNetwork> EncoderNetwork;

	/** The underlying policy neural network. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsNeuralNetwork> PolicyNetwork;

	/** The underlying decoder neural network. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsNeuralNetwork> DecoderNetwork;

	/** The internal memory state of each agent before evaluation. */
	TLearningArray<2, float> PreEvaluationMemoryState;

	/** The internal memory state of each agent. */
	TLearningArray<2, float> MemoryState;

	/** The global random seed state used to initialize network weights and during sampling */
	uint32 GlobalSeed = 0;

	/** The individual per-agent seeds used during sampling of actions. */
	TLearningArray<1, uint32> Seeds;

	/** Encoded observation vector buffers */
	TLearningArray<2, float> ObservationVectorsEncoded;

	/** Encoded action vector buffers */
	TLearningArray<2, float> ActionVectorsEncoded;

	/** Action distribution vector buffers */
	TLearningArray<2, float> ActionDistributionVectors;

	/** Internal Encoder Function Object */
	TSharedPtr<UE::Learning::FNeuralNetworkFunction> EncoderObject;

	/** Internal Policy Function Object */
	TSharedPtr<UE::Learning::FNeuralNetworkPolicy> PolicyObject;

	/** Internal Decoder Function Object */
	TSharedPtr<UE::Learning::FNeuralNetworkFunction> DecoderObject;

	/** Number of times encoded observation vector has been set for all agents */
	TLearningArray<1, uint64, TInlineAllocator<32>> ObservationVectorEncodedIteration;

	/** Number of times encoded action vector has been set for all agents */
	TLearningArray<1, uint64, TInlineAllocator<32>> ActionVectorEncodedIteration;

	/** Number of times the memory state has been updated for all agents. */
	TLearningArray<1, uint64, TInlineAllocator<32>> MemoryStateIteration;

	/** Temp buffers used to record the set of agents that are valid for evaluation */
	TArray<int32> ValidAgentIds;
	UE::Learning::FIndexSet ValidAgentSet;

	FLearningAgentsPolicySettings PolicySettings;

};

#undef UE_API
