// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncAction_LearningAgentsQueryReplays.h"

#include "GameFramework/PlayerController.h"
#include "LearningAgentsReplaySubsystem.h"
#include "Templates/Greater.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncAction_LearningAgentsQueryReplays)

UAsyncAction_LearningAgentsQueryReplays::UAsyncAction_LearningAgentsQueryReplays(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAsyncAction_LearningAgentsQueryReplays* UAsyncAction_LearningAgentsQueryReplays::QueryLearningAgentsReplays(APlayerController* InPlayerController)
{
	UAsyncAction_LearningAgentsQueryReplays* Action = nullptr;

	if (InPlayerController != nullptr)
	{
		Action = NewObject<UAsyncAction_LearningAgentsQueryReplays>();
		Action->PlayerController = InPlayerController;
	}

	return Action;
}

void UAsyncAction_LearningAgentsQueryReplays::Activate()
{
	ReplayStreamer = FNetworkReplayStreaming::Get().GetFactory().CreateReplayStreamer();

	ResultList = NewObject<ULearningAgentsReplayList>();
	if (ReplayStreamer.IsValid())
	{
		FNetworkReplayVersion EnumerateStreamsVersion = FNetworkVersion::GetReplayVersion();

		ReplayStreamer->EnumerateStreams(EnumerateStreamsVersion, INDEX_NONE, FString(), TArray<FString>(), FEnumerateStreamsCallback::CreateUObject(this, &ThisClass::OnEnumerateStreamsComplete));
	}
	else
	{
		QueryComplete.Broadcast(ResultList);
	}
}

void UAsyncAction_LearningAgentsQueryReplays::OnEnumerateStreamsComplete(const FEnumerateStreamsResult& Result)
{
	for (const FNetworkReplayStreamInfo& StreamInfo : Result.FoundStreams)
	{
		ULearningAgentsReplayListEntry* NewReplayEntry = NewObject<ULearningAgentsReplayListEntry>(ResultList);
		NewReplayEntry->StreamInfo = StreamInfo;
		ResultList->Results.Add(NewReplayEntry);
	}

	// Sort demo names by date
	Algo::SortBy(ResultList->Results, [](const TObjectPtr<ULearningAgentsReplayListEntry>& Data) { return Data->StreamInfo.Timestamp.GetTicks(); }, TGreater<>());

	QueryComplete.Broadcast(ResultList);
}

