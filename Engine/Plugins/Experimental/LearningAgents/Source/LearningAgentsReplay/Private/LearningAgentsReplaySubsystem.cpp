// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsReplaySubsystem.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "Engine/DemoNetDriver.h"
#include "Internationalization/Text.h"
#include "Misc/DateTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsReplaySubsystem)

ULearningAgentsReplaySubsystem::ULearningAgentsReplaySubsystem()
{
}

bool ULearningAgentsReplaySubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return true;
}

void ULearningAgentsReplaySubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	
	FCoreUObjectDelegates::PostDemoPlay.AddUObject(this, &ThisClass::OnDemoPlayStarted);
}

void ULearningAgentsReplaySubsystem::Deinitialize()
{
	Super::Deinitialize();

	FCoreUObjectDelegates::PostDemoPlay.RemoveAll(this);
}

void ULearningAgentsReplaySubsystem::OnDemoPlayStarted()
{
	
}

bool ULearningAgentsReplaySubsystem::DoesPlatformSupportReplays()
{
	return true;
}

void ULearningAgentsReplaySubsystem::PlayReplay(ULearningAgentsReplayListEntry* Replay)
{
	if (Replay != nullptr)
	{
		FString DemoName = Replay->StreamInfo.Name;
		GetGameInstance()->PlayReplay(DemoName);
	}
}

void ULearningAgentsReplaySubsystem::StopRecordingReplay()
{
	if (UGameInstance* GameInstance = GetGameInstance())
	{
		GameInstance->StopRecordingReplay();	
	}
}

void ULearningAgentsReplaySubsystem::RecordClientReplay(APlayerController* PlayerController)
{
	UGameInstance* GameInstance = GetGameInstance();
	if (ensure(DoesPlatformSupportReplays() && PlayerController && GameInstance))
	{
		FText FriendlyNameText = FText::Format(NSLOCTEXT("LearningAgents", "LearningAgentsReplayName_Format", "Client Replay {0}"), FText::AsDateTime(FDateTime::UtcNow(), EDateTimeStyle::Short, EDateTimeStyle::Short));
		GameInstance->StartRecordingReplay(FString(), FriendlyNameText.ToString());
	}
}

void ULearningAgentsReplaySubsystem::SeekInActiveReplay(float TimeInSeconds)
{
	if (UDemoNetDriver* DemoDriver = GetDemoDriver())
	{
		DemoDriver->GotoTimeInSeconds(TimeInSeconds);
	}
}

float ULearningAgentsReplaySubsystem::GetReplayLengthInSeconds() const
{
	if (UDemoNetDriver* DemoDriver = GetDemoDriver())
	{
		return DemoDriver->GetDemoTotalTime();
	}
	return 0.0f;
}

float ULearningAgentsReplaySubsystem::GetReplayCurrentTime() const
{
	if (UDemoNetDriver* DemoDriver = GetDemoDriver())
	{
		return DemoDriver->GetDemoCurrentTime();
	}
	return 0.0f;
}

UDemoNetDriver* ULearningAgentsReplaySubsystem::GetDemoDriver() const
{
	UGameInstance* GameInstance = GetGameInstance();
	if(GameInstance == nullptr)
	{
		return nullptr;
	}
	
	if (UWorld* World = GameInstance->GetWorld())
	{
		return World->GetDemoNetDriver();
	}
	return nullptr;
}



