// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintAsyncActionBase.h"

#include "UObject/ObjectPtr.h"
#include "AsyncAction_LearningAgentsQueryReplays.generated.h"

class APlayerController;
class INetworkReplayStreamer;
class ULearningAgentsReplayList;
class UObject;
struct FEnumerateStreamsResult;
struct FFrame;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FQueryReplayAsyncDelegate, ULearningAgentsReplayList*, Results);

/**
 * Returns a list of stored replays for a given player controller 
 */
UCLASS()
class UAsyncAction_LearningAgentsQueryReplays : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()

public:
	UAsyncAction_LearningAgentsQueryReplays(const FObjectInitializer& ObjectInitializer);

	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static UAsyncAction_LearningAgentsQueryReplays* QueryLearningAgentsReplays(APlayerController* PlayerController);

	virtual void Activate() override;

public:
	// Called when the replay query completes
	UPROPERTY(BlueprintAssignable)
	FQueryReplayAsyncDelegate QueryComplete;

private:
	void OnEnumerateStreamsComplete(const FEnumerateStreamsResult& Result);

private:
	UPROPERTY()
	TObjectPtr<ULearningAgentsReplayList> ResultList;

	TWeakObjectPtr<APlayerController> PlayerController;

	TSharedPtr<INetworkReplayStreamer> ReplayStreamer;
};
