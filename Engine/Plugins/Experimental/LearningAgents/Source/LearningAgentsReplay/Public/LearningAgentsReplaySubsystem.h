// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkReplayStreaming.h"
#include "Subsystems/GameInstanceSubsystem.h"
#include "LearningAgentsReplaySubsystem.generated.h"

#define UE_API LEARNINGAGENTSREPLAY_API

class UDemoNetDriver;
class APlayerController;
class ULocalPlayer;

/** An available replay for display in the UI */
UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsReplayListEntry : public UObject
{
	GENERATED_BODY()

public:
	FNetworkReplayStreamInfo StreamInfo;

	/** The UI friendly name of the stream */
	UFUNCTION(BlueprintPure, Category=Replays)
	FString GetFriendlyName() const { return StreamInfo.FriendlyName; }

	/** The date and time the stream was recorded */
	UFUNCTION(BlueprintPure, Category=Replays)
	FDateTime GetTimestamp() const { return StreamInfo.Timestamp; }

	/** The duration of the stream in MS */
	UFUNCTION(BlueprintPure, Category=Replays)
	FTimespan GetDuration() const { return FTimespan::FromMilliseconds(StreamInfo.LengthInMS); }

	/** Number of viewers viewing this stream */
	UFUNCTION(BlueprintPure, Category=Replays)
	int32 GetNumViewers() const { return StreamInfo.NumViewers; }

	/** True if the stream is live and the game hasn't completed yet */
	UFUNCTION(BlueprintPure, Category=Replays)
	bool GetIsLive() const { return StreamInfo.bIsLive; }
};

/** Results of querying for replays list of results for the UI */
UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsReplayList : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category=Replays)
	TArray<TObjectPtr<ULearningAgentsReplayListEntry>> Results;
};

/** Subsystem to handle recording/loading replays. Exposes functionality to Blueprints */
UCLASS(MinimalAPI)
class ULearningAgentsReplaySubsystem : public UGameInstanceSubsystem
{
	GENERATED_BODY()

public:
	UE_API ULearningAgentsReplaySubsystem();

	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	UFUNCTION()
	UE_API void OnDemoPlayStarted();

	/** Returns true if this platform supports replays at all */
	UFUNCTION(BlueprintCallable, Category = Replays, BlueprintPure = false)
	static UE_API bool DoesPlatformSupportReplays();

	/** Loads the appropriate map and plays a replay */
	UFUNCTION(BlueprintCallable, Category=Replays)
	UE_API void PlayReplay(ULearningAgentsReplayListEntry* Replay);
	
	/** Stops the current active recording */
	UFUNCTION(BlueprintCallable, Category=Replays)
	UE_API void StopRecordingReplay();

	/** Starts recording a client replay, and handles any file cleanup needed */
	UFUNCTION(BlueprintCallable, Category = Replays)
	UE_API void RecordClientReplay(APlayerController* PlayerController);

	/** Move forward or back in currently playing replay */
	UFUNCTION(BlueprintCallable, Category=Replays)
	UE_API void SeekInActiveReplay(float TimeInSeconds);

	/** Gets length of current replay */
	UFUNCTION(BlueprintCallable, Category = Replays, BlueprintPure = false)
	UE_API float GetReplayLengthInSeconds() const;

	/** Gets current playback time */
	UFUNCTION(BlueprintCallable, Category=Replays, BlueprintPure=false)
	UE_API float GetReplayCurrentTime() const;

private:
	TSharedPtr<INetworkReplayStreamer> CurrentReplayStreamer;

	UPROPERTY()
	TObjectPtr<ULocalPlayer> LocalPlayerDeletingReplays;

	UE_API UDemoNetDriver* GetDemoDriver() const;
};

#undef UE_API
