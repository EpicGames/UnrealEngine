// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LearningAgentsTraining : ModuleRules
{
	public LearningAgentsTraining(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AIModule",
				"Core",
				"GameplayTags",
				"NavigationSystem",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Json",
				"JsonUtilities",
				"Learning",
				"LearningAgents",
				"LearningTraining",
				"Projects",
			});

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
