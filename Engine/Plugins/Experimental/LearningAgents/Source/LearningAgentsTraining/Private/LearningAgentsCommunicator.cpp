// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsCommunicator.h"

#include "LearningAgentsInteractor.h"
#include "LearningExternalTrainer.h"

#include "Misc/CommandLine.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsCommunicator)

FLearningAgentsCommunicator ULearningAgentsCommunicatorLibrary::MakeSharedMemoryTrainingProcess(
	const FLearningAgentsTrainerProcessSettings& TrainerProcessSettings,
	const FLearningAgentsSharedMemoryCommunicatorSettings& SharedMemorySettings)
{
	FLearningAgentsSharedMemoryTrainerProcess TrainerProcess = SpawnSharedMemoryTrainingProcess(TrainerProcessSettings, SharedMemorySettings);
	return MakeSharedMemoryCommunicator(TrainerProcess, TrainerProcessSettings, SharedMemorySettings);
}

FLearningAgentsSharedMemoryTrainerProcess ULearningAgentsCommunicatorLibrary::SpawnSharedMemoryTrainingProcess(
	const FLearningAgentsTrainerProcessSettings& TrainerProcessSettings,
	const FLearningAgentsSharedMemoryCommunicatorSettings& SharedMemorySettings)
{
	FLearningAgentsSharedMemoryTrainerProcess TrainerProcess;

	if (FParse::Param(FCommandLine::Get(), TEXT("LearningAgentsUseExternalTrainingProcess")) || SharedMemorySettings.bUseExternalTrainingProcess)
	{
		UE_LOGF(LogLearning, Display, "SpawnSharedMemoryTrainingProcess: Skipping spawning because bUseExternalTrainingProcess is True.");
		return TrainerProcess;
	}

	const FString PythonExecutablePath = UE::Learning::Trainer::GetPythonExecutablePath(TrainerProcessSettings.GetIntermediatePath());
	if (!FPaths::FileExists(PythonExecutablePath))
	{
		UE_LOGF(LogLearning, Error, "SpawnSharedMemoryTrainingProcess: Can't find Python executable \"%ls\".", *PythonExecutablePath);
		return TrainerProcess;
	}

	const FString PythonContentPath = UE::Learning::Trainer::GetPythonContentPath(TrainerProcessSettings.GetEditorEnginePath());
	if (!FPaths::DirectoryExists(PythonContentPath))
	{
		UE_LOGF(LogLearning, Error, "SpawnSharedMemoryTrainingProcess: Can't find LearningAgents plugin Content \"%ls\".", *PythonContentPath);
		return TrainerProcess;
	}

	const FString IntermediatePath = UE::Learning::Trainer::GetIntermediatePath(TrainerProcessSettings.GetIntermediatePath());

	const FString CustomTrainerModulePath = TrainerProcessSettings.GetCustomTrainerModulePath();
	if (!CustomTrainerModulePath.IsEmpty() and !FPaths::DirectoryExists(CustomTrainerModulePath))
	{
		UE_LOGF(LogLearning, Error, "SpawnSharedMemoryTrainingProcess: Can't find custom trainer module \"%ls\".", *CustomTrainerModulePath);
		return TrainerProcess;
	}

	const FString NneCpuContentPath = TrainerProcessSettings.GetNneCpuContentPath();

	TrainerProcess.TrainerProcess = MakeShared<UE::Learning::FSharedMemoryTrainerServerProcess>(
		TrainerProcessSettings.TaskName,
		CustomTrainerModulePath,
		TrainerProcessSettings.TrainerFileName,
		PythonExecutablePath,
		PythonContentPath,
		IntermediatePath,
		NneCpuContentPath,
		SharedMemorySettings.Timeout);

	return TrainerProcess;
}

FLearningAgentsCommunicator ULearningAgentsCommunicatorLibrary::MakeSharedMemoryCommunicator(
	FLearningAgentsSharedMemoryTrainerProcess TrainerProcess,
	const FLearningAgentsTrainerProcessSettings& TrainerProcessSettings,
	const FLearningAgentsSharedMemoryCommunicatorSettings& SharedMemorySettings)
{
	FLearningAgentsCommunicator Communicator;

	// Task Name
	FString CmdLineTaskName;
	bool bTaskNameOverridden = FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsTaskName="), CmdLineTaskName);
	if (bTaskNameOverridden)
	{
		UE_LOGF(LogLearning, Display, "Overriding Task Name from the cmdline: %ls", *CmdLineTaskName);
	}

	// Timeout
	float CmdLineTimeout;
	bool bTimeoutOverridden = FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsTimeout="), CmdLineTimeout);
	if (bTimeoutOverridden)
	{
		UE_LOGF(LogLearning, Display, "Overriding Timeout with value from the cmdline: %f", CmdLineTimeout);
	}

	// Controls Guid
	// We have to use FGuid::Parse to accept more guid formats
	FString CmdLineControlsGuidString;
	FGuid CmdLineControlsGuid;
	if (FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsControlsGuid="), CmdLineControlsGuidString) && FGuid::Parse(CmdLineControlsGuidString, CmdLineControlsGuid))
	{
		UE_LOGF(LogLearning, Display, "Overriding Controls Guid with value from the cmdline: %ls", *CmdLineControlsGuid.ToString(PLATFORM_MAC ? EGuidFormats::Short : EGuidFormats::DigitsWithHyphensInBraces));
	}

	// Config Path
	FString CmdLineConfigPath;
	bool bConfigPathOverridden = FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsConfigPath="), CmdLineConfigPath);
	if (bConfigPathOverridden)
	{
		UE_LOGF(LogLearning, Display, "Overriding Config Path with value from the cmdline: %ls", *CmdLineConfigPath);
	}

	Communicator.Trainer = MakeShared<UE::Learning::FSharedMemoryTrainer>(
		bTaskNameOverridden ? CmdLineTaskName : TrainerProcessSettings.TaskName,
		TrainerProcess.TrainerProcess,
		FParse::Param(FCommandLine::Get(), TEXT("LearningAgentsUseExternalTrainingProcess")) || SharedMemorySettings.bUseExternalTrainingProcess,
		CmdLineControlsGuid.IsValid() ? CmdLineControlsGuid : SharedMemorySettings.ControlsGuid,
		bConfigPathOverridden ? CmdLineConfigPath : SharedMemorySettings.ConfigPath.Path,
		UE::Learning::Trainer::GetIntermediatePath(TrainerProcessSettings.GetIntermediatePath()),
		bTimeoutOverridden ? CmdLineTimeout : SharedMemorySettings.Timeout);

	return Communicator;
}

FLearningAgentsCommunicator ULearningAgentsCommunicatorLibrary::MakeSocketTrainingProcess(
	const FLearningAgentsTrainerProcessSettings& TrainerProcessSettings,
	const FLearningAgentsSocketCommunicatorSettings& SocketSettings)
{
	FLearningAgentsSocketTrainerProcess TrainerProcess = SpawnSocketTrainingProcess(TrainerProcessSettings, SocketSettings);
	return MakeSocketCommunicator(TrainerProcess, SocketSettings);
}

FLearningAgentsSocketTrainerProcess ULearningAgentsCommunicatorLibrary::SpawnSocketTrainingProcess(
	const FLearningAgentsTrainerProcessSettings& TrainerProcessSettings,
	const FLearningAgentsSocketCommunicatorSettings& SocketSettings)
{
	FLearningAgentsSocketTrainerProcess TrainerProcess;

	if (FParse::Param(FCommandLine::Get(), TEXT("LearningAgentsUseExternalTrainingProcess")) || SocketSettings.bUseExternalTrainingProcess)
	{
		UE_LOGF(LogLearning, Display, "SpawnSocketTrainingProcess: Skipping spawning because bUseExternalTrainingProcess is True.");
		return TrainerProcess;
	}

	const FString PythonExecutablePath = UE::Learning::Trainer::GetPythonExecutablePath(TrainerProcessSettings.GetIntermediatePath());
	if (!FPaths::FileExists(PythonExecutablePath))
	{
		UE_LOGF(LogLearning, Error, "SpawnSocketTrainingProcess: Can't find Python executable \"%ls\".", *PythonExecutablePath);
		return TrainerProcess;
	}

	const FString PythonContentPath = UE::Learning::Trainer::GetPythonContentPath(TrainerProcessSettings.GetEditorEnginePath());
	if (!FPaths::DirectoryExists(PythonContentPath))
	{
		UE_LOGF(LogLearning, Error, "SpawnSocketTrainingProcess: Can't find LearningAgents plugin Content \"%ls\".", *PythonContentPath);
		return TrainerProcess;
	}

	const FString IntermediatePath = UE::Learning::Trainer::GetIntermediatePath(TrainerProcessSettings.GetIntermediatePath());

	const FString CustomTrainerModulePath = TrainerProcessSettings.GetCustomTrainerModulePath();
	if (!CustomTrainerModulePath.IsEmpty() and !FPaths::DirectoryExists(CustomTrainerModulePath))
	{
		UE_LOGF(LogLearning, Error, "SpawnSocketTrainingProcess: Can't find custom trainer module \"%ls\".", *CustomTrainerModulePath);
		return TrainerProcess;
	}

	const FString NneCpuContentPath = TrainerProcessSettings.GetNneCpuContentPath();

	TrainerProcess.TrainerProcess = MakeShared<UE::Learning::FSocketTrainerServerProcess>(
		TrainerProcessSettings.TaskName,
		CustomTrainerModulePath,
		TrainerProcessSettings.TrainerFileName,
		PythonExecutablePath,
		PythonContentPath,
		IntermediatePath,
		NneCpuContentPath,
		*SocketSettings.IpAddress,
		SocketSettings.Port,
		SocketSettings.Timeout);

	return TrainerProcess;
}

FLearningAgentsCommunicator ULearningAgentsCommunicatorLibrary::MakeSocketCommunicator(
	FLearningAgentsSocketTrainerProcess TrainerProcess,
	const FLearningAgentsSocketCommunicatorSettings& SocketSettings)
{
	UE::Learning::ETrainerResponse Response = UE::Learning::ETrainerResponse::Success;

	// Ip Address
	FString CmdLineIpAddress;
	bool bIpAddressOverridden = FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsIpAddress="), CmdLineIpAddress);

	// Hostname
	FString CmdLineHostname;
	bool bHostnameOverridden = FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsHostname="), CmdLineHostname);
	if (bHostnameOverridden)
	{
		UE_LOGF(LogLearning, Display, "Using Hostname override from the cmdline: %ls", *CmdLineHostname);
	}
	else if (bIpAddressOverridden)
	{
		UE_LOGF(LogLearning, Display, "Using Ip Address override from the cmdline: %ls", *CmdLineIpAddress);
	}

	// Port
	uint32 CmdLinePort;
	bool bPortOverridden = FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsPort="), CmdLinePort);
	if (bPortOverridden)
	{
		UE_LOGF(LogLearning, Display, "Using Port override from the cmdline: %u", CmdLinePort);
	}

	// Timeout
	float CmdLineTimeout;
	bool bTimeoutOverridden = FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsTimeout="), CmdLineTimeout);
	if (bTimeoutOverridden)
	{
		UE_LOGF(LogLearning, Display, "Overriding Timeout with value from the cmdline: %f", CmdLineTimeout);
	}

	FLearningAgentsCommunicator Communicator;
	Communicator.Trainer = MakeShared<UE::Learning::FSocketTrainer>(
		Response,
		TrainerProcess.TrainerProcess,
		FParse::Param(FCommandLine::Get(), TEXT("LearningAgentsUseExternalTrainingProcess")) || SocketSettings.bUseExternalTrainingProcess,
		bHostnameOverridden
			? *CmdLineHostname
			: bIpAddressOverridden ? *CmdLineIpAddress: *SocketSettings.IpAddress,
		bPortOverridden ? CmdLinePort : SocketSettings.Port,
		bTimeoutOverridden ? CmdLineTimeout : SocketSettings.Timeout,
		bHostnameOverridden);

	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "MakeSocketCommunicator: Failed to connect to training process: %ls. Check log for additional errors.", UE::Learning::Trainer::GetResponseString(Response));
		Communicator.Trainer->Terminate();
	}

	return Communicator;
}
