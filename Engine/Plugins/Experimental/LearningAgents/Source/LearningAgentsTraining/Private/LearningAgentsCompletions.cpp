// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsCompletions.h"

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsDebug.h"

#include "LearningCompletion.h"
#include "LearningLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsCompletions)

namespace UE::Learning::Agents
{
	ELearningAgentsCompletion GetLearningAgentsCompletion(const ECompletionMode CompletionMode)
	{
		switch (CompletionMode)
		{
		case ECompletionMode::Running: return ELearningAgentsCompletion::Running;
		case ECompletionMode::Terminated: return ELearningAgentsCompletion::Termination;
		case ECompletionMode::Truncated: return ELearningAgentsCompletion::Truncation;
		default: UE_LOGF(LogLearning, Error, "Unknown Completion Mode."); return ELearningAgentsCompletion::Running;
		}
	}

	ECompletionMode GetCompletionMode(const ELearningAgentsCompletion Completion)
	{
		switch (Completion)
		{
		case ELearningAgentsCompletion::Running: return ECompletionMode::Running;
		case ELearningAgentsCompletion::Termination: return ECompletionMode::Terminated;
		case ELearningAgentsCompletion::Truncation: return ECompletionMode::Truncated;
		default: UE_LOGF(LogLearning, Error, "Unknown Completion."); return ECompletionMode::Running;
		}
	}

	namespace Private
	{
		const TCHAR* GetCompletionName(const ELearningAgentsCompletion Completion)
		{
			switch (Completion)
			{
			case ELearningAgentsCompletion::Running: return TEXT("Running");
			case ELearningAgentsCompletion::Termination: return TEXT("Termination");
			case ELearningAgentsCompletion::Truncation: return TEXT("Truncation");
			default: return TEXT("Error");
			}
		}
	}
}


bool ULearningAgentsCompletions::IsCompletionRunning(const ELearningAgentsCompletion Completion)
{
	return Completion == ELearningAgentsCompletion::Running;
}

bool ULearningAgentsCompletions::IsCompletionCompleted(const ELearningAgentsCompletion Completion)
{
	return Completion == ELearningAgentsCompletion::Truncation || Completion == ELearningAgentsCompletion::Termination;
}

bool ULearningAgentsCompletions::IsCompletionTruncation(const ELearningAgentsCompletion Completion)
{
	return Completion == ELearningAgentsCompletion::Truncation;
}

bool ULearningAgentsCompletions::IsCompletionTermination(const ELearningAgentsCompletion Completion)
{
	return Completion == ELearningAgentsCompletion::Termination;
}


ELearningAgentsCompletion ULearningAgentsCompletions::CompletionOr(ELearningAgentsCompletion A, ELearningAgentsCompletion B)
{
	if ((A == ELearningAgentsCompletion::Running && B != ELearningAgentsCompletion::Running) ||
		(A == ELearningAgentsCompletion::Truncation && B == ELearningAgentsCompletion::Termination))
	{
		return B;
	}
	else
	{
		return A;
	}
}

ELearningAgentsCompletion ULearningAgentsCompletions::CompletionAnd(ELearningAgentsCompletion A, ELearningAgentsCompletion B)
{
	if (A == ELearningAgentsCompletion::Running ||
		(A == ELearningAgentsCompletion::Truncation && B != ELearningAgentsCompletion::Running))
	{
		return A;
	}
	else
	{
		return B;
	}
}

ELearningAgentsCompletion ULearningAgentsCompletions::CompletionNot(ELearningAgentsCompletion A, ELearningAgentsCompletion NotRunningType)
{
	return A == ELearningAgentsCompletion::Running ? NotRunningType : ELearningAgentsCompletion::Running;
}


ELearningAgentsCompletion ULearningAgentsCompletions::MakeCompletion(
	const ELearningAgentsCompletion CompletionType,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nCompletion: [%s]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			UE::Learning::Agents::Private::GetCompletionName(CompletionType));
	}
#endif

	return CompletionType;
}

ELearningAgentsCompletion ULearningAgentsCompletions::MakeCompletionOnCondition(
	const bool bCondition, 
	const ELearningAgentsCompletion CompletionType,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const ELearningAgentsCompletion Completion = bCondition ? CompletionType : ELearningAgentsCompletion::Running;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nCondition: [%s]\nCompletion Type: [%s]\nCompletion: [%s]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			bCondition ? TEXT("true") : TEXT("false"),
			UE::Learning::Agents::Private::GetCompletionName(CompletionType),
			UE::Learning::Agents::Private::GetCompletionName(Completion));
	}
#endif

	return Completion;
}

ELearningAgentsCompletion ULearningAgentsCompletions::MakeCompletionOnTimeElapsed(
	const float Time, 
	const float TimeThreshold, 
	const ELearningAgentsCompletion CompletionType,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const bool bCondition = Time > TimeThreshold;
	const ELearningAgentsCompletion Completion = MakeCompletionOnCondition(bCondition, CompletionType);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nTime: [% 6.2f]\nThreshold: [% 6.2f]\nCondition: [%s]\nCompletion Type: [%s]\nCompletion: [%s]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			Time,
			TimeThreshold,
			bCondition ? TEXT("true") : TEXT("false"),
			UE::Learning::Agents::Private::GetCompletionName(CompletionType),
			UE::Learning::Agents::Private::GetCompletionName(Completion));
	}
#endif

	return Completion;
}

ELearningAgentsCompletion ULearningAgentsCompletions::MakeCompletionOnEpisodeStepsRecorded(
	const int32 EpisodeSteps, 
	const int32 MaxEpisodeSteps, 
	const ELearningAgentsCompletion CompletionType,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const bool bCondition = EpisodeSteps >= MaxEpisodeSteps;
	const ELearningAgentsCompletion Completion = MakeCompletionOnCondition(bCondition, CompletionType);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nEpisodeSteps: [% 4i]\nMaxEpisodeSteps: [% 4i]\nCondition: [%s]\nCompletion Type: [%s]\nCompletion: [%s]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			EpisodeSteps,
			MaxEpisodeSteps,
			bCondition ? TEXT("true") : TEXT("false"),
			UE::Learning::Agents::Private::GetCompletionName(CompletionType),
			UE::Learning::Agents::Private::GetCompletionName(Completion));
	}
#endif

	return Completion;
}

ELearningAgentsCompletion ULearningAgentsCompletions::MakeCompletionOnLocationDifferenceBelowThreshold(
	const FVector LocationA, 
	const FVector LocationB, 
	const float DistanceThreshold, 
	const ELearningAgentsCompletion CompletionType,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const float Distance = FVector::Distance(LocationA, LocationB);
	const bool bCondition = Distance < DistanceThreshold;
	const ELearningAgentsCompletion Completion = MakeCompletionOnCondition(bCondition, CompletionType);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationA,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			LocationA,
			LocationB,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationB,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocationA: [% 6.1f % 6.1f % 6.1f]\nLocationB: [% 6.1f % 6.1f % 6.1f]\nDistance: [% 6.2f]\nThreshold: [% 6.2f]\nCondition: [%s]\nCompletion Type: [%s]\nCompletion: [%s]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			LocationA.X, LocationA.Y, LocationA.Z,
			LocationB.X, LocationB.Y, LocationB.Z,
			Distance,
			DistanceThreshold,
			bCondition ? TEXT("true") : TEXT("false"),
			UE::Learning::Agents::Private::GetCompletionName(CompletionType),
			UE::Learning::Agents::Private::GetCompletionName(Completion));
	}
#endif

	return Completion;
}

ELearningAgentsCompletion ULearningAgentsCompletions::MakeCompletionOnLocationDifferenceAboveThreshold(
	const FVector LocationA, 
	const FVector LocationB, 
	const float DistanceThreshold, 
	const ELearningAgentsCompletion CompletionType,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const float Distance = FVector::Distance(LocationA, LocationB);
	const bool bCondition = Distance > DistanceThreshold;
	const ELearningAgentsCompletion Completion = MakeCompletionOnCondition(bCondition, CompletionType);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationA,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			LocationA,
			LocationB,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationB,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocationA: [% 6.1f % 6.1f % 6.1f]\nLocationB: [% 6.1f % 6.1f % 6.1f]\nDistance: [% 6.2f]\nThreshold: [% 6.2f]\nCondition: [%s]\nCompletion Type: [%s]\nCompletion: [%s]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			LocationA.X, LocationA.Y, LocationA.Z,
			LocationB.X, LocationB.Y, LocationB.Z,
			Distance,
			DistanceThreshold,
			bCondition ? TEXT("true") : TEXT("false"),
			UE::Learning::Agents::Private::GetCompletionName(CompletionType),
			UE::Learning::Agents::Private::GetCompletionName(Completion));
	}
#endif

	return Completion;
}

ELearningAgentsCompletion ULearningAgentsCompletions::MakeCompletionOnLocationOutsideBounds(
	const FVector Location,
	const FTransform BoundsTransform,
	const FVector BoundsMins,
	const FVector BoundsMaxs,
	const ELearningAgentsCompletion CompletionType,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const FVector LocalLocation = BoundsTransform.InverseTransformPosition(Location);
	
	const bool bCondition = (
		LocalLocation.X < BoundsMins.X || LocalLocation.X > BoundsMaxs.X ||
		LocalLocation.Y < BoundsMins.Y || LocalLocation.Y > BoundsMaxs.Y ||
		LocalLocation.Z < BoundsMins.Z || LocalLocation.Z > BoundsMaxs.Z);

	const ELearningAgentsCompletion Completion = MakeCompletionOnCondition(bCondition, CompletionType);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			Location,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			Location,
			BoundsTransform.GetLocation(),
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_OBOX(VisualLoggerObject, LogLearning, Display,
			FBox(BoundsMins, BoundsMaxs),
			BoundsTransform.ToMatrixWithScale(),
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocation: [% 6.1f % 6.1f % 6.1f]\nLocal Location: [% 6.1f % 6.1f % 6.1f]\nBounds Mins: [% 6.1f % 6.1f % 6.1f]\nBounds Maxs: [% 6.1f % 6.1f % 6.1f]\nCondition: [%s]\nCompletion Type: [%s]\nCompletion: [%s]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			Location.X, Location.Y, Location.Z,
			LocalLocation.X, LocalLocation.Y, LocalLocation.Z,
			BoundsMins.X, BoundsMins.Y, BoundsMins.Z,
			BoundsMaxs.X, BoundsMaxs.Y, BoundsMaxs.Z,
			bCondition ? TEXT("true") : TEXT("false"),
			UE::Learning::Agents::Private::GetCompletionName(CompletionType),
			UE::Learning::Agents::Private::GetCompletionName(Completion));
	}
#endif

	return Completion;
}

