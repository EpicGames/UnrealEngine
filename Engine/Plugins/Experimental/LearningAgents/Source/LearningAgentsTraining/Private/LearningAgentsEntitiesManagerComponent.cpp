// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsEntitiesManagerComponent.h"
#include "Engine/World.h"
#include "LearningLog.h"
#include "LearningAgentsGym.h"
#include "LearningAgentsEntityInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsEntitiesManagerComponent)

void ULearningAgentsEntitiesManagerComponent::InitializeLearningComponent()
{
	TObjectPtr<ALearningAgentsGymBase> Gym = Cast<ALearningAgentsGymBase>(GetOwner());
	if (Gym)
	{
		if (!CheckEntityClasses())
		{
			return;
		}

		if (Gym->GetLocalRole() == ROLE_Authority)
		{
			for (FLearningAgentsEntityInfo& EntityInfo : Entities)
			{
				int32 SpawnCount = 0;
				if (ensure(Gym->GetRandomStream()))
				{
					SpawnCount = Gym->GetRandomStream()->RandRange(EntityInfo.EpisodeEntitySpawnCountMin, EntityInfo.EpisodeEntitySpawnCountMax);
				}
				SpawnEntitiesAtRandomLocations(EntityInfo.EntityClass, EntityInfo.EntitySpawnZOffset, SpawnCount);
			}
		}
	}
}

void ULearningAgentsEntitiesManagerComponent::ResetLearningComponent()
{
	TObjectPtr<ALearningAgentsGymBase> Gym = Cast<ALearningAgentsGymBase>(GetOwner());
	if (Gym)
	{
		if (!CheckEntityClasses())
		{
			return;
		}
		for (TPair<FName, FSpawnedEntitiesInfo>& UniqueTypeEntities : EntitiesPool)
		{
			int32 SpawnCount = 0;
			if (ensure(Gym->GetRandomStream()))
			{
				SpawnCount = Gym->GetRandomStream()->RandRange(UniqueTypeEntities.Value.EntityInfo.EpisodeEntitySpawnCountMin, UniqueTypeEntities.Value.EntityInfo.EpisodeEntitySpawnCountMax);
			}
			for (int32 i = 0; i < UniqueTypeEntities.Value.SpawnedEntities.Num(); ++i)
			{
				if (i < SpawnCount)
				{
					// Make sure transform is applied before reset to wipe all data (ex: distance traveled)
					if (TObjectPtr<AActor> ActorEntity = Cast<AActor>(UniqueTypeEntities.Value.SpawnedEntities[i].GetObject()))
					{
						FTransform RandomTransform;
						RandomizeTransform(RandomTransform, UniqueTypeEntities.Value.EntityInfo.EntitySpawnZOffset);
						ActorEntity->SetActorTransform(RandomTransform);
					}

					ILearningAgentsEntityInterface::Execute_ResetEntity(UniqueTypeEntities.Value.SpawnedEntities[i].GetObject(), Gym);
					ILearningAgentsEntityInterface::Execute_EnableEntity(UniqueTypeEntities.Value.SpawnedEntities[i].GetObject());

					if (int32 AdditionalSpawnCount = SpawnCount - i - 1)
					{
						SpawnEntitiesAtRandomLocations(UniqueTypeEntities.Value.EntityInfo.EntityClass, UniqueTypeEntities.Value.EntityInfo.EntitySpawnZOffset, SpawnCount - i - 1);
					}
				}
				else
				{
					ILearningAgentsEntityInterface::Execute_DisableEntity(UniqueTypeEntities.Value.SpawnedEntities[i].GetObject());
				}
			}
		}
	}
}

TScriptInterface<ILearningAgentsEntityInterface> ULearningAgentsEntitiesManagerComponent::SpawnEntitiesAtRandomLocations(TSubclassOf<AActor> EntityClass, float EntitySpawnZOffset, int32 SpawnCount)
{
	if (SpawnCount <= 0)
	{
		UE_LOGF(LogLearning, Warning, "Spawn Count must be greater than 0 in %ls!", *EntityClass->GetName());
		return TScriptInterface<ILearningAgentsEntityInterface>();
	}

	FTransform Transform = FTransform();
	RandomizeTransform(Transform, EntitySpawnZOffset);

	TArray<TScriptInterface<ILearningAgentsEntityInterface>> SpawnedEntities = SpawnEntities(EntityClass, EntitySpawnZOffset, SpawnCount, Transform);
	if (!SpawnedEntities.IsEmpty())
	{
		return SpawnedEntities.Top();
	}
	else
	{
		UE_LOGF(LogLearning, Warning, "Could not spawn entity from class %ls!", *EntityClass->GetName());
		return TScriptInterface<ILearningAgentsEntityInterface>();
	}
}

TScriptInterface<ILearningAgentsEntityInterface> ULearningAgentsEntitiesManagerComponent::SpawnEntityAtProjectedLocation(TSubclassOf<AActor> EntityClass, float EntitySpawnZOffset, const FTransform& InTransform)
{
	FTransform Transform = InTransform;
	ProjectTransform(Transform);

	TArray<TScriptInterface<ILearningAgentsEntityInterface>> SpawnedEntities = SpawnEntities(EntityClass, EntitySpawnZOffset, 1, Transform);
	if (!SpawnedEntities.IsEmpty())
	{
		return SpawnedEntities.Top();
	}
	else
	{
		UE_LOGF(LogLearning, Warning, "Could not spawn entity from class %ls!", *EntityClass->GetName());
		return TScriptInterface<ILearningAgentsEntityInterface>();
	}
}

TArray<TScriptInterface<ILearningAgentsEntityInterface>> ULearningAgentsEntitiesManagerComponent::SpawnEntities(TSubclassOf<AActor> EntityClass, float EntitySpawnZOffset, int32 SpawnCount, const FTransform& InTransform)
{
	TArray<TScriptInterface<ILearningAgentsEntityInterface>> SpawnedEntities;
	SpawnedEntities.SetNumUninitialized(SpawnCount);

	if (SpawnCount <= 0)
	{
		UE_LOGF(LogLearning, Warning, "Spawn Count must be greater than 0 in %ls!", *EntityClass->GetName());
		return SpawnedEntities;
	}

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Owner = GetOwner();

	FTransform Transform = InTransform;

	int32 SuccessfulSpawnCount = 0;
	if (FSpawnedEntitiesInfo* SpawnEntitiesInfo = EntitiesPool.Find(EntityClass->GetFName()))
	{
		for (TScriptInterface<ILearningAgentsEntityInterface>& Entity : SpawnEntitiesInfo->SpawnedEntities)
		{
			if (SuccessfulSpawnCount == SpawnCount)
			{
				return SpawnedEntities;
			}
			if (!ILearningAgentsEntityInterface::Execute_IsEntityEnabled(Entity.GetObject()))
			{
				if (TObjectPtr<AActor> ActorEntity = Cast<AActor>(Entity.GetObject()))
				{
					ActorEntity->SetActorTransform(Transform);
				}

				ILearningAgentsEntityInterface::Execute_EnableEntity(Entity.GetObject());
				SpawnedEntities.Add(Entity);

				SuccessfulSpawnCount++;
			}
		}
		// Allocate new object and add to pool because there's not enough in pool to pull from.
		for (int32 Index = SuccessfulSpawnCount; SuccessfulSpawnCount < SpawnCount; ++SuccessfulSpawnCount)
		{
			if (EntityClass->IsChildOf(AActor::StaticClass()))
			{
				TObjectPtr<AActor> Entity = GetWorld()->SpawnActor<AActor>(EntityClass, Transform.GetLocation(), Transform.GetRotation().Rotator(), SpawnParameters);
				if (TObjectPtr<ALearningAgentsGymBase> Gym = Cast<ALearningAgentsGymBase>(GetOwner()))
				{
					ILearningAgentsEntityInterface::Execute_InitializeEntity(Entity, Gym);
				}
				SpawnEntitiesInfo->SpawnedEntities.Add(Entity);
				SpawnedEntities.Add(TScriptInterface<ILearningAgentsEntityInterface>(Entity));
			}
			else
			{
				UE_LOGF(LogLearning, Warning, "Could not spawn entity from class %ls because it's not an AActor!", *EntityClass->GetName());
			}
		}
	}
	// New entity spawned during an episode
	else
	{
		FSpawnedEntitiesInfo NewSpawnEntitiesInfo;
		FLearningAgentsEntityInfo NewEntityInfo;
		NewEntityInfo.EntityClass = EntityClass;
		NewEntityInfo.EntitySpawnZOffset = EntitySpawnZOffset;
		NewSpawnEntitiesInfo.EntityInfo = NewEntityInfo;

		for (int32 Index = 0; Index < SpawnCount; ++Index)
		{
			if (EntityClass->IsChildOf(AActor::StaticClass()))
			{
				TObjectPtr<AActor> Entity = GetWorld()->SpawnActor<AActor>(EntityClass, Transform.GetLocation(), Transform.GetRotation().Rotator(), SpawnParameters);
				if (TObjectPtr<ALearningAgentsGymBase> Gym = Cast<ALearningAgentsGymBase>(GetOwner()))
				{
					ILearningAgentsEntityInterface::Execute_InitializeEntity(Entity, Gym);
				}
				NewSpawnEntitiesInfo.SpawnedEntities.Add(Entity);
				SpawnedEntities.Add(TScriptInterface<ILearningAgentsEntityInterface>(Entity));
			}
			else
			{
				UE_LOGF(LogLearning, Warning, "Could not spawn entity from class %ls because it's not an AActor!", *EntityClass->GetName());
			}
		}
		EntitiesPool.Add(EntityClass->GetFName(), NewSpawnEntitiesInfo);
	}
	return SpawnedEntities;
}

void ULearningAgentsEntitiesManagerComponent::BeginPlay()
{
	Super::BeginPlay();
	if (CheckEntityClasses())
	{
		for (FLearningAgentsEntityInfo& EntityInfo : Entities)
		{
			FSpawnedEntitiesInfo SpawnedEntitiesInfo;
			SpawnedEntitiesInfo.EntityInfo = EntityInfo;
			EntitiesPool.Add(EntityInfo.EntityClass->GetFName(), SpawnedEntitiesInfo);
		}
	}
}

void ULearningAgentsEntitiesManagerComponent::ProjectTransform(FTransform& Transform) const
{
	if (TObjectPtr<ALearningAgentsGymBase> Gym = Cast<ALearningAgentsGymBase>(GetOwner()))
	{
		Transform.SetLocation(Gym->ProjectPointToGym(Transform.GetLocation()));
	}
	else
	{
		UE_LOGF(LogLearning, Warning, "Entities Manager Component attached to an owner that is not a ALearningAgentsGymBase!");
	}
}

void ULearningAgentsEntitiesManagerComponent::RandomizeTransform(FTransform& OutTransform, float LocationZOffset) const
{
	if (TObjectPtr<ALearningAgentsGymBase> Gym = Cast<ALearningAgentsGymBase>(GetOwner()))
	{
		FVector Location = Gym->GenerateRandomLocationInGym();
		Location.Z += LocationZOffset;
		OutTransform.SetLocation(Location);
		OutTransform.SetRotation(Gym->GenerateRandomRotationInGym().Quaternion());
	}
	else
	{
		UE_LOGF(LogLearning, Warning, "Entities Manager Component attached to an owner that is not a ALearningAgentsGymBase!");
	}
}

bool ULearningAgentsEntitiesManagerComponent::CheckEntityClasses() const
{
	if (Entities.IsEmpty())
	{
		UE_LOGF(LogLearning, Error, "No entity classes are setup for %ls!", *GetName());
		return false;
	}

	for (const FLearningAgentsEntityInfo& EntityInfo : Entities)
	{
		if (!EntityInfo.EntityClass)
		{
			UE_LOGF(LogLearning, Error, "An entity spawn entry is set to None for %ls!", *GetName());
			return false;
		}

		if (!EntityInfo.EntityClass->ImplementsInterface(ULearningAgentsEntityInterface::StaticClass()))
		{
			UE_LOGF(LogLearning, Error, "Invalid entity class! %ls does not implement ILearningAgentsEntityInterface!", *EntityInfo.EntityClass->GetName());
			return false;
		}
	}
	return true;
}
