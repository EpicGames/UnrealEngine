// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsFlowMatchingTrainer.h"

#include "LearningAgentsCommunicator.h"
#include "LearningAgentsInteractor.h"
#include "LearningAgentsManager.h"
#include "LearningAgentsFlowMatching.h"
#include "LearningAgentsRecording.h"

#include "LearningExperience.h"
#include "LearningExternalTrainer.h"
#include "LearningLog.h"
#include "LearningNeuralNetwork.h"
#include "LearningTrainer.h"

#include "Dom/JsonObject.h"
#include "HAL/PlatformMisc.h"
#include "Misc/Paths.h"
#include "Logging/StructuredLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsFlowMatchingTrainer)

TSharedRef<FJsonObject> FLearningAgentsFlowMatchingTrainerTrainingSettings::AsJsonConfig() const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	ConfigObject->SetNumberField(TEXT("IterationNum"), Hyperparameters.NumberOfIterations);
	ConfigObject->SetNumberField(TEXT("LearningRate"), Hyperparameters.LearningRate);
	ConfigObject->SetNumberField(TEXT("WeightDecay"), Hyperparameters.WeightDecay);
	ConfigObject->SetNumberField(TEXT("BatchSize"), Hyperparameters.BatchSize);

	ConfigObject->SetNumberField(TEXT("ObservationNoiseScale"), Hyperparameters.ObservationNoiseScale);
	ConfigObject->SetBoolField(TEXT("NormalizeLatents"), Hyperparameters.bNormalizeLatents);
	ConfigObject->SetNumberField(TEXT("Seed"), Hyperparameters.RandomSeed);

	ConfigObject->SetBoolField(TEXT("SaveSnapshots"), LoggingSettings.bSaveSnapshots);
	ConfigObject->SetNumberField(TEXT("IterationsPerSnapshot"), LoggingSettings.IterationsPerSnapshot);

	return ConfigObject;
}

ULearningAgentsFlowMatchingTrainer::ULearningAgentsFlowMatchingTrainer() : Super(FObjectInitializer::Get()) {}
ULearningAgentsFlowMatchingTrainer::ULearningAgentsFlowMatchingTrainer(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsFlowMatchingTrainer::~ULearningAgentsFlowMatchingTrainer() = default;

void ULearningAgentsFlowMatchingTrainer::BeginDestroy()
{
	if (IsTraining())
	{
		EndTraining();
	}

	Super::BeginDestroy();
}

ULearningAgentsFlowMatchingTrainer* ULearningAgentsFlowMatchingTrainer::MakeFlowMatchingTrainer(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	ULearningAgentsFlowMatching*& InFlowMatchingModel,
	const FLearningAgentsCommunicator& Communicator,
	TSubclassOf<ULearningAgentsFlowMatchingTrainer> Class,
	const FName Name)
{
	if (!InManager)
	{
		UE_LOGFMT(LogLearning, Error, "MakeFlowMatchingTrainer: InManager is nullptr.");
		return nullptr;
	}

	if (!Class)
	{
		UE_LOGFMT(LogLearning, Error, "MakeFlowMatchingTrainer: Class is nullptr.");
		return nullptr;
	}

	const FName UniqueName = MakeUniqueObjectName(InManager, Class, Name, EUniqueObjectNameOptions::GloballyUnique);

	ULearningAgentsFlowMatchingTrainer* FlowMatchingTrainer = NewObject<ULearningAgentsFlowMatchingTrainer>(InManager, Class, UniqueName);
	if (!FlowMatchingTrainer) { return nullptr; }

	FlowMatchingTrainer->SetupFlowMatchingTrainer(InManager, InInteractor, InFlowMatchingModel, Communicator);

	return FlowMatchingTrainer->IsSetup() ? FlowMatchingTrainer : nullptr;
}

void ULearningAgentsFlowMatchingTrainer::SetupFlowMatchingTrainer(
	ULearningAgentsManager* InManager,
	ULearningAgentsInteractor* InInteractor,
	ULearningAgentsFlowMatching* InFlowMatchingModel,
	const FLearningAgentsCommunicator& Communicator)
{
	if (IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup already performed!", ("Name", GetName()));
		return;
	}

	if (!InManager)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: InManager is nullptr.", ("Name", GetName()));
		return;
	}

	if (!InInteractor)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: InInteractor is nullptr.", ("Name", GetName()));
		return;
	}

	if (!InInteractor->IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: {Interactor}'s Setup must be run before it can be used.", ("Name", GetName()), ("Interactor", InInteractor->GetName()));
		return;
	}

	if (!InFlowMatchingModel)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: InFlowMatchingModel is nullptr.", ("Name", GetName()));
		return;
	}

	if (!InFlowMatchingModel->IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: {InFlowMatchingModel}'s Setup must be run before it can be used.", ("Name", GetName()), ("InFlowMatchingModel", InFlowMatchingModel->GetName()));
		return;
	}

	if (!Communicator.Trainer)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Communicator's Trainer is nullptr.", ("Name", GetName()));
		return;
	}

	Manager = InManager;
	Interactor = InInteractor;
	Model = InFlowMatchingModel;
	Trainer = Communicator.Trainer;

	bIsSetup = true;

	Manager->AddListener(this);
}

void ULearningAgentsFlowMatchingTrainer::BeginTraining(
	const ULearningAgentsRecording* Recording,
	const FLearningAgentsFlowMatchingTrainerSettings& FlowMatchingTrainerSettings,
	const FLearningAgentsFlowMatchingTrainerTrainingSettings& FlowMatchingTrainerTrainingSettings,
	const FLearningAgentsTrainerProcessSettings& FlowMatchingTrainerPathSettings)
{
	if (!IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup not complete.", ("Name", GetName()));
		return;
	}

	if (IsTraining())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Cannot begin training as we are already training!", ("Name", GetName()));
		return;
	}

	if (!Recording)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Recording is nullptr.", ("Name", GetName()));
		return;
	}

	if (Recording->Records.IsEmpty())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Recording is empty!", ("Name", GetName()));
		return;
	}

	// Build Replay Buffer from Recording
	ReplayBuffer = Recording->BuildReplayBuffer(
		Interactor,
		FlowMatchingTrainerSettings.RecordingFilterTag,
		FlowMatchingTrainerTrainingSettings.ValidationSettings.bCheckObservationSchemaCompatibility,
		FlowMatchingTrainerTrainingSettings.ValidationSettings.bCheckActionSchemaCompatibility);

	if (!ReplayBuffer)
	{
		UE_LOGFMT(LogLearning, Warning, "{Name}: Recording contains no valid training data.", ("Name", GetName()));
		return;
	}

	DenoiserNetworkId = Trainer->AddNetwork(*Model->GetDenoiserNetworkAsset()->NeuralNetworkData);
	EncoderNetworkId = Trainer->AddNetwork(*Model->GetEncoderNetworkAsset()->NeuralNetworkData);
	DecoderNetworkId = Trainer->AddNetwork(*Model->GetDecoderNetworkAsset()->NeuralNetworkData);
	ReplayBufferId = Trainer->AddReplayBuffer(*ReplayBuffer);

	TSharedRef<FJsonObject> DataConfigObject = CreateDataConfig();
	TSharedRef<FJsonObject> ModelConfigObject = CreateModelConfig();
	TSharedRef<FJsonObject> TrainerConfigObject = CreateTrainerConfig(FlowMatchingTrainerTrainingSettings);

	UE_LOGFMT(LogLearning, Display, "{Name}: Sending configs...", ("Name", GetName()));
	SendConfigs(DataConfigObject, ModelConfigObject, TrainerConfigObject);

	UE_LOGFMT(LogLearning, Display, "{Name}: Flow Matching Training Started", ("Name", GetName()));
	UE::Learning::ETrainerResponse Response = UE::Learning::ETrainerResponse::Success;

	UE_LOGFMT(LogLearning, Display, "{Name}: Sending / Receiving initial Denoiser...", ("Name", GetName()));
	Response = Trainer->SendNetwork(DenoiserNetworkId, *Model->GetDenoiserNetworkAsset()->NeuralNetworkData);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Error sending Denoiser to trainer: {Response}. Check log for additional errors.", ("Name", GetName()), ("Response", UE::Learning::Trainer::GetResponseString(Response)));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	Response = Trainer->SendNetwork(EncoderNetworkId, *Model->GetEncoderNetworkAsset()->NeuralNetworkData);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Error sending encoder to trainer: {Response}. Check log for additional errors.", ("Name", GetName()), ("Response", UE::Learning::Trainer::GetResponseString(Response)));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	Response = Trainer->SendNetwork(DecoderNetworkId, *Model->GetDecoderNetworkAsset()->NeuralNetworkData);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Error sending Decoder to trainer: {Response}. Check log for additional errors.", ("Name", GetName()), ("Response", UE::Learning::Trainer::GetResponseString(Response)));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	UE_LOGFMT(LogLearning, Display, "{Name}: Sending Experience...", ("Name", GetName()));
	Response = Trainer->SendReplayBuffer(ReplayBufferId, *ReplayBuffer);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Error sending experience to trainer: {Response}. Check log for additional errors.", ("Name", GetName()), ("Response", UE::Learning::Trainer::GetResponseString(Response)));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	bIsTraining = true;
}

void ULearningAgentsFlowMatchingTrainer::ExportToFile(
	UE::Learning::FFileTrainer& FileTrainer,
	const ULearningAgentsRecording* Recording,
	const FLearningAgentsFlowMatchingTrainerSettings& FlowMatchingTrainerSettings,
	const FLearningAgentsFlowMatchingTrainerTrainingSettings& FlowMatchingTrainerTrainingSettings,
	UE::Learning::ETrainerExportFlags ExportFlags)
{
	using namespace UE::Learning;

	if (!IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup not complete.", ("Name", GetName()));
		return;
	}

	if (IsTraining())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Cannot export while training is active.", ("Name", GetName()));
		return;
	}

	const bool bNeedReplayBuffers = EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::ReplayBuffers);
	const bool bNeedNetworks = EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::Networks);
	const bool bNeedTrainerConfig = EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::TrainerConfig);

	// Networks: register to get IDs (cheap)
	if (bNeedNetworks)
	{
		DenoiserNetworkId = FileTrainer.AddNetwork(*Model->GetDenoiserNetworkAsset()->NeuralNetworkData);
		EncoderNetworkId = FileTrainer.AddNetwork(*Model->GetEncoderNetworkAsset()->NeuralNetworkData);
		DecoderNetworkId = FileTrainer.AddNetwork(*Model->GetDecoderNetworkAsset()->NeuralNetworkData);
	}

	// Replay buffers: build from recording (expensive)
	if (bNeedReplayBuffers)
	{
		if (!Recording)
		{
			UE_LOGFMT(LogLearning, Error, "{Name}: Recording is nullptr.", ("Name", GetName()));
			return;
		}

		if (Recording->Records.IsEmpty())
		{
			UE_LOGFMT(LogLearning, Error, "{Name}: Recording is empty!", ("Name", GetName()));
			return;
		}

		ReplayBuffer = Recording->BuildReplayBuffer(
			Interactor,
			FlowMatchingTrainerSettings.RecordingFilterTag,
			FlowMatchingTrainerTrainingSettings.ValidationSettings.bCheckObservationSchemaCompatibility,
			FlowMatchingTrainerTrainingSettings.ValidationSettings.bCheckActionSchemaCompatibility);

		if (!ReplayBuffer)
		{
			UE_LOGFMT(LogLearning, Warning, "{Name}: Recording contains no valid training data.", ("Name", GetName()));
			return;
		}

		ReplayBufferId = FileTrainer.AddReplayBuffer(*ReplayBuffer);
	}

	// Build and send configs
	TSharedRef<FJsonObject> DataConfigObject = bNeedReplayBuffers ? CreateDataConfig() : MakeShared<FJsonObject>();
	TSharedRef<FJsonObject> ModelConfigObject = bNeedNetworks ? CreateModelConfig() : MakeShared<FJsonObject>();
	TSharedRef<FJsonObject> TrainerConfigObject = bNeedTrainerConfig ? CreateTrainerConfig(FlowMatchingTrainerTrainingSettings) : MakeShared<FJsonObject>();
	FileTrainer.SendConfigs(DataConfigObject, ModelConfigObject, TrainerConfigObject);

	// Send network weights
	if (bNeedNetworks)
	{
		FileTrainer.SendNetwork(DenoiserNetworkId, *Model->GetDenoiserNetworkAsset()->NeuralNetworkData);
		FileTrainer.SendNetwork(EncoderNetworkId, *Model->GetEncoderNetworkAsset()->NeuralNetworkData);
		FileTrainer.SendNetwork(DecoderNetworkId, *Model->GetDecoderNetworkAsset()->NeuralNetworkData);
	}

	// Send replay buffer data
	if (bNeedReplayBuffers)
	{
		FileTrainer.SendReplayBuffer(ReplayBufferId, *ReplayBuffer);
	}

	UE_LOGFMT(LogLearning, Display, "{Name}: Export complete.", ("Name", GetName()));
}

TSharedRef<FJsonObject> ULearningAgentsFlowMatchingTrainer::CreateSchemaConfig(const int32 ObservationSchemaId, const int32 ActionSchemaId) const
{
	TSharedRef<FJsonObject> SchemasObject = MakeShared<FJsonObject>();

	TArray<TSharedPtr<FJsonValue>> ObservationSchemaObjects;
	{
		TSharedPtr<FJsonObject> ObservationSchemaObject = MakeShared<FJsonObject>();
		ObservationSchemaObject->SetNumberField(TEXT("Id"), ObservationSchemaId);
		ObservationSchemaObject->SetStringField(TEXT("Name"), "Default");
		ObservationSchemaObject->SetObjectField(TEXT("Schema"),
			UE::Learning::Trainer::ConvertObservationSchemaToJSON(Interactor->GetObservationSchema()->ObservationSchema,
				Interactor->GetObservationSchemaElement().SchemaElement));

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(ObservationSchemaObject);
		ObservationSchemaObjects.Add(JsonValue);
	}
	SchemasObject->SetArrayField(TEXT("Observations"), ObservationSchemaObjects);

	TArray<TSharedPtr<FJsonValue>> ActionSchemaObjects;
	{
		TSharedPtr<FJsonObject> ActionSchemaObject = MakeShared<FJsonObject>();
		ActionSchemaObject->SetNumberField(TEXT("Id"), ActionSchemaId);
		ActionSchemaObject->SetStringField(TEXT("Name"), "Default");
		ActionSchemaObject->SetObjectField(TEXT("Schema"),
			UE::Learning::Trainer::ConvertActionSchemaToJSON(Interactor->GetActionSchema()->ActionSchema,
				Interactor->GetActionSchemaElement().SchemaElement));

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(ActionSchemaObject);
		ActionSchemaObjects.Add(JsonValue);
	}
	SchemasObject->SetArrayField(TEXT("Actions"), ActionSchemaObjects);

	return SchemasObject;
}

TSharedRef<FJsonObject> ULearningAgentsFlowMatchingTrainer::CreateDataConfig() const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	const int32 ObservationSchemaId = 0;
	const int32 ActionSchemaId = 0;

	// Add Replay Buffers Config
	TArray<TSharedPtr<FJsonValue>> ReplayBufferObjects;
	TSharedRef<FJsonValueObject> ReplayBufferJsonValue = MakeShared<FJsonValueObject>(ReplayBuffer->AsJsonConfig(ReplayBufferId));
	ReplayBufferObjects.Add(ReplayBufferJsonValue);
	ConfigObject->SetArrayField(TEXT("ReplayBuffers"), ReplayBufferObjects);

	ConfigObject->SetObjectField(TEXT("Schemas"), CreateSchemaConfig(ObservationSchemaId, ActionSchemaId));

	return ConfigObject;
}

TSharedRef<FJsonObject> ULearningAgentsFlowMatchingTrainer::CreateModelConfig() const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	const int32 ObservationSchemaId = 0;
	const int32 ActionSchemaId = 0;

	// Add Neural Network Config Entries
	TArray<TSharedPtr<FJsonValue>> NetworkObjects;

	// Denoiser
	{
		TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
		NetworkObject->SetNumberField(TEXT("Id"), DenoiserNetworkId);
		NetworkObject->SetStringField(TEXT("Name"), Model->GetDenoiserNetworkAsset()->GetFName().ToString());
		NetworkObject->SetNumberField(TEXT("MaxByteNum"), Model->GetDenoiserNetworkAsset()->NeuralNetworkData->GetSnapshotByteNum());

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
		NetworkObjects.Add(JsonValue);
	}

	// Encoder
	{
		TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
		NetworkObject->SetNumberField(TEXT("Id"), EncoderNetworkId);
		NetworkObject->SetStringField(TEXT("Name"), Model->GetEncoderNetworkAsset()->GetFName().ToString());
		NetworkObject->SetNumberField(TEXT("MaxByteNum"), Model->GetEncoderNetworkAsset()->NeuralNetworkData->GetSnapshotByteNum());
		NetworkObject->SetNumberField(TEXT("InputSchemaId"), ObservationSchemaId);

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
		NetworkObjects.Add(JsonValue);
	}

	// Decoder
	{
		TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
		NetworkObject->SetNumberField(TEXT("Id"), DecoderNetworkId);
		NetworkObject->SetStringField(TEXT("Name"), Model->GetDecoderNetworkAsset()->GetFName().ToString());
		NetworkObject->SetNumberField(TEXT("MaxByteNum"), Model->GetDecoderNetworkAsset()->NeuralNetworkData->GetSnapshotByteNum());
		NetworkObject->SetNumberField(TEXT("OutputSchemaId"), ActionSchemaId);

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
		NetworkObjects.Add(JsonValue);
	}

	ConfigObject->SetArrayField(TEXT("Networks"), NetworkObjects);

	ConfigObject->SetObjectField(TEXT("FlowMatchingSettings"), Model->AsJsonConfig());

	return ConfigObject;
}

TSharedRef<FJsonObject> ULearningAgentsFlowMatchingTrainer::CreateTrainerConfig(const FLearningAgentsFlowMatchingTrainerTrainingSettings& TrainingSettings) const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	ConfigObject->SetStringField(TEXT("TrainerMethod"), TEXT("FlowMatching"));
	ConfigObject->SetStringField(TEXT("TimeStamp"), *FDateTime::Now().ToFormattedString(TEXT("%Y-%m-%d_%H-%M-%s")));

	ConfigObject->SetObjectField(TEXT("FlowMatchingTrainingSettings"), TrainingSettings.AsJsonConfig());

	ConfigObject->SetBoolField(TEXT("EmbedNormalizationsInWeights"), TrainingSettings.bEmbedNormalizationsInWeights);

	ConfigObject->SetStringField(TEXT("Device"), UE::Learning::Trainer::GetDeviceString(UE::Learning::Agents::GetTrainingDevice(TrainingSettings.Device)));
	ConfigObject->SetBoolField(TEXT("UseTensorBoard"), TrainingSettings.LoggingSettings.bUseTensorboard);
	ConfigObject->SetBoolField(TEXT("UseMLflow"), TrainingSettings.LoggingSettings.bUseMLflow);
	ConfigObject->SetStringField(TEXT("MLflowTrackingUri"), TrainingSettings.LoggingSettings.MLflowTrackingUri);

	return ConfigObject;
}

void ULearningAgentsFlowMatchingTrainer::SendConfigs(const TSharedRef<FJsonObject>& DataConfigObject, const TSharedRef<FJsonObject>& ModelConfigObject, const TSharedRef<FJsonObject>& TrainerConfigObject)
{
	UE::Learning::ETrainerResponse Response = UE::Learning::ETrainerResponse::Success;
	Response = Trainer->SendConfigs(DataConfigObject, ModelConfigObject, TrainerConfigObject);

	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Error sending config to trainer: {Response}. Check log for additional errors.", ("Name", GetName()), ("Response", UE::Learning::Trainer::GetResponseString(Response)));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}
}

void ULearningAgentsFlowMatchingTrainer::DoneTraining()
{
	if (!IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup not complete.", ("Name", GetName()));
		return;
	}

	if (IsTraining())
	{
		if (Trainer)
		{
			// Wait for Trainer to finish
			Trainer->Wait();

			// If not finished in time, terminate
			Trainer->Terminate();
		}
		bIsTraining = false;
	}
}

void ULearningAgentsFlowMatchingTrainer::EndTraining()
{
	if (!IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup not complete.", ("Name", GetName()));
		return;
	}

	if (IsTraining())
	{
		UE_LOGFMT(LogLearning, Display, "{Name}: Stopping training...", ("Name", GetName()));
		if (Trainer)
		{
			Trainer->SendStop();
		}
		DoneTraining();
	}
}

void ULearningAgentsFlowMatchingTrainer::IterateTraining()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsFlowMatchingTrainer::IterateTraining);

	if (!IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup not complete.", ("Name", GetName()));
		return;
	}

	if (!IsTraining())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Training not running.", ("Name", GetName()));
		return;
	}

	if (Trainer->HasNetworkOrCompleted())
	{
		UE_LOGFMT(LogLearning, Display, "Receiving trained networks...");

		UE::Learning::ETrainerResponse Response = Trainer->ReceiveNetwork(DenoiserNetworkId, *Model->GetDenoiserNetworkAsset()->NeuralNetworkData);
		if (Response == UE::Learning::ETrainerResponse::Completed)
		{
			UE_LOGFMT(LogLearning, Display, "{Name}: Trainer completed training.", ("Name", GetName()));
			DoneTraining();
			return;
		}
		else if (Response != UE::Learning::ETrainerResponse::Success)
		{
			UE_LOGFMT(LogLearning, Error, "{Name}: Error receiving Denoiser from trainer: {Response}. Check log for additional errors.", ("Name", GetName()), ("Response", UE::Learning::Trainer::GetResponseString(Response)));
			bHasTrainingFailed = true;
			EndTraining();
			return;
		}
		Model->GetDenoiserNetworkAsset()->ForceMarkDirty();

		Response = Trainer->ReceiveNetwork(EncoderNetworkId, *Model->GetEncoderNetworkAsset()->NeuralNetworkData);
		if (Response != UE::Learning::ETrainerResponse::Success)
		{
			UE_LOGFMT(LogLearning, Error, "{Name}: Error receiving encoder from trainer: {Response}. Check log for additional errors.", ("Name", GetName()), ("Response", UE::Learning::Trainer::GetResponseString(Response)));
			bHasTrainingFailed = true;
			EndTraining();
			return;
		}
		Model->GetEncoderNetworkAsset()->ForceMarkDirty();

		Response = Trainer->ReceiveNetwork(DecoderNetworkId, *Model->GetDecoderNetworkAsset()->NeuralNetworkData);
		if (Response != UE::Learning::ETrainerResponse::Success)
		{
			UE_LOGFMT(LogLearning, Error, "{Name}: Error receiving Decoder from trainer: {Response}. Check log for additional errors.", ("Name", GetName()), ("Response", UE::Learning::Trainer::GetResponseString(Response)));
			bHasTrainingFailed = true;
			EndTraining();
			return;
		}
		Model->GetDecoderNetworkAsset()->ForceMarkDirty();
	}
}

void ULearningAgentsFlowMatchingTrainer::RunTraining(
	const ULearningAgentsRecording* Recording,
	const FLearningAgentsFlowMatchingTrainerSettings& FlowMatchingTrainerSettings,
	const FLearningAgentsFlowMatchingTrainerTrainingSettings& FlowMatchingTrainerTrainingSettings,
	const FLearningAgentsTrainerProcessSettings& FlowMatchingTrainerPathSettings)
{
	if (!IsSetup())
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Setup not complete.", ("Name", GetName()));
		return;
	}

	if (bHasTrainingFailed)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Training has failed. Check log for errors.", ("Name", GetName()));

#if !WITH_EDITOR
		FGenericPlatformMisc::RequestExitWithStatus(false, 99);
#endif

		return;
	}

	// If we aren't training yet, then start training and do the first inference step.
	if (!IsTraining())
	{
		BeginTraining(
			Recording,
			FlowMatchingTrainerSettings,
			FlowMatchingTrainerTrainingSettings,
			FlowMatchingTrainerPathSettings);

		if (!IsTraining())
		{
			// If IsTraining is false, then BeginTraining must have failed and we can't continue.
			return;
		}
	}

	// Otherwise, do the regular training process.
	IterateTraining();
}

bool ULearningAgentsFlowMatchingTrainer::IsTraining() const
{
	return bIsTraining;
}

bool ULearningAgentsFlowMatchingTrainer::HasTrainingFailed() const
{
	return bHasTrainingFailed;
}
