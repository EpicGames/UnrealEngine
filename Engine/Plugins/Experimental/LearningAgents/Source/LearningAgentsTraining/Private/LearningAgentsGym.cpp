// Copyright Epic Games, Inc. All Rights Reserved.


#include "LearningAgentsGym.h"
#include "LearningLog.h"
#include "LearningAgentsEntityInterface.h"
#include "LearningAgentsLearningComponentInterface.h"
#include "Kismet/KismetMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsGym)

ALearningAgentsGymBase::ALearningAgentsGymBase()
{
	PrimaryActorTick.bCanEverTick = false;
	PrimaryActorTick.bStartWithTickEnabled = false;
}

void ALearningAgentsGymBase::Initialize()
{
	if (!RandomStream.IsValid())
	{
		RandomStream = MakeShareable(new FRandomStream);
		RandomStream->Initialize(RandomSeed);
	}
	PopulateLearningComponents();
	for (TScriptInterface<ILearningAgentsLearningComponentInterface>& LearningComponent : LearningComponents)
	{
		LearningComponent->InitializeLearningComponent();
	}
	
	OnGymInitialized.Broadcast();
}

void ALearningAgentsGymBase::Reset()
{
	OnBeginGymReset.Broadcast();

	for (TScriptInterface<ILearningAgentsLearningComponentInterface>& LearningComponent : LearningComponents)
	{
		LearningComponent->ResetLearningComponent();
	}

	OnPostGymReset.Broadcast();
}

void ALearningAgentsGymBase::GetRandomStream(FRandomStream& OutRandomStream) const
{
	OutRandomStream = *RandomStream;
}

TSharedPtr<FRandomStream> ALearningAgentsGymBase::GetRandomStream() const
{
	return RandomStream;
}

void ALearningAgentsGymBase::SetRandomStream(const TSharedPtr<FRandomStream>& InRandomStream)
{
	RandomStream = InRandomStream;
}

bool ALearningAgentsGymBase::IsMemberOfGym(TObjectPtr<AActor> Actor) const
{
	if (Actor and Actor->Implements<ULearningAgentsEntityInterface>())
	{
		return ILearningAgentsEntityInterface::Execute_GetGym(Actor) == this;
	}
	return false;
}

void ALearningAgentsGymBase::PopulateLearningComponents()
{
	TArray<TObjectPtr<UActorComponent>> AllComponents;
	this->GetComponents(AllComponents);

	for (TObjectPtr<UActorComponent> Component : AllComponents)
	{
		if (Component && Component->Implements<ULearningAgentsLearningComponentInterface>())
		{
			TScriptInterface<ILearningAgentsLearningComponentInterface> LearningComponentInterface = TScriptInterface<ILearningAgentsLearningComponentInterface>(Component);
			LearningComponents.Add(LearningComponentInterface);
		}
	}
}
