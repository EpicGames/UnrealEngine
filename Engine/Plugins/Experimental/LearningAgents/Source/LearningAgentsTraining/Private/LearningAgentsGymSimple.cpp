// Copyright Epic Games, Inc. All Rights Reserved.


#include "LearningAgentsGymSimple.h"
#include "LearningLog.h"
#include "Engine/World.h"
#include "NavigationSystem.h"
#include "AI/NavigationSystemBase.h"
#include "NavMesh/NavMeshBoundsVolume.h"
#include "Components/StaticMeshComponent.h"
#include "Kismet/KismetMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsGymSimple)

ALearningAgentsGymSimple::ALearningAgentsGymSimple()
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));
	SimpleGymFloor = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("SimpleGymFloor"));
	SimpleGymFloor->SetupAttachment(RootComponent);
}

void ALearningAgentsGymSimple::GetGymExtents(FVector& OutMinBounds, FVector& OutMaxBounds) const
{
	SimpleGymFloor->GetLocalBounds(OutMinBounds, OutMaxBounds);
	OutMinBounds *= SimpleGymFloor->GetComponentScale();
	OutMaxBounds *= SimpleGymFloor->GetComponentScale();
}

FRotator ALearningAgentsGymSimple::GenerateRandomRotationInGym() const
{
	check(RandomStream);

	return FRotator(0.0f, RandomStream->FRandRange(0.0f, 360.0f), 0.0f);
}

FVector ALearningAgentsGymSimple::GenerateRandomLocationInGym() const
{
	FVector MinBounds, MaxBounds;
	GetGymExtents(MinBounds, MaxBounds);

	// Find random point in radius taken from min axis of max bounds
	FVector RandomPoint = GetActorLocation() + RandomStream->VRand() * RandomStream->FRandRange(0.0f, FMath::Min(MaxBounds.X, MaxBounds.Y));
	RandomPoint.Z = GetActorLocation().Z;
	return ProjectPointToGym(RandomPoint);
}

FVector ALearningAgentsGymSimple::ProjectPointToGym(const FVector& InLocation) const
{
	FVector MinBounds, MaxBounds;
	GetGymExtents(MinBounds, MaxBounds);
	MaxBounds.Z += 100.0f;

	TObjectPtr<UWorld> World = GetWorld();
	
	check(World);
	TObjectPtr<UNavigationSystemV1> NavigationSystem = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
	
	if (NavigationSystem)
	{
		FNavLocation NavLocation;
		NavigationSystem->ProjectPointToNavigation(InLocation, NavLocation, MaxBounds);
		return NavLocation.Location;
	}
	else
	{
		return InLocation;
	}
}
