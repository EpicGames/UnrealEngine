// Copyright Epic Games, Inc. All Rights Reserved.


#include "LearningAgentsGymsManager.h"
#include "LearningAgentsGym.h"
#include "LearningLog.h"
#include "Engine/World.h"
#include "NavigationSystem.h"
#include "AI/NavigationSystemBase.h"
#include "Misc/CommandLine.h"
#include "Misc/DateTime.h"
#include "Math/UnrealMathUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsGymsManager)

ALearningAgentsGymsManager::ALearningAgentsGymsManager()
{
	PrimaryActorTick.bCanEverTick = false;
	PrimaryActorTick.bStartWithTickEnabled = false;
}

void ALearningAgentsGymsManager::SpawnGyms()
{
	InitializeRandomStream();

	FVector TemplateSpawnLocation = GetActorLocation();

	for (FSpawnGymInfo& GymTemplate : GymTemplates)
	{
		FString SpawnCountInput;
		if (FParse::Value(FCommandLine::Get(), TEXT("GymsCountOverride"), SpawnCountInput))
		{
			if (SpawnCountInput.StartsWith(TEXT("=")))
			{
				SpawnCountInput = SpawnCountInput.RightChop(1);
			}
			int32 SpawnCountOverride = FCString::Atoi(*SpawnCountInput);
			if (SpawnCountOverride > 0)
			{
				GymTemplate.SpawnCount = SpawnCountOverride;
				UE_LOGF(LogLearning, Log, "Commandline flag -GymsCountOverride is active! Generating %d Gyms for the template %ls.", GymTemplate.GetCount(), *GymTemplate.GymClass->GetName());
			}
			else
			{
				UE_LOGF(LogLearning, Log, "Commandline flag -GymsCountOverride was passed an invalid input. Generating %d Gyms for the template %ls.", GymTemplate.GetCount(), *GymTemplate.GymClass->GetName());
			}
		}

		FActorSpawnParameters SpawnParameters;
		SpawnParameters.Owner = this;

		int32 CurrentSpawnCount = 0;
		int32 GridSize = FMath::CeilToInt32(FMath::Sqrt(static_cast<float>(GymTemplate.GetCount())));
		bool bFirstGym = true;

		FVector MinBounds = FVector::ZeroVector;
		FVector MaxBounds = FVector::ZeroVector;
		FName GymTag;

		for (int32 Row = 0; Row < GridSize; ++Row)
		{
			for (int32 Col = 0; Col < GridSize; ++Col)
			{
				if (CurrentSpawnCount >= GymTemplate.GetCount())
				{
					break;
				}

				TObjectPtr<ALearningAgentsGymBase> Gym;

				if (bFirstGym)
				{
					bFirstGym = false;
					Gym = GetWorld()->SpawnActor<ALearningAgentsGymBase>(GymTemplate.GymClass, TemplateSpawnLocation, FRotator::ZeroRotator, SpawnParameters);
					Gym->GetGymExtents(MinBounds, MaxBounds);
				}
				else
				{
					FVector SpawnLocation = FVector(
						Row * (GymsSpacing + (MaxBounds.X - MinBounds.X)) + TemplateSpawnLocation.X, 
						Col * (GymsSpacing + (MaxBounds.Y - MinBounds.Y)) + TemplateSpawnLocation.Y, 
						TemplateSpawnLocation.Z);
					Gym = GetWorld()->SpawnActor<ALearningAgentsGymBase>(GymTemplate.GymClass, SpawnLocation, FRotator::ZeroRotator, SpawnParameters);
				}

				SpawnedGyms.Add(Gym);
				
				check(RandomStream.IsValid());
				Gym->SetRandomStream(RandomStream);

				CurrentSpawnCount++;
			}
		}
		if (GridSize > 0)
		{
			TemplateSpawnLocation.X += (GymsSpacing + (MaxBounds.X - MinBounds.X)) * FMath::CeilToInt(static_cast<float>(GymTemplate.GetCount()) / GridSize);
		}
	}
	
	if (TObjectPtr<UNavigationSystemV1> NavigationSystem = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld()))
	{
		NavigationSystem->Build();
	}
	else
	{
		UE_LOGF(LogLearning, Log, "Unable to build navigation system due to invalid navigation system reference!");
	}
}

int32 ALearningAgentsGymsManager::GetGymsCount() const
{
	FString InputString;
	if (FParse::Value(FCommandLine::Get(), TEXT("GymsCountOverride"), InputString))
	{
		if (InputString.StartsWith(TEXT("=")))
		{
			InputString = InputString.RightChop(1);
		}
		int32 SpawnCountOverride = FCString::Atoi(*InputString);
		if (SpawnCountOverride > 0)
		{
			return SpawnCountOverride * GymTemplates.Num();
		}
	}

	int32 Count = 0;
	for (const FSpawnGymInfo& GymTemplate : GymTemplates)
	{
		Count += GymTemplate.GetCount();
	}
	return Count;
}

void ALearningAgentsGymsManager::InitializeRandomStream()
{
	FString RandomSeedInput;
	if (FParse::Value(FCommandLine::Get(), TEXT("GymsManagerRandomSeed"), RandomSeedInput))
	{
		if (RandomSeedInput.StartsWith(TEXT("=")))
		{
			RandomSeedInput = RandomSeedInput.RightChop(1);
			UE_LOGF(LogLearning, Log, "Removed leading '=' from RandomSeedInput: %ls", *RandomSeedInput);
		}
		RandomSeed = FCString::Atoi(*RandomSeedInput);
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("GymsManagerRandomizeNoSeed")))
	{
		RandomSeed = FDateTime::Now().GetTicks();
		UE_LOGF(LogLearning, Log, "Commandline flag -GymsManagerRandomizeNoSeed is active! Using randomly generated seed from timestamp: %d.", RandomSeed);
	}
	RandomStream = MakeShareable(new FRandomStream);
	RandomStream->Initialize(RandomSeed);
	UE_LOGF(LogLearning, Log, "GymsManager initialized with seed %d!", RandomSeed);
}

void ALearningAgentsGymsManager::Start()
{
	SpawnGyms();
	for (TObjectPtr<ALearningAgentsGymBase>& SpawnedGym : SpawnedGyms)
	{
		SpawnedGym->Initialize();
	}
}
