// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsImitationTrainer.h"

#include "LearningAgentsCommunicator.h"
#include "LearningAgentsInteractor.h"
#include "LearningAgentsManager.h"
#include "LearningAgentsPolicy.h"
#include "LearningAgentsRecording.h"

#include "LearningExperience.h"
#include "LearningExternalTrainer.h"
#include "LearningLog.h"
#include "LearningNeuralNetwork.h"
#include "LearningTrainer.h"

#include "Dom/JsonObject.h"
#include "HAL/PlatformMisc.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsImitationTrainer)

TSharedRef<FJsonObject> FLearningAgentsImitationTrainerTrainingSettings::AsJsonConfig() const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	ConfigObject->SetNumberField(TEXT("IterationNum"), Hyperparameters.NumberOfIterations);
	ConfigObject->SetNumberField(TEXT("LearningRate"), Hyperparameters.LearningRate);
	ConfigObject->SetNumberField(TEXT("LearningRateDecay"), Hyperparameters.LearningRateDecay);
	ConfigObject->SetNumberField(TEXT("LearningRateDecayStepSize"), Hyperparameters.LearningRateDecayStepSize);
	ConfigObject->SetNumberField(TEXT("WeightDecay"), Hyperparameters.WeightDecay);
	ConfigObject->SetNumberField(TEXT("BatchSize"), Hyperparameters.BatchSize);
	ConfigObject->SetNumberField(TEXT("Window"), Hyperparameters.Window);
	ConfigObject->SetNumberField(TEXT("ActionRegularizationWeight"), Hyperparameters.ActionRegularizationWeight);
	ConfigObject->SetNumberField(TEXT("ActionEntropyWeight"), Hyperparameters.ActionEntropyWeight);
	ConfigObject->SetNumberField(TEXT("ObservationNoiseScale"), Hyperparameters.ObservationNoiseScale);
	ConfigObject->SetNumberField(TEXT("Seed"), Hyperparameters.RandomSeed);
	ConfigObject->SetBoolField(TEXT("SaveSnapshots"), LoggingSettings.bSaveSnapshots);
	ConfigObject->SetNumberField(TEXT("IterationsPerSnapshot"), LoggingSettings.IterationsPerSnapshot);
	// Eval
	ConfigObject->SetBoolField(TEXT("RunEvaluation"), bRunEvaluation);
	ConfigObject->SetNumberField(TEXT("TrainEvalDatasetSplit"), TrainEvalDatasetSplit);
	ConfigObject->SetNumberField(TEXT("EvaluationFrequency"), EvaluationFrequency);
	ConfigObject->SetNumberField(TEXT("BatchCountPerEvaluation"), BatchCountPerEvaluation);

	return ConfigObject;
}

ULearningAgentsImitationTrainer::ULearningAgentsImitationTrainer() : Super(FObjectInitializer::Get()) {}
ULearningAgentsImitationTrainer::ULearningAgentsImitationTrainer(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsImitationTrainer::~ULearningAgentsImitationTrainer() = default;

void ULearningAgentsImitationTrainer::BeginDestroy()
{
	if (IsTraining())
	{
		EndTraining();
	}

	Super::BeginDestroy();
}

ULearningAgentsImitationTrainer* ULearningAgentsImitationTrainer::MakeImitationTrainer(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	ULearningAgentsPolicy*& InPolicy,
	const FLearningAgentsCommunicator& Communicator,
	TSubclassOf<ULearningAgentsImitationTrainer> Class,
	const FName Name)
{
	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "MakeImitationTrainer: InManager is nullptr.");
		return nullptr;
	}

	if (!Class)
	{
		UE_LOGF(LogLearning, Error, "MakeImitationTrainer: Class is nullptr.");
		return nullptr;
	}

	const FName UniqueName = MakeUniqueObjectName(InManager, Class, Name, EUniqueObjectNameOptions::GloballyUnique);

	ULearningAgentsImitationTrainer* ImitationTrainer = NewObject<ULearningAgentsImitationTrainer>(InManager, Class, UniqueName);
	if (!ImitationTrainer) { return nullptr; }

	ImitationTrainer->SetupImitationTrainer(InManager, InInteractor, InPolicy, Communicator);

	return ImitationTrainer->IsSetup() ? ImitationTrainer : nullptr;
}

void ULearningAgentsImitationTrainer::SetupImitationTrainer(
	ULearningAgentsManager* InManager,
	ULearningAgentsInteractor* InInteractor,
	ULearningAgentsPolicy* InPolicy,
	const FLearningAgentsCommunicator& Communicator)
{
	if (IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup already performed!", *GetName());
		return;
	}

	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "%ls: InManager is nullptr.", *GetName());
		return;
	}

	if (!InInteractor)
	{
		UE_LOGF(LogLearning, Error, "%ls: InInteractor is nullptr.", *GetName());
		return;
	}

	if (!InInteractor->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InInteractor->GetName());
		return;
	}

	if (!InPolicy)
	{
		UE_LOGF(LogLearning, Error, "%ls: InPolicy is nullptr.", *GetName());
		return;
	}

	if (!InPolicy->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InPolicy->GetName());
		return;
	}

	if (!Communicator.Trainer)
	{
		UE_LOGF(LogLearning, Error, "%ls: Communicator's Trainer is nullptr.", *GetName());
		return;
	}

	Manager = InManager;
	Interactor = InInteractor;
	Policy = InPolicy;
	Trainer = Communicator.Trainer;

	bIsSetup = true;

	Manager->AddListener(this);
}

void ULearningAgentsImitationTrainer::BeginTraining(
	const ULearningAgentsRecording* Recording,
	const FLearningAgentsImitationTrainerSettings& ImitationTrainerSettings,
	const FLearningAgentsImitationTrainerTrainingSettings& ImitationTrainerTrainingSettings,
	const FLearningAgentsTrainerProcessSettings& ImitationTrainerPathSettings)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (IsTraining())
	{
		UE_LOGF(LogLearning, Error, "%ls: Cannot begin training as we are already training!", *GetName());
		return;
	}

	if (!Recording)
	{
		UE_LOGF(LogLearning, Error, "%ls: Recording is nullptr.", *GetName());
		return;
	}

	if (Recording->Records.IsEmpty())
	{
		UE_LOGF(LogLearning, Error, "%ls: Recording is empty!", *GetName());
		return;
	}

	// Build Replay Buffer from Recording
	ReplayBuffer = Recording->BuildReplayBuffer(
		Interactor,
		ImitationTrainerSettings.RecordingFilterTag,
		ImitationTrainerTrainingSettings.ValidationSettings.bCheckObservationSchemaCompatibility,
		ImitationTrainerTrainingSettings.ValidationSettings.bCheckActionSchemaCompatibility);

	if (!ReplayBuffer)
	{
		UE_LOGF(LogLearning, Warning, "%ls: Recording contains no valid training data.", *GetName());
		return;
	}

	// We need to setup the trainer prior to sending the config
	PolicyNetworkId = Trainer->AddNetwork(*Policy->GetPolicyNetworkAsset()->NeuralNetworkData);
	EncoderNetworkId = Trainer->AddNetwork(*Policy->GetEncoderNetworkAsset()->NeuralNetworkData);
	DecoderNetworkId = Trainer->AddNetwork(*Policy->GetDecoderNetworkAsset()->NeuralNetworkData);
	ReplayBufferId = Trainer->AddReplayBuffer(*ReplayBuffer);

	TSharedRef<FJsonObject> DataConfigObject = CreateDataConfig();
	TSharedRef<FJsonObject> ModelConfigObject = CreateModelConfig();
	TSharedRef<FJsonObject> TrainerConfigObject = CreateTrainerConfig(ImitationTrainerTrainingSettings);

	UE_LOGF(LogLearning, Display, "%ls: Sending configs...", *GetName());
	SendConfigs(DataConfigObject, ModelConfigObject, TrainerConfigObject);

	UE_LOGF(LogLearning, Display, "%ls: Imitation Training Started", *GetName());
	UE::Learning::ETrainerResponse Response = UE::Learning::ETrainerResponse::Success;

	UE_LOGF(LogLearning, Display, "%ls: Sending / Receiving initial policy...", *GetName());
	Response = Trainer->SendNetwork(PolicyNetworkId, *Policy->GetPolicyNetworkAsset()->NeuralNetworkData);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "%ls: Error sending policy to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	Response = Trainer->SendNetwork(EncoderNetworkId, *Policy->GetEncoderNetworkAsset()->NeuralNetworkData);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "%ls: Error sending encoder to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	Response = Trainer->SendNetwork(DecoderNetworkId, *Policy->GetDecoderNetworkAsset()->NeuralNetworkData);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "%ls: Error sending decoder to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	UE_LOGF(LogLearning, Display, "%ls: Sending Experience...", *GetName());
	Response = Trainer->SendReplayBuffer(ReplayBufferId, *ReplayBuffer);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "%ls: Error sending experience to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	bIsTraining = true;
}

void ULearningAgentsImitationTrainer::ExportToFile(
	UE::Learning::FFileTrainer& FileTrainer,
	const ULearningAgentsRecording* Recording,
	const FLearningAgentsImitationTrainerSettings& ImitationTrainerSettings,
	const FLearningAgentsImitationTrainerTrainingSettings& ImitationTrainerTrainingSettings,
	UE::Learning::ETrainerExportFlags ExportFlags)
{
	using namespace UE::Learning;

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (IsTraining())
	{
		UE_LOGF(LogLearning, Error, "%ls: Cannot export while training is active.", *GetName());
		return;
	}

	const bool bNeedReplayBuffers = EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::ReplayBuffers);
	const bool bNeedNetworks = EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::Networks);
	const bool bNeedTrainerConfig = EnumHasAnyFlags(ExportFlags, ETrainerExportFlags::TrainerConfig);

	// Networks: register to get IDs (cheap)
	if (bNeedNetworks)
	{
		PolicyNetworkId = FileTrainer.AddNetwork(*Policy->GetPolicyNetworkAsset()->NeuralNetworkData);
		EncoderNetworkId = FileTrainer.AddNetwork(*Policy->GetEncoderNetworkAsset()->NeuralNetworkData);
		DecoderNetworkId = FileTrainer.AddNetwork(*Policy->GetDecoderNetworkAsset()->NeuralNetworkData);
	}

	// Replay buffers: build from recording (expensive)
	if (bNeedReplayBuffers)
	{
		if (!Recording)
		{
			UE_LOGF(LogLearning, Error, "%ls: Recording is nullptr.", *GetName());
			return;
		}

		if (Recording->Records.IsEmpty())
		{
			UE_LOGF(LogLearning, Error, "%ls: Recording is empty!", *GetName());
			return;
		}

		ReplayBuffer = Recording->BuildReplayBuffer(
			Interactor,
			ImitationTrainerSettings.RecordingFilterTag,
			ImitationTrainerTrainingSettings.ValidationSettings.bCheckObservationSchemaCompatibility,
			ImitationTrainerTrainingSettings.ValidationSettings.bCheckActionSchemaCompatibility);

		if (!ReplayBuffer)
		{
			UE_LOGF(LogLearning, Warning, "%ls: Recording contains no valid training data.", *GetName());
			return;
		}

		ReplayBufferId = FileTrainer.AddReplayBuffer(*ReplayBuffer);
	}

	// Build and send configs
	TSharedRef<FJsonObject> DataConfigObject = bNeedReplayBuffers ? CreateDataConfig() : MakeShared<FJsonObject>();
	TSharedRef<FJsonObject> ModelConfigObject = bNeedNetworks ? CreateModelConfig() : MakeShared<FJsonObject>();
	TSharedRef<FJsonObject> TrainerConfigObject = bNeedTrainerConfig ? CreateTrainerConfig(ImitationTrainerTrainingSettings) : MakeShared<FJsonObject>();
	FileTrainer.SendConfigs(DataConfigObject, ModelConfigObject, TrainerConfigObject);

	// Send network weights
	if (bNeedNetworks)
	{
		FileTrainer.SendNetwork(PolicyNetworkId, *Policy->GetPolicyNetworkAsset()->NeuralNetworkData);
		FileTrainer.SendNetwork(EncoderNetworkId, *Policy->GetEncoderNetworkAsset()->NeuralNetworkData);
		FileTrainer.SendNetwork(DecoderNetworkId, *Policy->GetDecoderNetworkAsset()->NeuralNetworkData);
	}

	// Send replay buffer data
	if (bNeedReplayBuffers)
	{
		FileTrainer.SendReplayBuffer(ReplayBufferId, *ReplayBuffer);
	}

	UE_LOGF(LogLearning, Display, "%ls: Export complete.", *GetName());
}

TSharedRef<FJsonObject> ULearningAgentsImitationTrainer::CreateDataConfig() const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	const int32 ObservationSchemaId = 0;
	const int32 ActionSchemaId = 0;

	// Add Replay Buffers Config Entries
	TArray<TSharedPtr<FJsonValue>> ReplayBufferObjects;
	TSharedRef<FJsonValueObject> ReplayBufferJsonValue = MakeShared<FJsonValueObject>(ReplayBuffer->AsJsonConfig(ReplayBufferId));
	ReplayBufferObjects.Add(ReplayBufferJsonValue);
	ConfigObject->SetArrayField(TEXT("ReplayBuffers"), ReplayBufferObjects);

	// Schemas
	TSharedPtr<FJsonObject> SchemasObject = MakeShared<FJsonObject>();

	// Add the observation schemas
	TArray<TSharedPtr<FJsonValue>> ObservationSchemaObjects;
	{
		// For this trainer, add the one observation schema we have
		TSharedPtr<FJsonObject> ObservationSchemaObject = MakeShared<FJsonObject>();
		ObservationSchemaObject->SetNumberField(TEXT("Id"), ObservationSchemaId);
		ObservationSchemaObject->SetStringField(TEXT("Name"), "Default");
		ObservationSchemaObject->SetObjectField(TEXT("Schema"),
			UE::Learning::Trainer::ConvertObservationSchemaToJSON(Interactor->GetObservationSchema()->ObservationSchema,
				Interactor->GetObservationSchemaElement().SchemaElement));

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(ObservationSchemaObject);
		ObservationSchemaObjects.Add(JsonValue);
	}
	SchemasObject->SetArrayField(TEXT("Observations"), ObservationSchemaObjects);

	// Add the action schemas
	TArray<TSharedPtr<FJsonValue>> ActionSchemaObjects;
	{
		// For this trainer, add the one action schema we have
		TSharedPtr<FJsonObject> ActionSchemaObject = MakeShared<FJsonObject>();
		ActionSchemaObject->SetNumberField(TEXT("Id"), ActionSchemaId);
		ActionSchemaObject->SetStringField(TEXT("Name"), "Default");
		ActionSchemaObject->SetObjectField(TEXT("Schema"),
			UE::Learning::Trainer::ConvertActionSchemaToJSON(Interactor->GetActionSchema()->ActionSchema,
				Interactor->GetActionSchemaElement().SchemaElement));

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(ActionSchemaObject);
		ActionSchemaObjects.Add(JsonValue);
	}
	SchemasObject->SetArrayField(TEXT("Actions"), ActionSchemaObjects);

	ConfigObject->SetObjectField(TEXT("Schemas"), SchemasObject);

	return ConfigObject;
}

TSharedRef<FJsonObject> ULearningAgentsImitationTrainer::CreateModelConfig() const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	const int32 ObservationSchemaId = 0;
	const int32 ActionSchemaId = 0;

	// Add Neural Network Config Entries
	TArray<TSharedPtr<FJsonValue>> NetworkObjects;

	// Policy
	{
		TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
		NetworkObject->SetNumberField(TEXT("Id"), PolicyNetworkId);
		NetworkObject->SetStringField(TEXT("Name"), Policy->GetPolicyNetworkAsset()->GetFName().ToString());
		NetworkObject->SetNumberField(TEXT("MaxByteNum"), Policy->GetPolicyNetworkAsset()->NeuralNetworkData->GetSnapshotByteNum());

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
		NetworkObjects.Add(JsonValue);
	}

	// Encoder
	{
		TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
		NetworkObject->SetNumberField(TEXT("Id"), EncoderNetworkId);
		NetworkObject->SetStringField(TEXT("Name"), Policy->GetEncoderNetworkAsset()->GetFName().ToString());
		NetworkObject->SetNumberField(TEXT("MaxByteNum"), Policy->GetEncoderNetworkAsset()->NeuralNetworkData->GetSnapshotByteNum());
		NetworkObject->SetNumberField(TEXT("InputSchemaId"), ObservationSchemaId);

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
		NetworkObjects.Add(JsonValue);
	}

	// Decoder
	{
		TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
		NetworkObject->SetNumberField(TEXT("Id"), DecoderNetworkId);
		NetworkObject->SetStringField(TEXT("Name"), Policy->GetDecoderNetworkAsset()->GetFName().ToString());
		NetworkObject->SetNumberField(TEXT("MaxByteNum"), Policy->GetDecoderNetworkAsset()->NeuralNetworkData->GetSnapshotByteNum());
		NetworkObject->SetNumberField(TEXT("OutputSchemaId"), ActionSchemaId);

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
		NetworkObjects.Add(JsonValue);
	}

	ConfigObject->SetArrayField(TEXT("Networks"), NetworkObjects);

	ConfigObject->SetObjectField(TEXT("PolicySettings"), Policy->AsJsonConfig());

	return ConfigObject;
}

TSharedRef<FJsonObject> ULearningAgentsImitationTrainer::CreateTrainerConfig(const FLearningAgentsImitationTrainerTrainingSettings& TrainingSettings) const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	// Add Training Config Entries
	ConfigObject->SetStringField(TEXT("TrainerMethod"), TEXT("BehaviorCloning"));
	ConfigObject->SetStringField(TEXT("TimeStamp"), *FDateTime::Now().ToFormattedString(TEXT("%Y-%m-%d_%H-%M-%S")));

	// Add Imitation Specific Config Entries
	ConfigObject->SetObjectField(TEXT("BehaviorCloningSettings"), TrainingSettings.AsJsonConfig());

	// Add Normalization Choice
	ConfigObject->SetBoolField(TEXT("EmbedNormalizationsInWeights"), TrainingSettings.bEmbedNormalizationsInWeights);

	// Add Training Workflow Config Entries
	ConfigObject->SetStringField(TEXT("Device"), UE::Learning::Trainer::GetDeviceString(UE::Learning::Agents::GetTrainingDevice(TrainingSettings.Device)));
	ConfigObject->SetBoolField(TEXT("UseTensorBoard"), TrainingSettings.LoggingSettings.bUseTensorboard);
	ConfigObject->SetBoolField(TEXT("UseMLflow"), TrainingSettings.LoggingSettings.bUseMLflow);
	ConfigObject->SetStringField(TEXT("MLflowTrackingUri"), TrainingSettings.LoggingSettings.MLflowTrackingUri);

	return ConfigObject;
}

void ULearningAgentsImitationTrainer::SendConfigs(const TSharedRef<FJsonObject>& DataConfigObject, const TSharedRef<FJsonObject>& ModelConfigObject, const TSharedRef<FJsonObject>& TrainerConfigObject)
{
	UE::Learning::ETrainerResponse Response = UE::Learning::ETrainerResponse::Success;
	Response = Trainer->SendConfigs(DataConfigObject, ModelConfigObject, TrainerConfigObject);

	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "%ls: Error sending config to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}
}

void ULearningAgentsImitationTrainer::DoneTraining()
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (IsTraining())
	{
		if (Trainer)
		{
			// Wait for Trainer to finish
			Trainer->Wait();

			// If not finished in time, terminate
			Trainer->Terminate();
		}
		bIsTraining = false;
	}
}

void ULearningAgentsImitationTrainer::EndTraining()
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (IsTraining())
	{
		UE_LOGF(LogLearning, Display, "%ls: Stopping training...", *GetName());
		if (Trainer)
		{
			Trainer->SendStop();
		}
		DoneTraining();
	}
}

void ULearningAgentsImitationTrainer::IterateTraining()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsImitationTrainer::IterateTraining);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (!IsTraining())
	{
		UE_LOGF(LogLearning, Error, "%ls: Training not running.", *GetName());
		return;
	}

	if (Trainer->HasNetworkOrCompleted())
	{
		UE_LOGF(LogLearning, Display, "Receiving trained networks...");

		UE::Learning::ETrainerResponse Response = Trainer->ReceiveNetwork(PolicyNetworkId, *Policy->GetPolicyNetworkAsset()->NeuralNetworkData);
		if (Response == UE::Learning::ETrainerResponse::Completed)
		{
			UE_LOGF(LogLearning, Display, "%ls: Trainer completed training.", *GetName());
			DoneTraining();
			return;
		}
		else if (Response != UE::Learning::ETrainerResponse::Success)
		{
			UE_LOGF(LogLearning, Error, "%ls: Error receiving policy from trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
			bHasTrainingFailed = true;
			EndTraining();
			return;
		}
		Policy->GetPolicyNetworkAsset()->ForceMarkDirty();

		Response = Trainer->ReceiveNetwork(EncoderNetworkId, *Policy->GetEncoderNetworkAsset()->NeuralNetworkData);
		if (Response != UE::Learning::ETrainerResponse::Success)
		{
			UE_LOGF(LogLearning, Error, "%ls: Error receiving encoder from trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
			bHasTrainingFailed = true;
			EndTraining();
			return;
		}
		Policy->GetEncoderNetworkAsset()->ForceMarkDirty();

		Response = Trainer->ReceiveNetwork(DecoderNetworkId, *Policy->GetDecoderNetworkAsset()->NeuralNetworkData);
		if (Response != UE::Learning::ETrainerResponse::Success)
		{
			UE_LOGF(LogLearning, Error, "%ls: Error receiving decoder from trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
			bHasTrainingFailed = true;
			EndTraining();
			return;
		}
	}
}

void ULearningAgentsImitationTrainer::RunTraining(
	const ULearningAgentsRecording* Recording,
	const FLearningAgentsImitationTrainerSettings& ImitationTrainerSettings,
	const FLearningAgentsImitationTrainerTrainingSettings& ImitationTrainerTrainingSettings,
	const FLearningAgentsTrainerProcessSettings& ImitationTrainerPathSettings)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (bHasTrainingFailed)
	{
		UE_LOGF(LogLearning, Error, "%ls: Training has failed. Check log for errors.", *GetName());

#if !WITH_EDITOR
		FGenericPlatformMisc::RequestExitWithStatus(false, 99);
#endif

		return;
	}

	// If we aren't training yet, then start training and do the first inference step.
	if (!IsTraining())
	{
		BeginTraining(
			Recording,
			ImitationTrainerSettings,
			ImitationTrainerTrainingSettings,
			ImitationTrainerPathSettings);

		if (!IsTraining())
		{
			// If IsTraining is false, then BeginTraining must have failed and we can't continue.
			return;
		}
	}

	// Otherwise, do the regular training process.
	IterateTraining();
}

bool ULearningAgentsImitationTrainer::IsTraining() const
{
	return bIsTraining;
}

bool ULearningAgentsImitationTrainer::HasTrainingFailed() const
{
	return bHasTrainingFailed;
}
