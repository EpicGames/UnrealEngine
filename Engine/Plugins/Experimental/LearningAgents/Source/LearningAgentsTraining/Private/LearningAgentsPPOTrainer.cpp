// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsPPOTrainer.h"

#include "LearningAgentsCritic.h"
#include "LearningAgentsInteractor.h"
#include "LearningAgentsManager.h"
#include "LearningAgentsPolicy.h"
#include "LearningCompletion.h"
#include "LearningExperience.h"
#include "LearningLog.h"
#include "LearningNeuralNetwork.h"
#include "LearningTrainer.h"
#include "LearningAgentsCommunicator.h"
#include "LearningAgentsTrainingEnvironment.h"
#include "LearningExternalTrainer.h"

#include "Dom/JsonObject.h"
#include "HAL/PlatformMisc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsPPOTrainer)

ULearningAgentsPPOTrainer::ULearningAgentsPPOTrainer() : Super(FObjectInitializer::Get()) {}
ULearningAgentsPPOTrainer::ULearningAgentsPPOTrainer(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsPPOTrainer::~ULearningAgentsPPOTrainer() = default;

TSharedRef<FJsonObject> FLearningAgentsPPOTrainingSettings::AsJsonConfig() const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	ConfigObject->SetNumberField(TEXT("IterationNum"), NumberOfIterations);
	ConfigObject->SetNumberField(TEXT("LearningRatePolicy"), LearningRatePolicy);
	ConfigObject->SetNumberField(TEXT("LearningRateCritic"), LearningRateCritic);
	ConfigObject->SetNumberField(TEXT("LearningRateDecay"), LearningRateDecay);
	ConfigObject->SetNumberField(TEXT("WeightDecay"), WeightDecay);
	ConfigObject->SetNumberField(TEXT("PolicyBatchSize"), PolicyBatchSize);
	ConfigObject->SetNumberField(TEXT("CriticBatchSize"), CriticBatchSize);
	ConfigObject->SetNumberField(TEXT("PolicyWindow"), PolicyWindowSize);
	ConfigObject->SetNumberField(TEXT("IterationsPerGather"), IterationsPerGather);
	ConfigObject->SetNumberField(TEXT("CriticWarmupIterations"), CriticWarmupIterations);
	ConfigObject->SetNumberField(TEXT("EpsilonClip"), EpsilonClip);
	ConfigObject->SetNumberField(TEXT("ActionSurrogateWeight"), ActionSurrogateWeight);
	ConfigObject->SetNumberField(TEXT("ActionRegularizationWeight"), ActionRegularizationWeight);
	ConfigObject->SetNumberField(TEXT("ActionEntropyWeight"), ActionEntropyWeight);
	ConfigObject->SetNumberField(TEXT("ReturnRegularizationWeight"), ReturnRegularizationWeight);
	ConfigObject->SetNumberField(TEXT("GaeLambda"), GaeLambda);
	ConfigObject->SetBoolField(TEXT("AdvantageNormalization"), bAdvantageNormalization);
	ConfigObject->SetNumberField(TEXT("AdvantageMin"), MinimumAdvantage);
	ConfigObject->SetNumberField(TEXT("AdvantageMax"), MaximumAdvantage);
	ConfigObject->SetBoolField(TEXT("UseGradNormMaxClipping"), bUseGradNormMaxClipping);
	ConfigObject->SetNumberField(TEXT("GradNormMax"), GradNormMax);
	ConfigObject->SetNumberField(TEXT("TrimEpisodeStartStepNum"), NumberOfStepsToTrimAtStartOfEpisode);
	ConfigObject->SetNumberField(TEXT("TrimEpisodeEndStepNum"), NumberOfStepsToTrimAtEndOfEpisode);
	ConfigObject->SetNumberField(TEXT("Seed"), RandomSeed);
	ConfigObject->SetNumberField(TEXT("DiscountFactor"), DiscountFactor);
	ConfigObject->SetBoolField(TEXT("SaveSnapshots"), bSaveSnapshots);
	ConfigObject->SetNumberField(TEXT("IterationsPerSnapshot"), IterationsPerSnapshot);

	return ConfigObject;
}

void ULearningAgentsPPOTrainer::BeginDestroy()
{
	if (IsTraining())
	{
		EndTraining();
	}

	Super::BeginDestroy();
}

ULearningAgentsPPOTrainer* ULearningAgentsPPOTrainer::MakePPOTrainer(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	ULearningAgentsTrainingEnvironment*& InTrainingEnvironment,
	ULearningAgentsPolicy*& InPolicy,
	ULearningAgentsCritic*& InCritic,
	const FLearningAgentsCommunicator& Communicator,
	TSubclassOf<ULearningAgentsPPOTrainer> Class,
	const FName Name,
	const FLearningAgentsPPOTrainerSettings& TrainerSettings)
{
	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "MakePPOTrainer: InManager is nullptr.");
		return nullptr;
	}

	if (!Class)
	{
		UE_LOGF(LogLearning, Error, "MakePPOTrainer: Class is nullptr.");
		return nullptr;
	}

	const FName UniqueName = MakeUniqueObjectName(InManager, Class, Name, EUniqueObjectNameOptions::GloballyUnique);

	ULearningAgentsPPOTrainer* PPOTrainer = NewObject<ULearningAgentsPPOTrainer>(InManager, Class, UniqueName);
	if (!PPOTrainer) { return nullptr; }

	PPOTrainer->SetupPPOTrainer(InManager, InInteractor, InTrainingEnvironment, InPolicy, InCritic, Communicator, TrainerSettings);

	return PPOTrainer->IsSetup() ? PPOTrainer : nullptr;
}

void ULearningAgentsPPOTrainer::SetupPPOTrainer(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	ULearningAgentsTrainingEnvironment*& InTrainingEnvironment,
	ULearningAgentsPolicy*& InPolicy,
	ULearningAgentsCritic*& InCritic,
	const FLearningAgentsCommunicator& Communicator,
	const FLearningAgentsPPOTrainerSettings& TrainerSettings)
{
	if (IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup already run!", *GetName());
		return;
	}

	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "%ls: InManager is nullptr.", *GetName());
		return;
	}

	if (!InInteractor)
	{
		UE_LOGF(LogLearning, Error, "%ls: InInteractor is nullptr.", *GetName());
		return;
	}

	if (!InInteractor->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InInteractor->GetName());
		return;
	}

	if (!InTrainingEnvironment)
	{
		UE_LOGF(LogLearning, Error, "%ls: InTrainingEnvironment is nullptr.", *GetName());
		return;
	}

	if (!InTrainingEnvironment->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InTrainingEnvironment->GetName());
		return;
	}

	if (!InPolicy)
	{
		UE_LOGF(LogLearning, Error, "%ls: InPolicy is nullptr.", *GetName());
		return;
	}

	if (!InPolicy->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InPolicy->GetName());
		return;
	}

	if (!InCritic)
	{
		UE_LOGF(LogLearning, Error, "%ls: InCritic is nullptr.", *GetName());
		return;
	}

	if (!InCritic->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InCritic->GetName());
		return;
	}

	if (!Communicator.Trainer || !Communicator.Trainer->IsValid())
	{
		UE_LOGF(LogLearning, Error, "%ls: Communicator's Trainer is nullptr.", *GetName());
		return;
	}

	Manager = InManager;
	Interactor = InInteractor;
	Policy = InPolicy;
	Critic = InCritic;
	TrainingEnvironment = InTrainingEnvironment;
	Trainer = Communicator.Trainer;

	const int32 ObservationSchemaId = 0;
	const int32 ActionSchemaId = 0;

	// Create Episode Buffer
	EpisodeBuffer = MakeUnique<UE::Learning::FEpisodeBuffer>();
	EpisodeBuffer->Resize(Manager->GetMaxAgentNum(), TrainerSettings.MaxEpisodeStepNum);
	ObservationId = EpisodeBuffer->AddObservations(
		TEXT("Observations"),
		ObservationSchemaId,
		Interactor->GetObservationVectorSize());
	ActionId = EpisodeBuffer->AddActions(
		TEXT("Actions"),
		ActionSchemaId,
		Interactor->GetActionVectorSize());
	ActionModifierId = EpisodeBuffer->AddActionModifiers(
		TEXT("ActionModifiers"),
		ActionSchemaId,
		Interactor->GetActionModifierVectorSize());
	MemoryStateId = EpisodeBuffer->AddMemoryStates(
		TEXT("MemoryStates"),
		Policy->GetMemoryStateSize());
	RewardId = EpisodeBuffer->AddRewards(
		TEXT("Rewards"),
		1);

	// Create Replay Buffer
	ReplayBuffer = MakeUnique<UE::Learning::FReplayBuffer>();
	ReplayBuffer->Resize(
		*EpisodeBuffer,
		TrainerSettings.MaximumRecordedEpisodesPerIteration,
		TrainerSettings.MaximumRecordedStepsPerIteration);

	bIsSetup = true;

	Manager->AddListener(this);
}

void ULearningAgentsPPOTrainer::OnAgentsAdded_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	EpisodeBuffer->Reset(AgentIds);
}

void ULearningAgentsPPOTrainer::OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	EpisodeBuffer->Reset(AgentIds);
}

void ULearningAgentsPPOTrainer::OnAgentsReset_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	EpisodeBuffer->Reset(AgentIds);
}

const bool ULearningAgentsPPOTrainer::IsTraining() const
{
	return bIsTraining;
}

void ULearningAgentsPPOTrainer::BeginTraining(
	const FLearningAgentsPPOTrainingSettings& TrainingSettings,
	const FLearningAgentsTrainingGameSettings& TrainingGameSettings,
	const bool bResetAgentsOnBegin)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (IsTraining())
	{
		UE_LOGF(LogLearning, Error, "%ls: Already Training!", *GetName());
		return;
	}

	UE::Learning::Agents::ApplyGameSettings(TrainingGameSettings, GetWorld(), PreviousGameSettingsState);

	// We need to setup the trainer prior to sending the config
	PolicyNetworkId = Trainer->AddNetwork(*Policy->GetPolicyNetworkAsset()->NeuralNetworkData);
	CriticNetworkId = Trainer->AddNetwork(*Critic->GetCriticNetworkAsset()->NeuralNetworkData);
	EncoderNetworkId = Trainer->AddNetwork(*Policy->GetEncoderNetworkAsset()->NeuralNetworkData);
	DecoderNetworkId = Trainer->AddNetwork(*Policy->GetDecoderNetworkAsset()->NeuralNetworkData);
	ReplayBufferId = Trainer->AddReplayBuffer(*ReplayBuffer);
	
	TSharedRef<FJsonObject> DataConfigObject = CreateDataConfig();
	TSharedRef<FJsonObject> TrainerConfigObject = CreateTrainerConfig(TrainingSettings);

	UE_LOGF(LogLearning, Display, "%ls: Sending configs...", *GetName());
	SendConfigs(DataConfigObject, TrainerConfigObject);

	UE_LOGF(LogLearning, Display, "%ls: Sending initial policy...", *GetName());
	UE::Learning::ETrainerResponse Response = UE::Learning::ETrainerResponse::Success;

	Response = Trainer->SendNetwork(PolicyNetworkId, *Policy->GetPolicyNetworkAsset()->NeuralNetworkData);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "%ls: Error sending policy to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	Response = Trainer->SendNetwork(CriticNetworkId, *Critic->GetCriticNetworkAsset()->NeuralNetworkData);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "%ls: Error sending critic to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	Response = Trainer->SendNetwork(EncoderNetworkId, *Policy->GetEncoderNetworkAsset()->NeuralNetworkData);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "%ls: Error sending encoder to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	Response = Trainer->SendNetwork(DecoderNetworkId, *Policy->GetDecoderNetworkAsset()->NeuralNetworkData);
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "%ls: Error sending decoder to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}

	if (bResetAgentsOnBegin)
	{
		Manager->ResetAllAgents();
	}

	ReplayBuffer->Reset();

	bIsTraining = true;
}

TSharedRef<FJsonObject> ULearningAgentsPPOTrainer::CreateDataConfig() const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	const int32 ObservationSchemaId = 0;
	const int32 ActionSchemaId = 0;

	// Add Neural Network Config Entries
	TArray<TSharedPtr<FJsonValue>> NetworkObjects;

	// Policy
	{
		TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
		NetworkObject->SetNumberField(TEXT("Id"), PolicyNetworkId);
		NetworkObject->SetStringField(TEXT("Name"), Policy->GetPolicyNetworkAsset()->GetFName().ToString());
		NetworkObject->SetNumberField(TEXT("MaxByteNum"), Policy->GetPolicyNetworkAsset()->NeuralNetworkData->GetSnapshotByteNum());
		
		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
		NetworkObjects.Add(JsonValue);
	}

	// Critic
	{
		TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
		NetworkObject->SetNumberField(TEXT("Id"), CriticNetworkId);
		NetworkObject->SetStringField(TEXT("Name"), Critic->GetCriticNetworkAsset()->GetFName().ToString());
		NetworkObject->SetNumberField(TEXT("MaxByteNum"), Critic->GetCriticNetworkAsset()->NeuralNetworkData->GetSnapshotByteNum());
		NetworkObject->SetNumberField(TEXT("InputSchemaId"), ObservationSchemaId);

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
		NetworkObjects.Add(JsonValue);
	}

	// Encoder
	{
		TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
		NetworkObject->SetNumberField(TEXT("Id"), EncoderNetworkId);
		NetworkObject->SetStringField(TEXT("Name"), Policy->GetEncoderNetworkAsset()->GetFName().ToString());
		NetworkObject->SetNumberField(TEXT("MaxByteNum"), Policy->GetEncoderNetworkAsset()->NeuralNetworkData->GetSnapshotByteNum());
		NetworkObject->SetNumberField(TEXT("InputSchemaId"), ObservationSchemaId);

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
		NetworkObjects.Add(JsonValue);
	}

	// Decoder
	{
		TSharedPtr<FJsonObject> NetworkObject = MakeShared<FJsonObject>();
		NetworkObject->SetNumberField(TEXT("Id"), DecoderNetworkId);
		NetworkObject->SetStringField(TEXT("Name"), Policy->GetDecoderNetworkAsset()->GetFName().ToString());
		NetworkObject->SetNumberField(TEXT("MaxByteNum"), Policy->GetDecoderNetworkAsset()->NeuralNetworkData->GetSnapshotByteNum());
		NetworkObject->SetNumberField(TEXT("OutputSchemaId"), ActionSchemaId);

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(NetworkObject);
		NetworkObjects.Add(JsonValue);
	}

	ConfigObject->SetArrayField(TEXT("Networks"), NetworkObjects);

	// Add Replay Buffers Config Entries
	TArray<TSharedPtr<FJsonValue>> ReplayBufferObjects;
	TSharedRef<FJsonValueObject> ReplayBufferJsonValue = MakeShared<FJsonValueObject>(ReplayBuffer->AsJsonConfig(ReplayBufferId));
	ReplayBufferObjects.Add(ReplayBufferJsonValue);
	ConfigObject->SetArrayField(TEXT("ReplayBuffers"), ReplayBufferObjects);

	// Schemas
	TSharedPtr<FJsonObject> SchemasObject = MakeShared<FJsonObject>();

	// Add the observation schemas
	TArray<TSharedPtr<FJsonValue>> ObservationSchemaObjects;
	{
		// For this PPO trainer, add the one observation schema we have
		TSharedPtr<FJsonObject> ObservationSchemaObject = MakeShared<FJsonObject>();
		ObservationSchemaObject->SetNumberField(TEXT("Id"), ObservationSchemaId);
		ObservationSchemaObject->SetStringField(TEXT("Name"), "Default");
		ObservationSchemaObject->SetObjectField(TEXT("Schema"),
			UE::Learning::Trainer::ConvertObservationSchemaToJSON(Interactor->GetObservationSchema()->ObservationSchema,
				Interactor->GetObservationSchemaElement().SchemaElement));

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(ObservationSchemaObject);
		ObservationSchemaObjects.Add(JsonValue);
	}
	SchemasObject->SetArrayField(TEXT("Observations"), ObservationSchemaObjects);

	// Add the action schemas
	TArray<TSharedPtr<FJsonValue>> ActionSchemaObjects;
	{
		// For this PPO trainer, add the one action schema we have
		TSharedPtr<FJsonObject> ActionSchemaObject = MakeShared<FJsonObject>();
		ActionSchemaObject->SetNumberField(TEXT("Id"), ActionSchemaId);
		ActionSchemaObject->SetStringField(TEXT("Name"), "Default");
		ActionSchemaObject->SetObjectField(TEXT("Schema"),
			UE::Learning::Trainer::ConvertActionSchemaToJSON(Interactor->GetActionSchema()->ActionSchema,
				Interactor->GetActionSchemaElement().SchemaElement));

		TSharedRef<FJsonValueObject> JsonValue = MakeShared<FJsonValueObject>(ActionSchemaObject);
		ActionSchemaObjects.Add(JsonValue);
	}
	SchemasObject->SetArrayField(TEXT("Actions"), ActionSchemaObjects);

	ConfigObject->SetObjectField(TEXT("Schemas"), SchemasObject);

	return ConfigObject;
}

TSharedRef<FJsonObject> ULearningAgentsPPOTrainer::CreateTrainerConfig(const FLearningAgentsPPOTrainingSettings& TrainingSettings) const
{
	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	// Add Training Task-specific Config Entries
	ConfigObject->SetStringField(TEXT("TrainerMethod"), TEXT("PPO"));
	ConfigObject->SetStringField(TEXT("TimeStamp"), *FDateTime::Now().ToFormattedString(TEXT("%Y-%m-%d_%H-%M-%S")));

	// Add PPO-specific Config Entries
	ConfigObject->SetObjectField(TEXT("PPOSettings"), TrainingSettings.AsJsonConfig());

	// Add Training Workflow Config Entries
	ConfigObject->SetStringField(TEXT("Device"), UE::Learning::Trainer::GetDeviceString(UE::Learning::Agents::GetTrainingDevice(TrainingSettings.Device)));
	ConfigObject->SetBoolField(TEXT("UseTensorBoard"), TrainingSettings.bUseTensorboard);
	ConfigObject->SetBoolField(TEXT("UseMLflow"), TrainingSettings.bUseMLflow);
	ConfigObject->SetStringField(TEXT("MLflowTrackingUri"), TrainingSettings.MLflowTrackingUri);

	return ConfigObject;
}

void ULearningAgentsPPOTrainer::SendConfigs(const TSharedRef<FJsonObject>& DataConfigObject, const TSharedRef<FJsonObject>& TrainerConfigObject)
{
	TSharedRef<FJsonObject> ModelConfigObject = MakeShared<FJsonObject>();
	UE::Learning::ETrainerResponse Response = UE::Learning::ETrainerResponse::Success;
	Response = Trainer->SendConfigs(DataConfigObject, ModelConfigObject, TrainerConfigObject);
	
	if (Response != UE::Learning::ETrainerResponse::Success)
	{
		UE_LOGF(LogLearning, Error, "%ls: Error sending config to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
		bHasTrainingFailed = true;
		Trainer->Terminate();
		return;
	}
}

void ULearningAgentsPPOTrainer::DoneTraining()
{
	if (IsTraining())
	{
		Trainer->Wait();

		// If not finished in time, terminate
		Trainer->Terminate();

		UE::Learning::Agents::RevertGameSettings(PreviousGameSettingsState, GetWorld());

		bIsTraining = false;
	}
}

void ULearningAgentsPPOTrainer::EndTraining()
{
	if (IsTraining())
	{
		UE_LOGF(LogLearning, Display, "%ls: Stopping training...", *GetName());
		Trainer->SendStop();
		DoneTraining();
	}
}

void ULearningAgentsPPOTrainer::ProcessExperience(const bool bResetAgentsOnUpdate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsPPOTrainer::ProcessExperience);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (!IsTraining())
	{
		UE_LOGF(LogLearning, Error, "%ls: Training not running.", *GetName());
		return;
	}

	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	// Check Observations, Actions, ActionModifiers, Rewards, and Completions have been completed and have matching iteration number

	TArray<int32> ValidAgentIds;
	ValidAgentIds.Empty(Manager->GetMaxAgentNum());

	for (const int32 AgentId : Manager->GetAllAgentSet())
	{
		if (Interactor->GetObservationIteration(AgentId) == 0 ||
			Interactor->GetActionIteration(AgentId) == 0 ||
			Interactor->GetActionModifierIteration(AgentId) == 0 ||
			TrainingEnvironment->GetRewardIteration(AgentId) == 0 ||
			TrainingEnvironment->GetCompletionIteration(AgentId) == 0)
		{
			UE_LOGF(LogLearning, Display, "%ls: Agent with id %i has not completed a full step of observations, action modifiers, actions, rewards, completions and so experience will not be processed for it.", *GetName(), AgentId);
			continue;
		}

		if (Interactor->GetObservationIteration(AgentId) != Interactor->GetActionIteration(AgentId) ||
			Interactor->GetObservationIteration(AgentId) != Interactor->GetActionModifierIteration(AgentId) ||
			Interactor->GetObservationIteration(AgentId) != TrainingEnvironment->GetRewardIteration(AgentId) ||
			Interactor->GetObservationIteration(AgentId) != TrainingEnvironment->GetCompletionIteration(AgentId))
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i has non-matching iteration numbers (observation: %lli, action: %lli, action modifiers: %lli, reward: %lli, completion: %lli). Experience will not be processed for it.", *GetName(), AgentId,
				Interactor->GetObservationIteration(AgentId),
				Interactor->GetActionIteration(AgentId),
				Interactor->GetActionModifierIteration(AgentId),
				TrainingEnvironment->GetRewardIteration(AgentId),
				TrainingEnvironment->GetCompletionIteration(AgentId));
			continue;
		}

		ValidAgentIds.Add(AgentId);
	}

	UE::Learning::FIndexSet ValidAgentSet = ValidAgentIds;
	ValidAgentSet.TryMakeSlice();

	// Check for episodes that have been immediately completed

	for (const int32 AgentId : ValidAgentSet)
	{
		if (TrainingEnvironment->GetAgentCompletion(AgentId) != UE::Learning::ECompletionMode::Running && EpisodeBuffer->GetEpisodeStepNums()[AgentId] == 0)
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i has completed episode and will be reset but has not generated any experience.", *GetName(), AgentId);
		}
	}

	// Add Experience to Episode Buffer
	EpisodeBuffer->PushObservations(ObservationId, Interactor->GetObservationVectorsArrayView(), ValidAgentSet);
	EpisodeBuffer->PushActions(ActionId, Interactor->GetActionVectorsArrayView(), ValidAgentSet);
	EpisodeBuffer->PushActionModifiers(ActionId, Interactor->GetActionModifierVectorsArrayView(), ValidAgentSet);
	EpisodeBuffer->PushMemoryStates(MemoryStateId, Policy->GetPreEvaluationMemoryState(), ValidAgentSet);
	EpisodeBuffer->PushRewards(RewardId, TrainingEnvironment->GetRewardArrayView(), ValidAgentSet);
	EpisodeBuffer->IncrementEpisodeStepNums(ValidAgentSet);

	// Find the set of agents which have reached the maximum episode length and mark them as truncated
	UE::Learning::Completion::EvaluateEndOfEpisodeCompletions(
		TrainingEnvironment->GetEpisodeCompletions(),
		EpisodeBuffer->GetEpisodeStepNums(),
		EpisodeBuffer->GetMaxStepNum(),
		ValidAgentSet);

	TrainingEnvironment->SetAllCompletions(ValidAgentSet);

	TrainingEnvironment->GetResetBuffer().SetResetInstancesFromCompletions(TrainingEnvironment->GetAllCompletions(), ValidAgentSet);

	// If there are no agents completed we are done
	if (TrainingEnvironment->GetResetBuffer().GetResetInstanceNum() == 0)
	{
		return;
	}

	// Otherwise Gather Observations for completed Instances without incrementing iteration number
	Interactor->GatherObservations(TrainingEnvironment->GetResetBuffer().GetResetInstances(), false);

	// And push those episodes to the Replay Buffer
	const bool bReplayBufferFull = ReplayBuffer->AddEpisodes(
		TrainingEnvironment->GetAllCompletions(),
		{ Interactor->GetObservationVectorsArrayView() },
		{ Policy->GetMemoryState() },
		*EpisodeBuffer,
		TrainingEnvironment->GetResetBuffer().GetResetInstances());

	if (bReplayBufferFull)
	{
		UE::Learning::ETrainerResponse Response = Trainer->SendReplayBuffer(ReplayBufferId, *ReplayBuffer);

		if (Response != UE::Learning::ETrainerResponse::Success)
		{
			UE_LOGF(LogLearning, Error, "%ls: Error waiting to push experience to trainer: %ls. Check log for additional errors.", *GetName(), UE::Learning::Trainer::GetResponseString(Response));
			bHasTrainingFailed = true;
			EndTraining();
			return;
		}

		ReplayBuffer->Reset();

		const TArray<int32> NetworkIds = {PolicyNetworkId, CriticNetworkId, EncoderNetworkId, DecoderNetworkId};
		const TArray<TObjectPtr<ULearningNeuralNetworkData>> Networks = {Policy->GetPolicyNetworkAsset()->NeuralNetworkData, Critic->GetCriticNetworkAsset()->NeuralNetworkData, Policy->GetEncoderNetworkAsset()->NeuralNetworkData, Policy->GetDecoderNetworkAsset()->NeuralNetworkData};
		TArray<UE::Learning::ETrainerResponse> NetworkResponses = Trainer->ReceiveNetworks(NetworkIds, Networks);
	
		TArray<ULearningAgentsNeuralNetwork*> NetworkAssets = { Policy->GetPolicyNetworkAsset(), Critic->GetCriticNetworkAsset(), Policy->GetEncoderNetworkAsset(), Policy->GetDecoderNetworkAsset() };
		for (ULearningAgentsNeuralNetwork* NetworkAsset : NetworkAssets)
		{
			NetworkAsset->ForceMarkDirty();	
		}

		for (int32 i = 0; i < NetworkIds.Num(); i++)
		{
			if (NetworkResponses[i] == UE::Learning::ETrainerResponse::Completed)
			{
				UE_LOGF(LogLearning, Display, "%ls: Trainer completed training.", *GetName());
				DoneTraining();
				return;
			}
			else if (NetworkResponses[i] != UE::Learning::ETrainerResponse::Success)
			{
				UE_LOGF(LogLearning, Error, "Error receiving network (id=%d) from trainer: %ls. Check log for errors.", NetworkIds[i], UE::Learning::Trainer::GetResponseString(NetworkResponses[i]));
				bHasTrainingFailed = true;
				EndTraining();
				return;
			}
		}

		if (bResetAgentsOnUpdate)
		{
			// Reset all agents since we have a new policy
			TrainingEnvironment->GetResetBuffer().SetResetInstances(Manager->GetAllAgentSet());
			Manager->ResetAgents(TrainingEnvironment->GetResetBuffer().GetResetInstancesArray());
			return;
		}
	}

	// Manually reset Episode Buffer for agents who have reached the maximum episode length as 
	// they wont get it reset via the agent manager's call to ResetAgents
	TrainingEnvironment->GetResetBuffer().SetResetInstancesFromCompletions(TrainingEnvironment->GetEpisodeCompletions(), ValidAgentSet);
	EpisodeBuffer->Reset(TrainingEnvironment->GetResetBuffer().GetResetInstances());

	// Call ResetAgents for agents which have manually signaled a completion
	TrainingEnvironment->GetResetBuffer().SetResetInstancesFromCompletions(TrainingEnvironment->GetAgentCompletions(), ValidAgentSet);
	if (TrainingEnvironment->GetResetBuffer().GetResetInstanceNum() > 0)
	{
		Manager->ResetAgents(TrainingEnvironment->GetResetBuffer().GetResetInstancesArray());
	}
}

void ULearningAgentsPPOTrainer::RunTraining(
	const FLearningAgentsPPOTrainingSettings& TrainingSettings,
	const FLearningAgentsTrainingGameSettings& TrainingGameSettings,
	const bool bResetAgentsOnBegin,
	const bool bResetAgentsOnUpdate)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (bHasTrainingFailed)
	{
		UE_LOGF(LogLearning, Error, "%ls: Training has failed. Check log for errors.", *GetName());

#if !WITH_EDITOR
		FGenericPlatformMisc::RequestExitWithStatus(false, 99);
#endif

		return;
	}

	// If we aren't training yet, then start training and do the first inference step.
	if (!IsTraining())
	{
		BeginTraining(TrainingSettings,	TrainingGameSettings, bResetAgentsOnBegin);

		if (!IsTraining())
		{
			// If IsTraining is false, then BeginTraining must have failed and we can't continue.
			return;
		}

		Policy->RunInference();
	}
	// Otherwise, do the regular training process.
	else
	{
		TrainingEnvironment->GatherCompletions();
		TrainingEnvironment->GatherRewards();
		ProcessExperience(bResetAgentsOnUpdate);
		Policy->RunInference();
	}
}

int32 ULearningAgentsPPOTrainer::GetEpisodeStepNum(const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0.0f;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return 0.0f;
	}

	return EpisodeBuffer->GetEpisodeStepNums()[AgentId];
}

bool ULearningAgentsPPOTrainer::HasTrainingFailed() const
{
	return bHasTrainingFailed;
}
