// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsRecorder.h"

#include "LearningAgentsManager.h"
#include "LearningAgentsInteractor.h"
#include "LearningLog.h"
#include "LearningTrainer.h"

#include "Misc/Paths.h"
#include "Misc/FileHelper.h"

#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsRecorder)

FLearningAgentsRecorderPathSettings::FLearningAgentsRecorderPathSettings()
{
	IntermediateRelativePath.Path = FPaths::ProjectIntermediateDir();
}

ULearningAgentsRecorder::FAgentRecordBuffer::FAgentRecordBuffer() = default;

ULearningAgentsRecorder::FAgentRecordBuffer::FAgentRecordBuffer(
	const int32 InObservationCompatibilityHash,
	const int32 InActionCompatibilityHash,
	const UE::Learning::Observation::FSchema InObservationSchema,
	const UE::Learning::Observation::FSchemaElement InObservationSchemaElement,
	const UE::Learning::Action::FSchema InActionSchema,
	const UE::Learning::Action::FSchemaElement InActionSchemaElement)
	: ObservationCompatibilityHash(InObservationCompatibilityHash)
	, ActionCompatibilityHash(InActionCompatibilityHash)
	, ObservationSchema(InObservationSchema)
	, ObservationSchemaElement(InObservationSchemaElement)
	, ActionSchema(InActionSchema)
	, ActionSchemaElement(InActionSchemaElement) {};

TLearningArrayView<1, float> ULearningAgentsRecorder::FAgentRecordBuffer::GetObservation(const int32 SampleIdx)
{
	return Observations[SampleIdx / ChunkSize][SampleIdx % ChunkSize];
}

TLearningArrayView<1, float> ULearningAgentsRecorder::FAgentRecordBuffer::GetAction(const int32 SampleIdx)
{
	return Actions[SampleIdx / ChunkSize][SampleIdx % ChunkSize];
}

TLearningArrayView<1, const float> ULearningAgentsRecorder::FAgentRecordBuffer::GetObservation(const int32 SampleIdx) const
{
	return Observations[SampleIdx / ChunkSize][SampleIdx % ChunkSize];
}

TLearningArrayView<1, const float> ULearningAgentsRecorder::FAgentRecordBuffer::GetAction(const int32 SampleIdx) const
{
	return Actions[SampleIdx / ChunkSize][SampleIdx % ChunkSize];
}

void ULearningAgentsRecorder::FAgentRecordBuffer::Push(
	const TLearningArrayView<1, const float> Observation,
	const TLearningArrayView<1, const float> Action)
{
	if (StepNum % ChunkSize == 0)
	{
		Observations.AddDefaulted_GetRef().SetNumUninitialized({ ChunkSize, Observation.Num() });
		Actions.AddDefaulted_GetRef().SetNumUninitialized({ ChunkSize, Action.Num() });
	}

	UE::Learning::Array::Copy(GetObservation(StepNum), Observation);
	UE::Learning::Array::Copy(GetAction(StepNum), Action);
	StepNum++;
}

bool ULearningAgentsRecorder::FAgentRecordBuffer::IsEmpty() const
{
	return StepNum == 0;
}

void ULearningAgentsRecorder::FAgentRecordBuffer::Empty()
{
	StepNum = 0;
	Observations.Empty();
	Actions.Empty();
}

void ULearningAgentsRecorder::FAgentRecordBuffer::CopyToRecord(FLearningAgentsRecord& Record) const
{
	check(StepNum > 0);

	Record.StepNum = StepNum;
	Record.ObservationDimNum = GetObservation(0).Num();
	Record.ActionDimNum = GetAction(0).Num();
	Record.ObservationCompatibilityHash = ObservationCompatibilityHash;
	Record.ActionCompatibilityHash = ActionCompatibilityHash;
	Record.Blob->ObservationData.SetNumUninitialized(StepNum * Record.ObservationDimNum);
	Record.Blob->ActionData.SetNumUninitialized(StepNum * Record.ActionDimNum);

	TLearningArrayView<2, float> ObservationsView = TLearningArrayView<2, float>(Record.Blob->ObservationData.GetData(), { Record.StepNum, Record.ObservationDimNum });
	TLearningArrayView<2, float> ActionsView = TLearningArrayView<2, float>(Record.Blob->ActionData.GetData(), { Record.StepNum, Record.ActionDimNum });

	for (int32 StepIdx = 0; StepIdx < StepNum; StepIdx++)
	{
		UE::Learning::Array::Copy(ObservationsView[StepIdx], GetObservation(StepIdx));
		UE::Learning::Array::Copy(ActionsView[StepIdx], GetAction(StepIdx));
	}
}

ULearningAgentsRecorder::ULearningAgentsRecorder() : Super(FObjectInitializer::Get()) {}
ULearningAgentsRecorder::ULearningAgentsRecorder(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsRecorder::~ULearningAgentsRecorder() = default;

void ULearningAgentsRecorder::BeginDestroy()
{
	if (IsRecording())
	{
		EndRecording();
	}

	Super::BeginDestroy();
}

ULearningAgentsRecorder* ULearningAgentsRecorder::MakeRecorder(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	TSubclassOf<ULearningAgentsRecorder> Class,
	const FName Name,
	const FLearningAgentsRecorderPathSettings& RecorderPathSettings,
	ULearningAgentsRecording* RecordingAsset,
	bool bReinitializeRecording)
{
	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "MakeRecorder: InManager is nullptr.");
		return nullptr;
	}

	if (!Class)
	{
		UE_LOGF(LogLearning, Error, "MakeRecorder: Class is nullptr.");
		return nullptr;
	}

	const FName UniqueName = MakeUniqueObjectName(InManager, Class, Name, EUniqueObjectNameOptions::GloballyUnique);

	ULearningAgentsRecorder* Recorder = NewObject<ULearningAgentsRecorder>(InManager, Class, UniqueName);
	if (!Recorder) { return nullptr; }

	Recorder->SetupRecorder(
		InManager,
		InInteractor,
		RecorderPathSettings,
		RecordingAsset,
		bReinitializeRecording);

	return Recorder->IsSetup() ? Recorder : nullptr;
}

void ULearningAgentsRecorder::SetupRecorder(
	ULearningAgentsManager*& InManager,
	ULearningAgentsInteractor*& InInteractor,
	const FLearningAgentsRecorderPathSettings& RecorderPathSettings,
	ULearningAgentsRecording* RecordingAsset,
	bool bReinitializeRecording)
{
	if (IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup already performed!", *GetName());
		return;
	}

	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "%ls: InManager is nullptr.", *GetName());
		return;
	}

	if (!InInteractor)
	{
		UE_LOGF(LogLearning, Error, "%ls: InInteractor is nullptr.", *GetName());
		return;
}

	if (!InInteractor->IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: %ls's Setup must be run before it can be used.", *GetName(), *InInteractor->GetName());
		return;
	}

	Manager = InManager;
	Interactor = InInteractor;

	if (RecordingAsset)
	{
		Recording = RecordingAsset;
	}
	else
	{
		const FName UniqueName = MakeUniqueObjectName(this, ULearningAgentsRecording::StaticClass(), TEXT("Recording"), EUniqueObjectNameOptions::GloballyUnique);
		Recording = NewObject<ULearningAgentsRecording>(this, UniqueName);
	}

	if (bReinitializeRecording)
	{
		Recording->ResetRecording();
	}

	RecordingDirectory = RecorderPathSettings.IntermediateRelativePath.Path / TEXT("LearningAgents") / RecorderPathSettings.RecordingsSubdirectory;

	// Find Compatibility Hash

	const int32 ObservationCompatibilityHash = UE::Learning::Observation::GetSchemaObjectsCompatibilityHash(Interactor->GetObservationSchema()->ObservationSchema, Interactor->GetObservationSchemaElement().SchemaElement);
	const int32 ActionCompatibilityHash = UE::Learning::Action::GetSchemaObjectsCompatibilityHash(Interactor->GetActionSchema()->ActionSchema, Interactor->GetActionSchemaElement().SchemaElement);

	// Create Record Buffers

	RecordBuffers.Empty(Manager->GetMaxAgentNum());
	for (int32 RecordBufferIdx = 0; RecordBufferIdx < Manager->GetMaxAgentNum(); RecordBufferIdx++)
	{
		RecordBuffers.Emplace(
			ObservationCompatibilityHash,
			ActionCompatibilityHash,
			Interactor->GetObservationSchema()->ObservationSchema,
			Interactor->GetObservationSchemaElement().SchemaElement,
			Interactor->GetActionSchema()->ActionSchema,
			Interactor->GetActionSchemaElement().SchemaElement);
	}

	FLearningAgentsSchema NewSchema = SaveSchemaConfig();

	if (bReinitializeRecording)
	{
		Recording->AddSchema(Recording->NewSchemaName, NewSchema);
	}

	bIsSetup = true;

	Manager->AddListener(this);
}

void ULearningAgentsRecorder::OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	for (const int32 AgentId : AgentIds)
	{
		if (!RecordBuffers[AgentId].IsEmpty())
		{
			FLearningAgentsRecord& Record = Recording->Records.AddDefaulted_GetRef();
			Record.Blob = NewObject<ULearningAgentsRecordBlob>(Recording);
			Record.SchemaName = Recording->NewSchemaName;
			Record.Tags.AddTag(Recording->NewTag);

			RecordBuffers[AgentId].CopyToRecord(Record);
			RecordBuffers[AgentId].Empty();
		}
	}
}

void ULearningAgentsRecorder::OnAgentsReset_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	for (const int32 AgentId : AgentIds)
	{
		if (!RecordBuffers[AgentId].IsEmpty())
		{
			FLearningAgentsRecord& Record = Recording->Records.AddDefaulted_GetRef();
			Record.Blob = NewObject<ULearningAgentsRecordBlob>(Recording);
			Record.SchemaName = Recording->NewSchemaName;
			Record.Tags.AddTag(Recording->NewTag);

			RecordBuffers[AgentId].CopyToRecord(Record);
			RecordBuffers[AgentId].Empty();
		}
	}
}

void ULearningAgentsRecorder::AddExperience()
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (!IsRecording())
	{
		UE_LOGF(LogLearning, Error, "%ls: Trying to add experience but we aren't currently recording. Call BeginRecording before AddExperience.", *GetName());
		return;
	}

	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	for (const int32 AgentId : Manager->GetAllAgentSet())
	{
		if (Interactor->GetObservationIteration(AgentId) == 0 || Interactor->GetActionIteration(AgentId) == 0)
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i has not made observations and taken actions so experience will not be recorded for it.", *GetName(), AgentId);
			continue;
		}

		if (Interactor->GetObservationIteration(AgentId) != Interactor->GetActionIteration(AgentId))
		{
			UE_LOGF(LogLearning, Warning, "%ls: Agent with id %i does not have matching iteration numbers for observations and actions so experience will not be recorded for it.", *GetName(), AgentId);
			continue;
		}

		RecordBuffers[AgentId].Push(Interactor->GetObservationVectorsArrayView()[AgentId], Interactor->GetActionVectorsArrayView()[AgentId]);
	}
}

bool ULearningAgentsRecorder::IsRecording() const
{
	return bIsRecording;
}

void ULearningAgentsRecorder::EndRecording()
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (!IsRecording())
	{
		UE_LOGF(LogLearning, Error, "%ls: Cannot end recording as we are not currently recording!", *GetName());
		return;
	}

	// Write to Recording

	for (const int32 AgentId : Manager->GetAllAgentSet())
	{
		if (!RecordBuffers[AgentId].IsEmpty())
		{
			FLearningAgentsRecord& Record = Recording->Records.AddDefaulted_GetRef();
			Record.Blob = NewObject<ULearningAgentsRecordBlob>(Recording);
			Record.SchemaName = Recording->NewSchemaName;
			Record.Tags.AddTag(Recording->NewTag);

			RecordBuffers[AgentId].CopyToRecord(Record);
			RecordBuffers[AgentId].Empty();
		}
	}

	bIsRecording = false;

	OnRecordingEnd.Broadcast();
}

void ULearningAgentsRecorder::EndRecordingAndDiscard()
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (!IsRecording())
	{
		UE_LOGF(LogLearning, Error, "%ls: Cannot end recording as we are not currently recording!", *GetName());
		return;
	}

	// Discard Records

	for (const int32 AgentId : Manager->GetAllAgentSet())
	{
		RecordBuffers[AgentId].Empty();
	}

	bIsRecording = false;

	OnRecordingEnd.Broadcast();
}

void ULearningAgentsRecorder::BeginRecording()
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (IsRecording())
	{
		UE_LOGF(LogLearning, Error, "%ls: Cannot begin recording as we are already Recording!", *GetName());
		return;
	}

	for (const int32 AgentId : Manager->GetAllAgentSet())
	{
		RecordBuffers[AgentId].Empty();
	}

	bIsRecording = true;

	OnRecordingBegin.Broadcast();
}

const ULearningAgentsRecording* ULearningAgentsRecorder::GetRecordingAsset() const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return nullptr;
	}

	return Recording;
}

ULearningAgentsInteractor* ULearningAgentsRecorder::GetInteractor() const
{
	return Interactor;
}

FLearningAgentsSchema ULearningAgentsRecorder::SaveSchemaConfig(const FString& FileName)
{
	// Save config file

	TSharedRef<FJsonObject> ConfigObject = MakeShared<FJsonObject>();

	TSharedPtr<FJsonObject> ObservationJson = UE::Learning::Trainer::ConvertObservationSchemaToJSON(
		Interactor->GetObservationSchema()->ObservationSchema,
		Interactor->GetObservationSchemaElement().SchemaElement);
	ConfigObject->SetObjectField(TEXT("ObservationSchema"), ObservationJson);

	TSharedPtr<FJsonObject> ActionJson = UE::Learning::Trainer::ConvertActionSchemaToJSON(
		Interactor->GetActionSchema()->ActionSchema,
		Interactor->GetActionSchemaElement().SchemaElement);
	ConfigObject->SetObjectField(TEXT("ActionSchema"), ActionJson);

	FString ConfigString;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ConfigString, 0);
	FJsonSerializer::Serialize(ConfigObject, JsonWriter, true);

	FFileHelper::SaveStringToFile(ConfigString, *(RecordingDirectory / FileName));

	// Return schema

	FLearningAgentsSchema NewSchema;
	TSharedRef<TJsonWriter<>> ObservationWriter = TJsonWriterFactory<>::Create(&NewSchema.ObservationSchemaJson);
	FJsonSerializer::Serialize(ObservationJson.ToSharedRef(), ObservationWriter);
	TSharedRef<TJsonWriter<>> ActionWriter = TJsonWriterFactory<>::Create(&NewSchema.ActionSchemaJson);
	FJsonSerializer::Serialize(ActionJson.ToSharedRef(), ActionWriter);

	return NewSchema;
}
