// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsRecording.h"

#include "LearningAgentsInteractor.h"

#include "LearningExperience.h"
#include "LearningLog.h"
#include "LearningTrainer.h"

#include "UObject/Package.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Dom/JsonObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsRecording)

ULearningAgentsRecording::ULearningAgentsRecording() = default;
ULearningAgentsRecording::ULearningAgentsRecording(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsRecording::~ULearningAgentsRecording() = default;

namespace UE::Learning::Agents::Recording::Private
{
	static constexpr int32 MagicNumber = 0x06b5fb26;
	static constexpr int32 VersionNumber = 1;
}

int32 UE::Learning::Agents::Recording::GetMagicNumber() 
{ 
	return UE::Learning::Agents::Recording::Private::MagicNumber; 
}

int32 UE::Learning::Agents::Recording::GetVersionNumber() 
{ 
	return UE::Learning::Agents::Recording::Private::VersionNumber; 
}

void ULearningAgentsRecording::ResetRecording()
{
	Schemas.Empty();
	Records.Empty();
	ForceMarkDirty();
}

void ULearningAgentsRecording::AppendRecording()
{
	AppendRecordingFromFile(RecordingFile, NewSchemaName, NewTag);
}

void ULearningAgentsRecording::AppendAllRecordingsFromFolder()
{
	if (RecordingFolder.Path.IsEmpty())
	{
		UE_LOGF(LogLearning, Error, "%ls: RecordingFolder path is empty.", *GetName());
		return;
	}

	IFileManager& FileManager = IFileManager::Get();
	FString BasePathAbs = FPaths::ConvertRelativePathToFull(RecordingFolder.Path);
	FPaths::NormalizeDirectoryName(BasePathAbs);

	TArray<FString> FileNames;
	FileManager.FindFilesRecursive(FileNames, *BasePathAbs, TEXT("*.record"), true, false);

	if (FileNames.Num() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No .record files found in folder: %ls", *GetName(), *BasePathAbs);
		return;
	}

	TArray<FLearningAgentsRecord> RecordsFromDir;

	for (const FString& FileName : FileNames)
	{
		if (!FileManager.FileExists(*FileName))
		{
			UE_LOGF(LogLearning, Warning, "%ls: Skipping missing file discovered earlier: %ls", *GetName(), *FileName);
			continue;
		}

		FFilePath FilePath;
		FilePath.FilePath = FileName;

		TArray<FLearningAgentsRecord> RecordsFromFile;
		LoadRecordingFromFile(FilePath, NewSchemaName, NewTag, RecordsFromFile);

		if (!RecordsFromFile.IsEmpty())
		{
			RecordsFromDir.Reserve(RecordsFromDir.Num() + RecordsFromFile.Num());
			RecordsFromDir.Append(MoveTemp(RecordsFromFile));
		}
	}

	if (!RecordsFromDir.IsEmpty())
	{
		Records.Reserve(Records.Num() + RecordsFromDir.Num());
		Records.Append(MoveTemp(RecordsFromDir));
	}

	ForceMarkDirty();
}

void ULearningAgentsRecording::AppendSchema()
{
	AppendSchemaFromFile(NewSchemaName, SchemaFile);
}

void ULearningAgentsRecording::AppendSchemaFromFile(const FName& SchemaName, const FFilePath& Schema)
{
	FString FileString;
	if (FFileHelper::LoadFileToString(FileString, *Schema.FilePath))
	{
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileString);
		TSharedPtr<FJsonObject> JsonObject;
		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			FLearningAgentsSchema NewSchema;

			const TSharedPtr<FJsonObject>& ObservationJson = JsonObject->GetObjectField(TEXT("ObservationSchema"));
			TSharedRef<TJsonWriter<>> ObservationWriter = TJsonWriterFactory<>::Create(&NewSchema.ObservationSchemaJson);
			FJsonSerializer::Serialize(ObservationJson.ToSharedRef(), ObservationWriter);

			const TSharedPtr<FJsonObject>& ActionJson = JsonObject->GetObjectField(TEXT("ActionSchema"));
			TSharedRef<TJsonWriter<>> ActionWriter = TJsonWriterFactory<>::Create(&NewSchema.ActionSchemaJson);
			FJsonSerializer::Serialize(ActionJson.ToSharedRef(), ActionWriter);

			AddSchema(SchemaName, NewSchema);
			ForceMarkDirty();
		}
		else
		{
			UE_LOGF(LogLearning, Error, "%ls: Failed to load schema. Failed to parse JSON from: \"%ls\"", *GetName(), *Schema.FilePath);
			return;
		}
	}
	else
	{
		UE_LOGF(LogLearning, Error, "%ls: Failed to load schema. File not found: \"%ls\"", *GetName(), *Schema.FilePath);
		return;
	}
}

void ULearningAgentsRecording::LoadRecordingFromFile(const FFilePath& File, const FName& SchemaName, const FGameplayTag& Tag, TArray<FLearningAgentsRecord>& OutRecords)
{
	TArray<uint8> RecordingData;

	if (FFileHelper::LoadFileToArray(RecordingData, *File.FilePath))
	{
		if (RecordingData.Num() < sizeof(int32) * 3)
		{
			UE_LOGF(LogLearning, Error, "%ls: Failed to load recording. Incorrect Format.", *GetName());
			return;
		}

		int64 Offset = 0;

		int32 MagicNumber;
		UE::Learning::DeserializeFromBytes(Offset, RecordingData, MagicNumber);

		if (MagicNumber != UE::Learning::Agents::Recording::Private::MagicNumber)
		{
			UE_LOGF(LogLearning, Error, "%ls: Failed to load recording. Incorrect Magic Number.", *GetName());
			return;
		}

		int32 VersionNumber;
		UE::Learning::DeserializeFromBytes(Offset, RecordingData, VersionNumber);

		if (VersionNumber != UE::Learning::Agents::Recording::Private::VersionNumber)
		{
			UE_LOGF(LogLearning, Error, "%ls: Failed to load recording. Unsupported Version Number %i.", *GetName(), VersionNumber);
			return;
		}

		int32 RecordNum;
		UE::Learning::DeserializeFromBytes(Offset, RecordingData, RecordNum);
		check(RecordNum > 0);

		OutRecords.Reset(RecordNum);
		
		for (int32 RecordIdx = 0; RecordIdx < RecordNum; RecordIdx++)
		{
			FLearningAgentsRecord Record;
			Record.Blob = NewObject<ULearningAgentsRecordBlob>(this);

			UE::Learning::DeserializeFromBytes(Offset, RecordingData, Record.StepNum);
			UE::Learning::DeserializeFromBytes(Offset, RecordingData, Record.ObservationDimNum);
			UE::Learning::DeserializeFromBytes(Offset, RecordingData, Record.ActionDimNum);
			UE::Learning::DeserializeFromBytes(Offset, RecordingData, Record.ObservationCompatibilityHash);
			UE::Learning::DeserializeFromBytes(Offset, RecordingData, Record.ActionCompatibilityHash);
			UE::Learning::Array::DeserializeFromBytes32<2, float>(Offset, RecordingData, Record.Blob->ObservationData);
			UE::Learning::Array::DeserializeFromBytes32<2, float>(Offset, RecordingData, Record.Blob->ActionData);
			
			Record.Name = FPaths::GetBaseFilename(File.FilePath);
			Record.SchemaName = SchemaName;
			Record.Tags.AddTag(Tag);

			OutRecords.Add(MoveTemp(Record));
		}

		check(Offset == RecordingData.Num());
	}
	else
	{
		UE_LOGF(LogLearning, Error, "%ls: Failed to load recording. File not found: \"%ls\"", *GetName(), *File.FilePath);
	}
}

void ULearningAgentsRecording::SaveRecordingToFile(const FFilePath& File) const
{
	TArray<uint8> RecordingData;

	int32 TotalByteNum = 
		sizeof(int32) + // Magic Num
		sizeof(int32) + // Version Num
		sizeof(int32);  // Record Num

	for (int32 RecordIdx = 0; RecordIdx < Records.Num(); RecordIdx++)
	{
		TotalByteNum +=
			sizeof(int32) + // StepNum
			sizeof(int32) + // ObservationDimNum
			sizeof(int32) + // ActionDimNum
			sizeof(int32) + // ObservationCompatibilityHash
			sizeof(int32) + // ActionCompatibilityHash
			UE::Learning::Array::SerializationByteNum32<2, float>({ Records[RecordIdx].StepNum, Records[RecordIdx].ObservationDimNum }) + // Observations
			UE::Learning::Array::SerializationByteNum32<2, float>({ Records[RecordIdx].StepNum, Records[RecordIdx].ActionDimNum });	   // Actions
	}

	RecordingData.SetNumUninitialized(TotalByteNum);

	int64 Offset = 0;
	UE::Learning::SerializeToBytes(Offset, RecordingData, UE::Learning::Agents::Recording::Private::MagicNumber);
	UE::Learning::SerializeToBytes(Offset, RecordingData, UE::Learning::Agents::Recording::Private::VersionNumber);
	UE::Learning::SerializeToBytes(Offset, RecordingData, Records.Num());

	for (int32 RecordIdx = 0; RecordIdx < Records.Num(); RecordIdx++)
	{
		UE::Learning::SerializeToBytes(Offset, RecordingData, Records[RecordIdx].StepNum);
		UE::Learning::SerializeToBytes(Offset, RecordingData, Records[RecordIdx].ObservationDimNum);
		UE::Learning::SerializeToBytes(Offset, RecordingData, Records[RecordIdx].ActionDimNum);
		UE::Learning::SerializeToBytes(Offset, RecordingData, Records[RecordIdx].ObservationCompatibilityHash);
		UE::Learning::SerializeToBytes(Offset, RecordingData, Records[RecordIdx].ActionCompatibilityHash);
		UE::Learning::Array::SerializeToBytes32<2, float>(Offset, RecordingData, { Records[RecordIdx].StepNum, Records[RecordIdx].ObservationDimNum }, Records[RecordIdx].Blob->ObservationData);
		UE::Learning::Array::SerializeToBytes32<2, float>(Offset, RecordingData, { Records[RecordIdx].StepNum, Records[RecordIdx].ActionDimNum }, Records[RecordIdx].Blob->ActionData);
	}

	check(Offset == RecordingData.Num());

	if (!FFileHelper::SaveArrayToFile(RecordingData, *File.FilePath))
	{
		UE_LOGF(LogLearning, Error, "%ls: Failed to save recording to file: \"%ls\"", *GetName(), *File.FilePath);
	}
}

void ULearningAgentsRecording::AppendRecordingFromFile(const FFilePath& File, const FName& SchemaName, const FGameplayTag& Tag)
{
	TArray<FLearningAgentsRecord> RecordsFromFile;
	LoadRecordingFromFile(File, SchemaName, Tag, RecordsFromFile);
	if (!RecordsFromFile.IsEmpty())
	{
		Records.Reserve(Records.Num() + RecordsFromFile.Num());
		Records.Append(MoveTemp(RecordsFromFile));
		ForceMarkDirty();
	}
}

void ULearningAgentsRecording::LoadRecordingFromAsset(ULearningAgentsRecording* RecordingAsset)
{
	if (!RecordingAsset)
	{
		UE_LOGF(LogLearning, Error, "%ls: Asset is nullptr.", *GetName());
		return;
	}

	if (RecordingAsset == this)
	{
		UE_LOGF(LogLearning, Error, "%ls: Asset is same as the current recording.", *GetName());
		return;
	}

	Records = RecordingAsset->Records;
	ForceMarkDirty();
}

void ULearningAgentsRecording::SaveRecordingToAsset(ULearningAgentsRecording* RecordingAsset)
{
	if (!RecordingAsset)
	{
		UE_LOGF(LogLearning, Error, "%ls: Asset is nullptr.", *GetName());
		return;
	}

	if (RecordingAsset == this)
	{
		UE_LOGF(LogLearning, Error, "%ls: Asset is same as the current recording.", *GetName());
		return;
	}

	RecordingAsset->Records = Records;
	RecordingAsset->ForceMarkDirty();
}

void ULearningAgentsRecording::AppendRecordingToAsset(ULearningAgentsRecording* RecordingAsset)
{
	if (!RecordingAsset)
	{
		UE_LOGF(LogLearning, Error, "%ls: Asset is invalid.", *GetName());
		return;
	}

	if (RecordingAsset == this)
	{
		UE_LOGF(LogLearning, Error, "%ls: Asset is same as the current recording.", *GetName());
		return;
	}

	RecordingAsset->Records.Append(Records);
	RecordingAsset->ForceMarkDirty();
}

int32 ULearningAgentsRecording::GetRecordNum() const
{
	return Records.Num();
}

int32 ULearningAgentsRecording::GetRecordStepNum(const int32 Record) const
{
	if (Record < 0 || Record >= Records.Num())
	{
		UE_LOGF(LogLearning, Error, "%ls: Record out of range. Asked for record %i but recording only has %i records.", *GetName(), Record, Records.Num());
		return 0;
	}

	return Records[Record].StepNum;
}


void ULearningAgentsRecording::GetObservationVector(TArray<float>& OutObservationVector, int32& OutObservationCompatibilityHash, const int32 Record, const int32 Step)
{
	if (Record < 0 || Record >= Records.Num())
	{
		UE_LOGF(LogLearning, Error, "%ls: Record out of range. Asked for record %i but recording only has %i records.", *GetName(), Record, Records.Num());
		OutObservationVector.Empty();
		OutObservationCompatibilityHash = 0;
		return;
	}

	if (Step < 0 || Step >= Records[Record].StepNum)
	{
		UE_LOGF(LogLearning, Error, "%ls: Step out of range. Asked for step %i but recording only has %i steps.", *GetName(), Step, Records[Record].StepNum);
		OutObservationVector.Empty();
		OutObservationCompatibilityHash = 0;
		return;
	}

	OutObservationVector = MakeArrayView(Records[Record].Blob->ObservationData).Slice(Records[Record].ObservationDimNum * Step, Records[Record].ObservationDimNum);
	OutObservationCompatibilityHash = Records[Record].ObservationCompatibilityHash;
}

void ULearningAgentsRecording::GetActionVector(TArray<float>& OutActionVector, int32& OutActionCompatibilityHash, const int32 Record, const int32 Step)
{
	if (Record < 0 || Record >= Records.Num())
	{
		UE_LOGF(LogLearning, Error, "%ls: Record out of range. Asked for record %i but recording only has %i records.", *GetName(), Record, Records.Num());
		OutActionVector.Empty();
		OutActionCompatibilityHash = 0;
		return;
	}

	if (Step < 0 || Step >= Records[Record].StepNum)
	{
		UE_LOGF(LogLearning, Error, "%ls: Step out of range. Asked for step %i but recording only has %i steps.", *GetName(), Step, Records[Record].StepNum);
		OutActionVector.Empty();
		OutActionCompatibilityHash = 0;
		return;
	}

	OutActionVector = MakeArrayView(Records[Record].Blob->ActionData).Slice(Records[Record].ActionDimNum * Step, Records[Record].ActionDimNum);
	OutActionCompatibilityHash = Records[Record].ActionCompatibilityHash;
}

TUniquePtr<UE::Learning::FReplayBuffer> ULearningAgentsRecording::BuildReplayBuffer(
	const ULearningAgentsInteractor* Interactor,
	const FGameplayTag& RecordingFilterTag,
	bool bCheckObservationSchemaCompatibility,
	bool bCheckActionSchemaCompatibility) const
{
	using namespace UE::Learning;

	// Count valid episodes and steps

	int32 TotalEpisodeNum = 0;
	int32 TotalStepNum = 0;
	for (const FLearningAgentsRecord& Record : Records)
	{
		if (RecordingFilterTag.IsValid() && !Record.Tags.HasTag(RecordingFilterTag))
		{
			UE_LOGF(LogLearning, Log, "Skipping record %ls because its missing tag %ls.", *Record.Name, *RecordingFilterTag.ToString());
			continue;
		}

		if (!Schemas.Contains(Record.SchemaName))
		{
			UE_LOGF(LogLearning, Error, "Schema name %ls not found in schema list. Is the schema added and are the names correct?", *Record.SchemaName.ToString());
			continue;
		}

		if (bCheckObservationSchemaCompatibility && !Trainer::IsObservationSchemaSubsetCompatible(Schemas[Record.SchemaName].ObservationSchemaJson, Interactor->GetObservationSchema()->ObservationSchema, Interactor->GetObservationSchemaElement().SchemaElement))
		{
			UE_LOGF(LogLearning, Warning, "Record has incompatible observation schema.");
			continue;
		}

		if (bCheckActionSchemaCompatibility && !Trainer::IsActionSchemaSubsetCompatible(Schemas[Record.SchemaName].ActionSchemaJson, Interactor->GetActionSchema()->ActionSchema, Interactor->GetActionSchemaElement().SchemaElement))
		{
			UE_LOGF(LogLearning, Warning, "Record has incompatible action schema.");
			continue;
		}

		TotalEpisodeNum++;
		TotalStepNum += Record.StepNum;
	}

	if (TotalStepNum == 0)
	{
		UE_LOGF(LogLearning, Warning, "Recording contains no valid training data.");
		return nullptr;
	}

	// Sizes

	const int32 ObservationNum = Interactor->GetObservationVectorSize();
	const int32 ActionNum = Interactor->GetActionVectorSize();

	// Copy into flat arrays

	TLearningArray<1, int32> RecordedEpisodeStarts;
	TLearningArray<1, int32> RecordedEpisodeLengths;
	TLearningArray<2, float> RecordedObservations;
	TLearningArray<2, float> RecordedActions;

	RecordedEpisodeStarts.SetNumUninitialized({ TotalEpisodeNum });
	RecordedEpisodeLengths.SetNumUninitialized({ TotalEpisodeNum });
	RecordedObservations.SetNumUninitialized({ TotalStepNum, ObservationNum });
	RecordedActions.SetNumUninitialized({ TotalStepNum, ActionNum });

	int32 EpisodeIdx = 0;
	int32 StepIdx = 0;
	for (const FLearningAgentsRecord& Record : Records)
	{
		if (RecordingFilterTag.IsValid() && !Record.Tags.HasTag(RecordingFilterTag))
		{
			continue;
		}

		if (!Schemas.Contains(Record.SchemaName))
		{
			continue;
		}

		const bool bObservationSchemaSubsetCompatible = (!bCheckObservationSchemaCompatibility) || Trainer::IsObservationSchemaSubsetCompatible(Schemas[Record.SchemaName].ObservationSchemaJson, Interactor->GetObservationSchema()->ObservationSchema, Interactor->GetObservationSchemaElement().SchemaElement);
		const bool bActionSchemaSubsetCompatible = (!bCheckActionSchemaCompatibility) || Trainer::IsActionSchemaSubsetCompatible(Schemas[Record.SchemaName].ActionSchemaJson, Interactor->GetActionSchema()->ActionSchema, Interactor->GetActionSchemaElement().SchemaElement);

		if (Record.ObservationDimNum == ObservationNum && Record.ActionDimNum == ActionNum)
		{
			TLearningArrayView<2, const float> ObservationsView(Record.Blob->ObservationData.GetData(), { Record.StepNum, Record.ObservationDimNum });
			TLearningArrayView<2, const float> ActionsView(Record.Blob->ActionData.GetData(), { Record.StepNum, Record.ActionDimNum });

			RecordedEpisodeStarts[EpisodeIdx] = StepIdx;
			RecordedEpisodeLengths[EpisodeIdx] = Record.StepNum;
			Array::Copy(RecordedObservations.Slice(StepIdx, Record.StepNum), ObservationsView);
			Array::Copy(RecordedActions.Slice(StepIdx, Record.StepNum), ActionsView);
		}
		else if (bObservationSchemaSubsetCompatible && bActionSchemaSubsetCompatible)
		{
			TLearningArrayView<2, const float> ObservationsView(Record.Blob->ObservationData.GetData(), { Record.StepNum, Record.ObservationDimNum });
			TLearningArrayView<2, const float> ActionsView(Record.Blob->ActionData.GetData(), { Record.StepNum, Record.ActionDimNum });

			RecordedEpisodeStarts[EpisodeIdx] = StepIdx;
			RecordedEpisodeLengths[EpisodeIdx] = Record.StepNum;

			TArray<int32> ObservationIndices = Trainer::ComputeObservationSchemaSubsetIndices(
				Schemas[Record.SchemaName].ObservationSchemaJson, Interactor->GetObservationSchema()->ObservationSchema, Interactor->GetObservationSchemaElement().SchemaElement);
			check(ObservationIndices.Num() == ObservationNum);

			for (int32 Step = 0; Step < Record.StepNum; Step++)
			{
				for (int32 i = 0; i < ObservationIndices.Num(); i++)
				{
					RecordedObservations[StepIdx + Step][i] = ObservationsView[Step][ObservationIndices[i]];
				}
			}

			TArray<int32> ActionIndices = Trainer::ComputeActionSchemaSubsetIndices(
				Schemas[Record.SchemaName].ActionSchemaJson, Interactor->GetActionSchema()->ActionSchema, Interactor->GetActionSchemaElement().SchemaElement);
			check(ActionIndices.Num() == ActionNum);

			for (int32 Step = 0; Step < Record.StepNum; Step++)
			{
				for (int32 i = 0; i < ActionIndices.Num(); i++)
				{
					RecordedActions[StepIdx + Step][i] = ActionsView[Step][ActionIndices[i]];
				}
			}
		}
		else { continue; }

		EpisodeIdx++;
		StepIdx += Record.StepNum;
	}

	check(EpisodeIdx == TotalEpisodeNum);
	check(StepIdx == TotalStepNum);

	const int32 ObservationSchemaId = 0;
	const int32 ActionSchemaId = 0;

	// Create Replay Buffer from Records
	TUniquePtr<FReplayBuffer> ReplayBuffer = MakeUnique<FReplayBuffer>();
	ReplayBuffer->AddRecords(
		TotalEpisodeNum,
		TotalStepNum,
		ObservationSchemaId,
		ObservationNum,
		ActionSchemaId,
		ActionNum,
		RecordedEpisodeStarts,
		RecordedEpisodeLengths,
		RecordedObservations,
		RecordedActions);

	return ReplayBuffer;
}

void ULearningAgentsRecording::ForceMarkDirty()
{
	// Manually mark the package as dirty since just using `Modify` prevents 
	// marking packages as dirty during PIE which is most likely when this
	// is being used.
	if (UPackage* Package = GetPackage())
	{
		const bool bIsDirty = Package->IsDirty();

		if (!bIsDirty)
		{
			Package->SetDirtyFlag(true);
		}

		Package->PackageMarkedDirtyEvent.Broadcast(Package, bIsDirty);
	}
}

void ULearningAgentsRecording::AddSchema(const FName& SchemaName, FLearningAgentsSchema NewSchema)
{
	Schemas.Add(SchemaName, NewSchema);
}
