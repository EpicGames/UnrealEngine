// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsRewards.h"

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsDebug.h"

#include "LearningLog.h"

#include "Components/SplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsRewards)

float ULearningAgentsRewards::MakeReward(
	const float RewardValue,
	const float RewardScale,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId, 
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const float Reward = RewardValue * RewardScale;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nValue: [% 6.1f]\nScale: [% 6.2f]\nReward: [% 6.2f]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			RewardValue,
			RewardScale,
			Reward);
	}
#endif

	return Reward;
}

float ULearningAgentsRewards::MakeRewardOnCondition(
	const bool bCondition, 
	const float RewardScale,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const float Reward = bCondition ? RewardScale : 0.0f;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nCondition: [%s]\nScale: [% 6.2f]\nReward: [% 6.2f]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			bCondition ? TEXT("true") : TEXT("false"),
			RewardScale,
			Reward);
	}
#endif

	return Reward;
}

float ULearningAgentsRewards::MakeRewardFromLocationDifference(
	const FVector LocationA,
	const FVector LocationB,
	const float LocationScale,
	const float RewardScale,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const float Distance = FVector::Dist(LocationA, LocationB);
	const float Difference = Distance / FMath::Max(LocationScale, UE_SMALL_NUMBER);
	const float Reward = Difference * RewardScale;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationA,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			LocationA,
			LocationB,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationB,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocationA: [% 6.1f % 6.1f % 6.1f]\nLocationB: [% 6.1f % 6.1f % 6.1f]\nDistance: [% 6.2f]\nLocationScale: [% 6.2f]\nDifference: [% 6.2f]\nScale: [% 6.2f]\nReward: [% 6.2f]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			LocationA.X, LocationA.Y, LocationA.Z,
			LocationB.X, LocationB.Y, LocationB.Z,
			Distance,
			LocationScale,
			Difference,
			RewardScale,
			Reward);
	}
#endif

	return Reward;
}

float ULearningAgentsRewards::MakeRewardOnLocationDifferenceBelowThreshold(
	const FVector LocationA, 
	const FVector LocationB, 
	const float DistanceThreshold, 
	const float RewardScale,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const float Distance = FVector::Distance(LocationA, LocationB);
	const bool bCondition = Distance < DistanceThreshold;
	const float Reward = MakeRewardOnCondition(bCondition, RewardScale);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationA,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			LocationA,
			LocationB,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationB,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocationA: [% 6.1f % 6.1f % 6.1f]\nLocationB: [% 6.1f % 6.1f % 6.1f]\nDistance: [% 6.2f]\nThreshold: [% 6.2f]\nCondition: [%s]\nScale: [% 6.2f]\nReward: [% 6.2f]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			LocationA.X, LocationA.Y, LocationA.Z,
			LocationB.X, LocationB.Y, LocationB.Z,
			Distance,
			DistanceThreshold,
			bCondition ? TEXT("true") : TEXT("false"),
			RewardScale,
			Reward);
	}
#endif

	return Reward;
}

float ULearningAgentsRewards::MakeRewardOnLocationDifferenceAboveThreshold(
	const FVector LocationA,
	const FVector LocationB,
	const float DistanceThreshold,
	const float RewardScale,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const float Distance = FVector::Distance(LocationA, LocationB);
	const bool bCondition = Distance > DistanceThreshold;
	const float Reward = MakeRewardOnCondition(bCondition, RewardScale);

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationA,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			LocationA,
			LocationB,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationB,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocationA: [% 6.1f % 6.1f % 6.1f]\nLocationB: [% 6.1f % 6.1f % 6.1f]\nDistance: [% 6.2f]\nThreshold: [% 6.2f]\nCondition: [%s]\nScale: [% 6.2f]\nReward: [% 6.2f]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			LocationA.X, LocationA.Y, LocationA.Z,
			LocationB.X, LocationB.Y, LocationB.Z,
			Distance,
			DistanceThreshold,
			bCondition ? TEXT("true") : TEXT("false"),
			RewardScale,
			Reward);
	}
#endif

	return Reward;
}

float ULearningAgentsRewards::MakeRewardFromLocationSimilarity(
	const FVector LocationA, 
	const FVector LocationB, 
	const float LocationScale, 
	const float RewardScale,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const float LocationDifference = FVector::Dist(LocationA, LocationB);
	const float Similarity = FMath::InvExpApprox(FMath::Square(LocationDifference / FMath::Max(LocationScale, UE_SMALL_NUMBER)));
	const float Reward = Similarity * RewardScale;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationA,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_SEGMENT(VisualLoggerObject, LogLearning, Display,
			LocationA,
			LocationB,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_LOCATION(VisualLoggerObject, LogLearning, Display,
			LocationB,
			10,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocationA: [% 6.1f % 6.1f % 6.1f]\nLocationB: [% 6.1f % 6.1f % 6.1f]\nDifference: [% 6.2f]\nLocationScale: [% 6.2f]\nSimilarity: [% 6.2f]\nScale: [% 6.2f]\nReward: [% 6.2f]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			LocationA.X, LocationA.Y, LocationA.Z,
			LocationB.X, LocationB.Y, LocationB.Z,
			LocationDifference,
			LocationScale,
			Similarity,
			RewardScale,
			Reward);
	}
#endif

	return Reward;
}

float ULearningAgentsRewards::MakeRewardFromAngleSimilarity(
	const float AngleA, 
	const float AngleB, 
	const float RewardScale,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerAngleLocationA,
	const FVector VisualLoggerAngleLocationB,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	const float AngleDifference = FMath::FindDeltaAngleDegrees(AngleA, AngleB);
	const float Similarity = 1.0f - FMath::Abs(AngleDifference) / 180.0f;
	const float Reward = Similarity * RewardScale;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ANGLE_DEGREES(
			VisualLoggerObject,
			LogLearning,
			Display,
			AngleA,
			0.0f,
			VisualLoggerAngleLocationA,
			5.0f,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_ANGLE_DEGREES(
			VisualLoggerObject,
			LogLearning,
			Display,
			AngleB,
			0.0f,
			VisualLoggerAngleLocationB,
			5.0f,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nAngleA [% 6.2f]\nAngleB [% 6.2f]\nDifference: [% 6.2f]\nSimilarity: [% 6.2f]\nScale: [% 6.2f]\nReward: [% 6.2f]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			AngleA,
			AngleB,
			AngleDifference,
			Similarity,
			RewardScale,
			Reward);
	}
#endif

	return Reward;
}

float ULearningAgentsRewards::MakeRewardFromRotationSimilarityAsQuats(
	const FQuat RotationA, 
	const FQuat RotationB, 
	const float RewardScale,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerRotationLocationA,
	const FVector VisualLoggerRotationLocationB,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	FQuat Difference = RotationA.Inverse() * RotationB;
	Difference.EnforceShortestArcWith(FQuat::Identity);
	const float AngleDifference = FMath::RadiansToDegrees(Difference.GetAngle());
	const float Similarity = 1.0f - FMath::Abs(AngleDifference) / 180.0f;
	const float Reward = Similarity * RewardScale;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(
			VisualLoggerObject,
			LogLearning,
			Display,
			VisualLoggerRotationLocationA,
			RotationA,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_TRANSFORM(
			VisualLoggerObject,
			LogLearning,
			Display,
			VisualLoggerRotationLocationB,
			RotationB,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nRotationA [% 6.2f % 6.2f % 6.2f % 6.2f]\nRotationB [% 6.2f % 6.2f % 6.2f % 6.2f]\nDifference: [% 6.2f]\nSimilarity: [% 6.2f]\nScale: [% 6.2f]\nReward: [% 6.2f]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			RotationA.X, RotationA.Y, RotationA.Z, RotationA.W,
			RotationB.X, RotationB.Y, RotationB.Z, RotationB.W,
			AngleDifference,
			Similarity,
			RewardScale,
			Reward);
	}
#endif

	return Reward;
}

float ULearningAgentsRewards::MakeRewardFromRotationSimilarity(
	const FRotator RotationA, 
	const FRotator RotationB, 
	const float RewardScale,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerRotationLocationA,
	const FVector VisualLoggerRotationLocationB,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	return MakeRewardFromRotationSimilarityAsQuats(
		RotationA.Quaternion(), 
		RotationB.Quaternion(), 
		RewardScale, 
		Tag, 
		bVisualLoggerEnabled, 
		VisualLoggerListener, 
		VisualLoggerAgentId,
		VisualLoggerRotationLocationA,
		VisualLoggerRotationLocationB,
		VisualLoggerLocation, 
		VisualLoggerColor);
}

float ULearningAgentsRewards::MakeRewardFromDirectionSimilarity(
	const FVector DirectionA, 
	const FVector DirectionB, 
	const float RewardScale,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerDirectionLocationA,
	const FVector VisualLoggerDirectionLocationB,
	const FVector VisualLoggerLocation,
	const float VisualLoggerArrowLength,
	const FLinearColor VisualLoggerColor)
{
	const float AngleDifference = FMath::RadiansToDegrees(FMath::Acos(DirectionA.GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector).Dot(DirectionB.GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector))));
	const float Similarity = 1.0f - FMath::Abs(AngleDifference) / 180.0f;
	const float Reward = Similarity * RewardScale;

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ARROW(
			VisualLoggerObject,
			LogLearning,
			Display,
			VisualLoggerDirectionLocationA,
			VisualLoggerDirectionLocationA + VisualLoggerArrowLength * DirectionA,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_ARROW(
			VisualLoggerObject,
			LogLearning,
			Display,
			VisualLoggerDirectionLocationB,
			VisualLoggerDirectionLocationB + VisualLoggerArrowLength * DirectionB,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nDirectionA [% 6.2f % 6.2f % 6.2f]\nDirectionB [% 6.2f % 6.2f % 6.2f]\nDifference: [% 6.2f]\nSimilarity: [% 6.2f]\nScale: [% 6.2f]\nReward: [% 6.2f]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			DirectionA.X, DirectionA.Y, DirectionA.Z,
			DirectionB.X, DirectionB.Y, DirectionB.Z,
			AngleDifference,
			Similarity,
			RewardScale,
			Reward);
	}
#endif

	return Reward;
}

// Spline Rewards

float ULearningAgentsRewards::MakeRewardFromVelocityAlongSpline(
	const USplineComponent* SplineComponent, 
	const FVector Location, 
	const FVector Velocity, 
	const float VelocityScale, 
	const float RewardScale, 
	const float FiniteDifferenceDelta,
	const FName Tag,
	const bool bVisualLoggerEnabled,
	ULearningAgentsManagerListener* VisualLoggerListener,
	const int32 VisualLoggerAgentId,
	const FVector VisualLoggerLocation,
	const FLinearColor VisualLoggerColor)
{
	if (!SplineComponent)
	{
		UE_LOGF(LogLearning, Error, "MakeRewardFromVelocityAlongSpline: SplineComponent is nullptr.");
		return 0.0f;
	}

	float FiniteDiff = FiniteDifferenceDelta;

	if (FiniteDiff < UE_KINDA_SMALL_NUMBER)
	{
		UE_LOGF(LogLearning, Warning, "MakeRewardFromVelocityAlongSpline: FiniteDifferenceDelta is too small (%6.4f). Clamping to %6.4f.", FiniteDifferenceDelta, UE_KINDA_SMALL_NUMBER);
		FiniteDiff = UE_KINDA_SMALL_NUMBER;
	}

	const float RawDistance0 = SplineComponent->GetDistanceAlongSplineAtLocation(Location, ESplineCoordinateSpace::World);
	const float RawDistance1 = SplineComponent->GetDistanceAlongSplineAtLocation(Location + FiniteDiff * Velocity.GetSafeNormal(), ESplineCoordinateSpace::World);

	float Distance0 = RawDistance0, Distance1 = RawDistance1;

	if (SplineComponent->IsClosedLoop())
	{
		const float SplineDistance = SplineComponent->GetSplineLength();

		if (FMath::Abs(Distance0 - (Distance1 + SplineDistance)) < FMath::Abs(Distance0 - Distance1))
		{
			Distance1 = Distance1 + SplineDistance;
		}
		else if (FMath::Abs((Distance0 + SplineDistance) - Distance1) < FMath::Abs(Distance0 - Distance1))
		{
			Distance0 = Distance0 + SplineDistance;
		}
	}

	const float SplineVelocity = ((Distance1 - Distance0) / FiniteDiff) * Velocity.Length();
	const float Reward = RewardScale * (SplineVelocity / FMath::Max(VelocityScale, UE_SMALL_NUMBER));

#if UE_LEARNING_AGENTS_ENABLE_VISUAL_LOG
	if (bVisualLoggerEnabled && VisualLoggerListener)
	{
		const FVector SplineLocation0 = SplineComponent->GetLocationAtDistanceAlongSpline(RawDistance0, ESplineCoordinateSpace::World);
		const FVector SplineLocation1 = SplineComponent->GetLocationAtDistanceAlongSpline(RawDistance1, ESplineCoordinateSpace::World);
		const FVector SplineVelocityDirection = (SplineLocation1 - SplineLocation0).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);

		const ULearningAgentsVisualLoggerObject* VisualLoggerObject = VisualLoggerListener->GetOrAddVisualLoggerObject(Tag);

		UE_LEARNING_AGENTS_VLOG_ARROW(
			VisualLoggerObject,
			LogLearning,
			Display,
			SplineLocation0,
			SplineLocation0 + FMath::Abs(SplineVelocity) * SplineVelocityDirection,
			VisualLoggerColor.ToFColor(true),
			TEXT(""));

		UE_LEARNING_AGENTS_VLOG_STRING(VisualLoggerObject, LogLearning, Display, VisualLoggerLocation,
			VisualLoggerColor.ToFColor(true),
			TEXT("Listener: %s\nTag: %s\nAgent Id: % 3i\nLocation [% 6.2f % 6.2f % 6.2f]\nVelocity [% 6.2f % 6.2f % 6.2f]\nVelocity Along Spline: [% 6.2f]\nVelocity Scale: [% 6.2f]\nScale: [% 6.2f]\nReward: [% 6.2f]"),
			*VisualLoggerListener->GetName(),
			*Tag.ToString(),
			VisualLoggerAgentId,
			Location.X, Location.Y, Location.Z,
			Velocity.X, Velocity.Y, Velocity.Z,
			SplineVelocity,
			VelocityScale,
			RewardScale,
			Reward);
	}
#endif

	return Reward;
}
