// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsTrainer.h"

#include "Misc/Paths.h"
#include "Interfaces/IPluginManager.h"

#include "Misc/App.h"
#include "GameFramework/GameUserSettings.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Engine/GameViewportClient.h"

#if WITH_EDITOR
#include "Editor/EditorPerformanceSettings.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsTrainer)

namespace UE::Learning::Agents
{
	ELearningAgentsTrainingDevice GetLearningAgentsTrainingDevice(const ETrainerDevice Device)
	{
		switch (Device)
		{
		case ETrainerDevice::CPU: return ELearningAgentsTrainingDevice::CPU;
		case ETrainerDevice::GPU: return ELearningAgentsTrainingDevice::GPU;
		default:UE_LOGF(LogLearning, Error, "Unknown Trainer Device."); return ELearningAgentsTrainingDevice::CPU;
		}
	}

	ETrainerDevice GetTrainingDevice(const ELearningAgentsTrainingDevice Device)
	{
		switch (Device)
		{
		case ELearningAgentsTrainingDevice::CPU: return ETrainerDevice::CPU;
		case ELearningAgentsTrainingDevice::GPU: return ETrainerDevice::GPU;
		default:UE_LOGF(LogLearning, Error, "Unknown Trainer Device."); return ETrainerDevice::CPU;
		}
	}

	void ApplyGameSettings(const FLearningAgentsTrainingGameSettings& Settings, const UWorld* World, FGameSettingsState& OutGameSettingsState)
	{
		// Record GameState Settings

		OutGameSettingsState.bFixedTimestepUsed = FApp::UseFixedTimeStep();
		OutGameSettingsState.FixedTimeStepDeltaTime = FApp::GetFixedDeltaTime();

		UGameUserSettings* GameSettings = UGameUserSettings::GetGameUserSettings();
		if (GameSettings)
		{
			OutGameSettingsState.bVSyncEnabled = GameSettings->IsVSyncEnabled();
		}

		UPhysicsSettings* PhysicsSettings = UPhysicsSettings::Get();
		if (PhysicsSettings)
		{
			OutGameSettingsState.MaxPhysicsStep = PhysicsSettings->MaxPhysicsDeltaTime;
		}

		IConsoleVariable* MaxFPSCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("t.MaxFPS"));
		if (MaxFPSCVar)
		{
			OutGameSettingsState.MaxFPS = MaxFPSCVar->GetInt();
		}

		UGameViewportClient* ViewportClient = World ? World->GetGameViewport() : nullptr;
		if (ViewportClient)
		{
			OutGameSettingsState.ViewModeIndex = ViewportClient->ViewModeIndex;
		}

#if WITH_EDITOR
		UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();
		if (EditorPerformanceSettings)
		{
			OutGameSettingsState.bUseLessCPUInTheBackground = EditorPerformanceSettings->bThrottleCPUWhenNotForeground;
			OutGameSettingsState.bEditorVSyncEnabled = EditorPerformanceSettings->bEnableVSync;
		}
#endif

		// Apply Training GameState Settings

		FApp::SetUseFixedTimeStep(Settings.bUseFixedTimeStep);

		if (Settings.FixedTimeStepFrequency > UE_SMALL_NUMBER)
		{
			FApp::SetFixedDeltaTime(1.0f / Settings.FixedTimeStepFrequency);
			if (Settings.bSetMaxPhysicsStepToFixedTimeStep && PhysicsSettings)
			{
				PhysicsSettings->MaxPhysicsDeltaTime = 1.0f / Settings.FixedTimeStepFrequency;
			}
		}
		else
		{
			UE_LOGF(LogLearning, Warning, "Provided invalid FixedTimeStepFrequency: %0.5f", Settings.FixedTimeStepFrequency);
		}

		if (Settings.bDisableMaxFPS && MaxFPSCVar)
		{
			MaxFPSCVar->Set(0);
		}

		if (Settings.bDisableVSync && GameSettings)
		{
			GameSettings->SetVSyncEnabled(false);
			GameSettings->ApplySettings(false);
		}

		if (Settings.bUseUnlitViewportRendering && ViewportClient)
		{
			ViewportClient->ViewModeIndex = EViewModeIndex::VMI_Unlit;
		}

#if WITH_EDITOR
		if (Settings.bDisableUseLessCPUInTheBackground && EditorPerformanceSettings)
		{
			EditorPerformanceSettings->bThrottleCPUWhenNotForeground = false;
			EditorPerformanceSettings->PostEditChange();
		}

		if (Settings.bDisableEditorVSync && EditorPerformanceSettings)
		{
			EditorPerformanceSettings->bEnableVSync = false;
			EditorPerformanceSettings->PostEditChange();
		}
#endif
	}

	void RevertGameSettings(const FGameSettingsState& Settings, const UWorld* World)
	{
		FApp::SetUseFixedTimeStep(Settings.bFixedTimestepUsed);
		FApp::SetFixedDeltaTime(Settings.FixedTimeStepDeltaTime);
		UGameUserSettings* GameSettings = UGameUserSettings::GetGameUserSettings();
		if (GameSettings)
		{
			GameSettings->SetVSyncEnabled(Settings.bVSyncEnabled);
			GameSettings->ApplySettings(true);
		}

		UPhysicsSettings* PhysicsSettings = UPhysicsSettings::Get();
		if (PhysicsSettings)
		{
			PhysicsSettings->MaxPhysicsDeltaTime = Settings.MaxPhysicsStep;
		}

		IConsoleVariable* MaxFPSCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("t.MaxFPS"));
		if (MaxFPSCVar)
		{
			MaxFPSCVar->Set(Settings.MaxFPS);
		}

		UGameViewportClient* ViewportClient = World ? World->GetGameViewport() : nullptr;
		if (ViewportClient)
		{
			ViewportClient->ViewModeIndex = Settings.ViewModeIndex;
		}

#if WITH_EDITOR
		UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();
		if (EditorPerformanceSettings)
		{
			EditorPerformanceSettings->bThrottleCPUWhenNotForeground = Settings.bUseLessCPUInTheBackground;
			EditorPerformanceSettings->bEnableVSync = Settings.bEditorVSyncEnabled;
			EditorPerformanceSettings->PostEditChange();
		}
#endif
	}
}

FLearningAgentsTrainerProcessSettings::FLearningAgentsTrainerProcessSettings()
{
	EditorEngineRelativePath.Path = FPaths::EngineDir();
	EditorIntermediateRelativePath.Path = FPaths::ProjectIntermediateDir();
}

FString FLearningAgentsTrainerProcessSettings::GetEditorEnginePath() const
{
	FString CmdLineEnginePath;
	bool bEnginePathOverridden = FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsEnginePath="), CmdLineEnginePath);
	if (bEnginePathOverridden)
	{
		UE_LOGF(LogLearning, Display, "Overriding Engine Path with value from the cmdline: %ls", *CmdLineEnginePath);
		return CmdLineEnginePath;
	}

#if WITH_EDITOR
	return EditorEngineRelativePath.Path;
#else
	if (NonEditorEngineRelativePath.IsEmpty())
	{
		UE_LOGF(LogLearning, Warning, "GetEditorEnginePath: NonEditorEngineRelativePath not set");
	}

	return NonEditorEngineRelativePath;
#endif
}

FString FLearningAgentsTrainerProcessSettings::GetIntermediatePath() const
{
	FString CmdLineIntermediatePath;
	bool bIntermediatePathOverridden = FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsIntermediatePath="), CmdLineIntermediatePath);
	if (bIntermediatePathOverridden)
	{
		UE_LOGF(LogLearning, Display, "Overriding Intermediate Path with value from the cmdline: %ls", *CmdLineIntermediatePath);
		return CmdLineIntermediatePath;
	}

#if WITH_EDITOR
	return EditorIntermediateRelativePath.Path;
#else
	if (NonEditorIntermediateRelativePath.IsEmpty())
	{
		UE_LOGF(LogLearning, Warning, "GetIntermediatePath: NonEditorIntermediateRelativePath not set");
	}

	return NonEditorIntermediateRelativePath;
#endif
}

FString FLearningAgentsTrainerProcessSettings::GetCustomTrainerModulePath() const
{
	FString CmdLineCustomTrainerPath;
	bool bCustomTrainerPathOverridden = FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsCustomTrainerPath="), CmdLineCustomTrainerPath);
	if (bCustomTrainerPathOverridden)
	{
		UE_LOGF(LogLearning, Display, "Overriding Custom Trainer Module Path with value from the cmdline: %ls", *CmdLineCustomTrainerPath);
		return CmdLineCustomTrainerPath;
	}

#if WITH_EDITOR
	return CustomTrainerModulePath.Path;
#else
	if (NonEditorCustomTrainerModulePath.IsEmpty())
	{
		UE_LOGF(LogLearning, Warning, "GetCustomTrainerModulePath: NonEditorCustomTrainerModulePath not set");
	}

	return NonEditorCustomTrainerModulePath;
#endif
}

FString FLearningAgentsTrainerProcessSettings::GetNneCpuContentPath() const
{
	FString CmdLineNneCpuPath;
	if (FParse::Value(FCommandLine::Get(), TEXT("LearningAgentsNneCpuPath="), CmdLineNneCpuPath))
	{
		UE_LOGF(LogLearning, Display, "Overriding NNE CPU Path with value from the cmdline: %ls", *CmdLineNneCpuPath);
		return CmdLineNneCpuPath;
	}

	TSharedPtr<IPlugin> NNERuntimeBasicCpuPlugin = IPluginManager::Get().FindEnabledPlugin(TEXT("NNERuntimeBasicCpu"));
	if (NNERuntimeBasicCpuPlugin.IsValid())
	{
		return FPaths::Combine(NNERuntimeBasicCpuPlugin->GetContentDir(), TEXT("Python"));
	}

	UE_LOGF(LogLearning, Error, "GetNneCpuContentPath: NNERuntimeBasicCpu plugin not found");
	return FString();
}
