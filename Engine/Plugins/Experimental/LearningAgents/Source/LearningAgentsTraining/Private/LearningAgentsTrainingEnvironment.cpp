// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsTrainingEnvironment.h"

#include "LearningAgentsManager.h"
#include "LearningCompletion.h"
#include "LearningLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsTrainingEnvironment)

ULearningAgentsTrainingEnvironment::ULearningAgentsTrainingEnvironment() : Super(FObjectInitializer::Get()) {}
ULearningAgentsTrainingEnvironment::ULearningAgentsTrainingEnvironment(FVTableHelper& Helper) : Super(Helper) {}
ULearningAgentsTrainingEnvironment::~ULearningAgentsTrainingEnvironment() = default;

ULearningAgentsTrainingEnvironment* ULearningAgentsTrainingEnvironment::MakeTrainingEnvironment(
	ULearningAgentsManager*& InManager,
	TSubclassOf<ULearningAgentsTrainingEnvironment> Class,
	const FName Name)
{
	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "MakeTrainer: InManager is nullptr.");
		return nullptr;
	}

	if (!Class)
	{
		UE_LOGF(LogLearning, Error, "MakeTrainer: Class is nullptr.");
		return nullptr;
	}

	const FName UniqueName = MakeUniqueObjectName(InManager, Class, Name, EUniqueObjectNameOptions::GloballyUnique);

	ULearningAgentsTrainingEnvironment* TrainingEnvironment = NewObject<ULearningAgentsTrainingEnvironment>(InManager, Class, UniqueName);
	if (!TrainingEnvironment) { return nullptr; }

	TrainingEnvironment->SetupTrainingEnvironment(InManager);

	return TrainingEnvironment->IsSetup() ? TrainingEnvironment : nullptr;
}

void ULearningAgentsTrainingEnvironment::SetupTrainingEnvironment(ULearningAgentsManager*& InManager)
{
	if (IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup already run!", *GetName());
		return;
	}

	if (!InManager)
	{
		UE_LOGF(LogLearning, Error, "%ls: InManager is nullptr.", *GetName());
		return;
	}

	Manager = InManager;

	// Create Reset Buffer
	ResetBuffer = MakeUnique<UE::Learning::FResetInstanceBuffer>();
	ResetBuffer->Reserve(Manager->GetMaxAgentNum());

	// Create Rewards and Completions Buffers
	Rewards.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	AgentCompletions.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	EpisodeCompletions.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	AllCompletions.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	EpisodeTimes.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	UE::Learning::Array::Set<1, float>(Rewards, FLT_MAX);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(AgentCompletions, UE::Learning::ECompletionMode::Terminated);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(EpisodeCompletions, UE::Learning::ECompletionMode::Terminated);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(AllCompletions, UE::Learning::ECompletionMode::Terminated);
	UE::Learning::Array::Set<1, float>(EpisodeTimes, FLT_MAX);

	// Reset Agent iteration
	RewardIteration.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	CompletionIteration.SetNumUninitialized({ Manager->GetMaxAgentNum() });
	UE::Learning::Array::Set<1, uint64>(RewardIteration, INDEX_NONE);
	UE::Learning::Array::Set<1, uint64>(CompletionIteration, INDEX_NONE);

	bIsSetup = true;

	Manager->AddListener(this);
}

void ULearningAgentsTrainingEnvironment::OnAgentsAdded_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Array::Set<1, float>(Rewards, 0.0f, AgentIds);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(AgentCompletions, UE::Learning::ECompletionMode::Running, AgentIds);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(EpisodeCompletions, UE::Learning::ECompletionMode::Running, AgentIds);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(AllCompletions, UE::Learning::ECompletionMode::Running, AgentIds);
	UE::Learning::Array::Set<1, uint64>(RewardIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, uint64>(CompletionIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, float>(EpisodeTimes, 0.0f, AgentIds);
}

void ULearningAgentsTrainingEnvironment::OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	UE::Learning::Array::Set<1, float>(Rewards, FLT_MAX, AgentIds);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(AgentCompletions, UE::Learning::ECompletionMode::Terminated, AgentIds);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(EpisodeCompletions, UE::Learning::ECompletionMode::Terminated, AgentIds);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(AllCompletions, UE::Learning::ECompletionMode::Terminated, AgentIds);
	UE::Learning::Array::Set<1, uint64>(RewardIteration, INDEX_NONE, AgentIds);
	UE::Learning::Array::Set<1, uint64>(CompletionIteration, INDEX_NONE, AgentIds);
	UE::Learning::Array::Set<1, float>(EpisodeTimes, FLT_MAX, AgentIds);
}

void ULearningAgentsTrainingEnvironment::OnAgentsReset_Implementation(const TArray<int32>& AgentIds)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	ResetAgentEpisodes(AgentIds);

	UE::Learning::Array::Set<1, float>(Rewards, 0.0f, AgentIds);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(AgentCompletions, UE::Learning::ECompletionMode::Running, AgentIds);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(EpisodeCompletions, UE::Learning::ECompletionMode::Running, AgentIds);
	UE::Learning::Array::Set<1, UE::Learning::ECompletionMode>(AllCompletions, UE::Learning::ECompletionMode::Running, AgentIds);
	UE::Learning::Array::Set<1, uint64>(RewardIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, uint64>(CompletionIteration, 0, AgentIds);
	UE::Learning::Array::Set<1, float>(EpisodeTimes, 0.0f, AgentIds);
}

void ULearningAgentsTrainingEnvironment::OnAgentsManagerTick_Implementation(const TArray<int32>& AgentIds, const float DeltaTime)
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	for (const int32 AgentId : AgentIds)
	{
		EpisodeTimes[AgentId] += DeltaTime;
	}
}

void ULearningAgentsTrainingEnvironment::GatherAgentReward_Implementation(float& OutReward, const int32 AgentId)
{
	UE_LOGF(LogLearning, Error, "%ls: GatherAgentReward function must be overridden!", *GetName());
	OutReward = 0.0f;
}

void ULearningAgentsTrainingEnvironment::GatherAgentRewards_Implementation(TArray<float>& OutRewards, const TArray<int32>& AgentIds)
{

	OutRewards.Empty(AgentIds.Num());
	for (const int32 AgentId : AgentIds)
	{
		float OutReward = 0.0f;
		GatherAgentReward(OutReward, AgentId);
		OutRewards.Add(OutReward);
	}
}

void ULearningAgentsTrainingEnvironment::GatherAgentCompletion_Implementation(ELearningAgentsCompletion& OutCompletion, const int32 AgentId)
{
	UE_LOGF(LogLearning, Error, "%ls: GatherAgentCompletion function must be overridden!", *GetName());
	OutCompletion = ELearningAgentsCompletion::Running;
}

void ULearningAgentsTrainingEnvironment::GatherAgentCompletions_Implementation(TArray<ELearningAgentsCompletion>& OutCompletions, const TArray<int32>& AgentIds)
{

	OutCompletions.Empty(AgentIds.Num());
	for (const int32 AgentId : AgentIds)
	{
		ELearningAgentsCompletion OutCompletion = ELearningAgentsCompletion::Running;
		GatherAgentCompletion(OutCompletion, AgentId);
		OutCompletions.Add(OutCompletion);
	}
}

void ULearningAgentsTrainingEnvironment::ResetAgentEpisode_Implementation(const int32 AgentId)
{
	UE_LOGF(LogLearning, Error, "%ls: ResetAgentEpisode function must be overridden!", *GetName());
}

void ULearningAgentsTrainingEnvironment::ResetAgentEpisodes_Implementation(const TArray<int32>& AgentIds)
{
	for (const int32 AgentId : AgentIds)
	{
		ResetAgentEpisode(AgentId);
	}
}

void ULearningAgentsTrainingEnvironment::GatherRewards()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsTrainingEnvironment::GatherRewards);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	TArray<int32> ValidAgentIds = Manager->GetAllAgentIds();
	UE::Learning::FIndexSet ValidAgentSet = Manager->GetAllAgentSet();

	RewardBuffer.Empty(Manager->GetMaxAgentNum());
	GatherAgentRewards(RewardBuffer, ValidAgentIds);

	if (ValidAgentSet.Num() != RewardBuffer.Num())
	{
		UE_LOGF(LogLearning, Warning, "%ls: Not enough rewards added by GetAgentRewards. Expected %i, Got %i.", *GetName(), ValidAgentSet.Num(), RewardBuffer.Num());
		return;
	}

	for (int32 AgentIdx = 0; AgentIdx < RewardBuffer.Num(); AgentIdx++)
	{
		const float RewardValue = RewardBuffer[AgentIdx];

		if (FMath::IsFinite(RewardValue) && RewardValue != MAX_flt && RewardValue != -MAX_flt)
		{
			Rewards[ValidAgentSet[AgentIdx]] = RewardValue;
			RewardIteration[ValidAgentSet[AgentIdx]]++;
		}
		else
		{
			UE_LOGF(LogLearning, Warning, "%ls: Got invalid reward for agent %i: %f.", *GetName(), ValidAgentSet[AgentIdx], RewardValue);
			continue;
		}
	}
}

void ULearningAgentsTrainingEnvironment::GatherCompletions()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULearningAgentsTrainingEnvironment::GatherCompletions);

	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return;
	}

	if (Manager->GetAgentNum() == 0)
	{
		UE_LOGF(LogLearning, Warning, "%ls: No agents added to Manager.", *GetName());
	}

	TArray<int32> ValidAgentIds = Manager->GetAllAgentIds();
	UE::Learning::FIndexSet ValidAgentSet = Manager->GetAllAgentSet();

	CompletionBuffer.Empty(Manager->GetMaxAgentNum());
	GatherAgentCompletions(CompletionBuffer, ValidAgentIds);

	if (ValidAgentSet.Num() != CompletionBuffer.Num())
	{
		UE_LOGF(LogLearning, Warning, "%ls: Not enough completions added by GetAgentCompletions. Expected %i, Got %i.", *GetName(), ValidAgentSet.Num(), CompletionBuffer.Num());
		return;
	}

	for (int32 AgentIdx = 0; AgentIdx < CompletionBuffer.Num(); AgentIdx++)
	{
		AgentCompletions[ValidAgentSet[AgentIdx]] = UE::Learning::Agents::GetCompletionMode(CompletionBuffer[AgentIdx]);
		CompletionIteration[ValidAgentSet[AgentIdx]]++;
	}
}

bool ULearningAgentsTrainingEnvironment::HasReward(const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return false;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return false;
	}

	return RewardIteration[AgentId] > 0;
}

bool ULearningAgentsTrainingEnvironment::HasCompletion(const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return false;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return false;
	}

	return CompletionIteration[AgentId] > 0;
}

float ULearningAgentsTrainingEnvironment::GetReward(const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0.0f;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return 0.0f;
	}

	if (RewardIteration[AgentId] == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Agent with id %d has not evaluated rewards. Did you run EvaluateRewards?", *GetName(), AgentId);
		return 0.0f;
	}

	return Rewards[AgentId];
}

ELearningAgentsCompletion ULearningAgentsTrainingEnvironment::GetCompletion(const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return ELearningAgentsCompletion::Running;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return ELearningAgentsCompletion::Running;
	}

	if (CompletionIteration[AgentId] == 0)
	{
		UE_LOGF(LogLearning, Error, "%ls: Agent with id %d has not evaluated completions. Did you run EvaluateCompletions?", *GetName(), AgentId);
		return ELearningAgentsCompletion::Running;
	}

	return UE::Learning::Agents::GetLearningAgentsCompletion(AgentCompletions[AgentId]);
}

float ULearningAgentsTrainingEnvironment::GetEpisodeTime(const int32 AgentId) const
{
	if (!IsSetup())
	{
		UE_LOGF(LogLearning, Error, "%ls: Setup not complete.", *GetName());
		return 0.0f;
	}

	if (!HasAgent(AgentId))
	{
		UE_LOGF(LogLearning, Error, "%ls: AgentId %d not found in the agents set.", *GetName(), AgentId);
		return 0.0f;
	}

	return EpisodeTimes[AgentId];
}

const TLearningArrayView<1, const float> ULearningAgentsTrainingEnvironment::GetRewardArrayView() const
{
	return Rewards;
}

uint64 ULearningAgentsTrainingEnvironment::GetRewardIteration(const int32 AgentId) const
{
	return RewardIteration[AgentId];
}

UE::Learning::ECompletionMode ULearningAgentsTrainingEnvironment::GetAgentCompletion(const int32 AgentId) const
{
	return AgentCompletions[AgentId];
}

const TLearningArrayView<1, const UE::Learning::ECompletionMode> ULearningAgentsTrainingEnvironment::GetAgentCompletions() const
{
	return AgentCompletions;
}

const TLearningArrayView<1, const UE::Learning::ECompletionMode> ULearningAgentsTrainingEnvironment::GetAllCompletions() const
{
	return AllCompletions;
}

void ULearningAgentsTrainingEnvironment::SetAllCompletions(UE::Learning::FIndexSet AgentSet)
{
	for (const int32 AgentIdx : AgentSet)
	{
		AllCompletions[AgentIdx] = UE::Learning::Completion::Or(AgentCompletions[AgentIdx], EpisodeCompletions[AgentIdx]);
	}
}

TLearningArrayView<1, UE::Learning::ECompletionMode> ULearningAgentsTrainingEnvironment::GetEpisodeCompletions()
{
	return EpisodeCompletions;
}

uint64 ULearningAgentsTrainingEnvironment::GetCompletionIteration(const int32 AgentId) const
{
	return CompletionIteration[AgentId];
}

UE::Learning::FResetInstanceBuffer& ULearningAgentsTrainingEnvironment::GetResetBuffer() const
{
	return *ResetBuffer;
}
