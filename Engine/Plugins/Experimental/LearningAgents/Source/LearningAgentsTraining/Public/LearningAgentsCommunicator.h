// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningAgentsTrainer.h"

#include "Engine/EngineTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include "LearningAgentsCommunicator.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

namespace UE::Learning
{
	struct IExternalTrainer;
	struct FSharedMemoryTrainerServerProcess;
	struct FSocketTrainerServerProcess;
}

class ULearningAgentsInteractor;

/** Settings specific to shared memory communicators. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsSharedMemoryCommunicatorSettings
{
	GENERATED_BODY()

public:

	/** Time in seconds to wait for the training process before timing out. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float Timeout = 10.0f;

	/** If true, then we will attach to existing training process and skip spawning one. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bUseExternalTrainingProcess = false;

	/** The existing controls memory to attach to. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (EditCondition = "bUseExternalTrainingProcess"))
	FGuid ControlsGuid;

	/** The absolute path to the config directory. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (EditCondition = "bUseExternalTrainingProcess"))
	FDirectoryPath ConfigPath;
};

/** Settings specific to socket communicators. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsSocketCommunicatorSettings
{
	GENERATED_BODY()

public:

	/** IP Address for the socket. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	FString IpAddress = TEXT("127.0.0.1");

	/** Port for the socket. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0", ClampMax = "65535", UIMax = "65535"))
	uint32 Port = 48491;

	/** Time in seconds to wait for the training process before timing out. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float Timeout = 10.0f;

	/** If true, then we will attach to existing training process and skip spawning one. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bUseExternalTrainingProcess = false;
};

/** Blueprint-compatible wrapper struct for Shared Memory Training Process. */
USTRUCT(BlueprintType)
struct FLearningAgentsSharedMemoryTrainerProcess
{
	GENERATED_BODY()

	TSharedPtr<UE::Learning::FSharedMemoryTrainerServerProcess> TrainerProcess;
};

/** Blueprint-compatible wrapper struct for Socket Training Process. */
USTRUCT(BlueprintType)
struct FLearningAgentsSocketTrainerProcess
{
	GENERATED_BODY()

	TSharedPtr<UE::Learning::FSocketTrainerServerProcess> TrainerProcess;
};

/** Blueprint-compatible wrapper struct for IExternalTrainer. */
USTRUCT(BlueprintType)
struct FLearningAgentsCommunicator
{
	GENERATED_BODY()

	TSharedPtr<UE::Learning::IExternalTrainer> Trainer;
};

/** Contains functions for starting external trainers and communicating with them. */
UCLASS(MinimalAPI)
class ULearningAgentsCommunicatorLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Start and connect to a training sub-process which will communicate via shared memory. Shared memory has the least
	 * communication overhead so prefer this for local development.
	 * 
	 * This must be called on game thread!
	 * 
	 * You can use this in place of calling "SpawnSharedMemoryTrainingProcess" followed by "MakeSharedMemoryCommunicator"
	 * for most use-cases. 
	 *
	 * If you wish to connect to an externally launched trainer (for debugging or scale-out reasons, etc.), you can use
	 * the following command-line options to override some settings:
	 *		-LearningAgentsUseExternalTrainingProcess	If present, spawning a sub-process will be skipped.
	 *		-LearningAgentsControlsGuid={guid}			This optional guid will override the SharedMemorySettings, so you
	 *													can attach to shared memory control created by the trainer.
	 *
	 * @param TrainerProcessSettings							Settings universal to all trainer processes.
	 * @param FLearningAgentsSharedMemoryCommunicatorSettings	Settings specific to shared memory communicators.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "TrainerProcessSettings,SharedMemorySettings"))
	static UE_API FLearningAgentsCommunicator MakeSharedMemoryTrainingProcess(
		const FLearningAgentsTrainerProcessSettings& TrainerProcessSettings = FLearningAgentsTrainerProcessSettings(),
		const FLearningAgentsSharedMemoryCommunicatorSettings& SharedMemorySettings = FLearningAgentsSharedMemoryCommunicatorSettings());

	/**
	 * Start a local python training sub-process which will communicate via shared memory. Shared memory has the least
	 * communication overhead so prefer this for local development.
	 * 
	 * This must be called on game thread!
	 * 
	 * @param TrainerProcessSettings							Settings universal to all trainer processes.
	 * @param FLearningAgentsSharedMemoryCommunicatorSettings	Settings specific to shared memory communicators.
	 */ 
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "TrainerProcessSettings,SharedMemorySettings"))
	static UE_API FLearningAgentsSharedMemoryTrainerProcess SpawnSharedMemoryTrainingProcess(
		const FLearningAgentsTrainerProcessSettings& TrainerProcessSettings = FLearningAgentsTrainerProcessSettings(),
		const FLearningAgentsSharedMemoryCommunicatorSettings& SharedMemorySettings = FLearningAgentsSharedMemoryCommunicatorSettings());

	/**
	 * Create a communicator which can be used to interact with a previously started shared memory trainer process.
	 *
	 * @param TrainerProcess									The shared memory trainer process to communicate with.
	 * @param TrainerProcessSettings							Settings universal to all trainer processes.
	 * @param FLearningAgentsSharedMemoryCommunicatorSettings	Settings specific to shared memory communicators.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "TrainerProcessSettings,SharedMemorySettings"))
	static UE_API FLearningAgentsCommunicator MakeSharedMemoryCommunicator(
		FLearningAgentsSharedMemoryTrainerProcess TrainerProcess,
		const FLearningAgentsTrainerProcessSettings& TrainerProcessSettings = FLearningAgentsTrainerProcessSettings(),
		const FLearningAgentsSharedMemoryCommunicatorSettings& SharedMemorySettings = FLearningAgentsSharedMemoryCommunicatorSettings());

	/**
	 * Start and connect to a training sub-process which will communicate via sockets. Sockets have some overhead
	 * compared to shared memory but can work over networked connects. This provides no encryption so do not use on
	 * public internet if privacy is a concern.
	 *
	 * This must be called on game thread!
	 * 
	 * If you wish to connect to an externally launched trainer (for debugging or scale-out reasons, etc.), you can use
	 * the following command-line options to override some settings:
	 *		-LearningAgentsUseExternalTrainingProcess	If present, spawning a sub-process will be skipped.
	 *		-LearningAgentsIpAddress=					This optional IP address will override the SocketSettings
	 *		-LearningAgentsPort=						This optional Port will override the SocketSettings
	 *
	 * @param TrainerProcessSettings					Settings universal to all trainer processes.
	 * @param FLearningAgentsSocketCommunicatorSettings	Settings specific to socket communicators.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "TrainerProcessSettings,SocketSettings"))
	static UE_API FLearningAgentsCommunicator MakeSocketTrainingProcess(
		const FLearningAgentsTrainerProcessSettings& TrainerProcessSettings = FLearningAgentsTrainerProcessSettings(),
		const FLearningAgentsSocketCommunicatorSettings& SocketSettings = FLearningAgentsSocketCommunicatorSettings());

	/**
	 * Start a local python training sub-process which will communicate via sockets. Sockets have some overhead
	 * compared to shared memory but can work over networked connects. This provides no encryption so do not use on
	 * public internet if privacy is a concern.
	 *
	 * This must be called on game thread!
	 *
	 * @param TrainerProcessSettings					Settings universal to all trainer processes.
	 * @param FLearningAgentsSocketCommunicatorSettings	Settings specific to socket communicators.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "TrainerProcessSettings,SocketSettings"))
	static UE_API FLearningAgentsSocketTrainerProcess SpawnSocketTrainingProcess(
		const FLearningAgentsTrainerProcessSettings& TrainerProcessSettings = FLearningAgentsTrainerProcessSettings(),
		const FLearningAgentsSocketCommunicatorSettings& SocketSettings = FLearningAgentsSocketCommunicatorSettings());

	/**
	 * Create a communicator which can be used to interact with a previously started socket trainer process.
	 *
	 * @param TrainerProcess							The socket trainer process to communicate with (optional).
	 * @param TrainerProcessSettings					Settings universal to all trainer processes.
	 * @param FLearningAgentsSocketCommunicatorSettings	Settings specific to socket communicators.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "SocketSettings"))
	static UE_API FLearningAgentsCommunicator MakeSocketCommunicator(
		FLearningAgentsSocketTrainerProcess TrainerProcess,
		const FLearningAgentsSocketCommunicatorSettings& SocketSettings = FLearningAgentsSocketCommunicatorSettings());
};

#undef UE_API
