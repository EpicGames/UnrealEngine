// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "LearningAgentsCompletions.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

class ULearningAgentsManagerListener;

namespace UE::Learning
{
	enum class ECompletionMode : uint8;
}

/** Completion modes for episodes. */
UENUM(BlueprintType, Category = "LearningAgents", meta = (ScriptName = "LearningAgentsCompletionEnum"))
enum class ELearningAgentsCompletion : uint8
{
	/** Episode is still running. */
	Running 	UMETA(DisplayName = "Running"),

	/** Episode ended while in progress. Critic will be used to estimate final return. */
	Truncation	UMETA(DisplayName = "Truncation"),

	/** Episode ended and zero reward was expected for all future steps. */
	Termination	UMETA(DisplayName = "Termination"),
};

namespace UE::Learning::Agents
{
	/** Get the learning agents completion from the UE::Learning completion. */
	LEARNINGAGENTSTRAINING_API ELearningAgentsCompletion GetLearningAgentsCompletion(const ECompletionMode CompletionMode);

	/** Get the UE::Learning completion from the learning agents completion. */
	LEARNINGAGENTSTRAINING_API ECompletionMode GetCompletionMode(const ELearningAgentsCompletion Completion);
}

UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsCompletions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Returns true if a completion is running, otherwise false. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	static UE_API bool IsCompletionRunning(const ELearningAgentsCompletion Completion);

	/** Returns true if a completion is either truncated or terminated, otherwise false. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	static UE_API bool IsCompletionCompleted(const ELearningAgentsCompletion Completion);

	/** Returns true if a completion is truncated, otherwise false. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	static UE_API bool IsCompletionTruncation(const ELearningAgentsCompletion Completion);

	/** Returns true if a completion is terminated, otherwise false. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	static UE_API bool IsCompletionTermination(const ELearningAgentsCompletion Completion);


	/** Returns a termination if either input is a termination, otherwise a truncation if either input is a truncation, otherwise returns running. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (CommutativeAssociativeBinaryOperator, DisplayName="Completion OR", CompactNodeTitle = "OR"))
	static UE_API ELearningAgentsCompletion CompletionOr(ELearningAgentsCompletion A, ELearningAgentsCompletion B);

	/** Returns a termination if both inputs are a termination, otherwise a truncation if both inputs are either a truncation or termination, otherwise returns running. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (CommutativeAssociativeBinaryOperator, DisplayName = "Completion AND", CompactNodeTitle="AND"))
	static UE_API ELearningAgentsCompletion CompletionAnd(ELearningAgentsCompletion A, ELearningAgentsCompletion B);

	/** Returns running if the input A is either a termination or truncation, otherwise returns the completion specified by NotRunningType */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (DisplayName = "Completion NOT", CompactNodeTitle = "NOT"))
	static UE_API ELearningAgentsCompletion CompletionNot(ELearningAgentsCompletion A, ELearningAgentsCompletion NotRunningType = ELearningAgentsCompletion::Termination);

	/**
	 * Make a completion.
	 *
	 * @param CompletionType The type of completion to make.
	 * @param Tag The tag for the completion. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this completion. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this completion.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting completion.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 1, DefaultToSelf = "VisualLoggerListener"))
	static UE_API ELearningAgentsCompletion MakeCompletion(
		const ELearningAgentsCompletion CompletionType = ELearningAgentsCompletion::Termination,
		const FName Tag = TEXT("Completion"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Yellow);

	/**
	 * Make a completion based on some condition.
	 *
	 * @param bCondition When true, returns the given CompletionType, otherwise returns Running.
	 * @param CompletionType The type of completion to make.
	 * @param Tag The tag for the completion. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this completion. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this completion.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting completion.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, DefaultToSelf = "VisualLoggerListener"))
	static UE_API ELearningAgentsCompletion MakeCompletionOnCondition(
		const bool bCondition, 
		const ELearningAgentsCompletion CompletionType = ELearningAgentsCompletion::Termination,
		const FName Tag = TEXT("ConditionCompletion"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Yellow);

	/**
	 * Make a completion when a time goes above a threshold.
	 *
	 * @param Time The current time.
	 * @param TimeThreshold The time threshold above which to complete with the given CompletionType.
	 * @param CompletionType The type of completion to make
	 * @param Tag The tag for the completion. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this completion. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this completion.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting completion.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, DefaultToSelf = "VisualLoggerListener"))
	static UE_API ELearningAgentsCompletion MakeCompletionOnTimeElapsed(
		const float Time, 
		const float TimeThreshold = 10.0f, 
		const ELearningAgentsCompletion CompletionType = ELearningAgentsCompletion::Truncation,
		const FName Tag = TEXT("TimeElapsedCompletion"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Yellow);

	/**
	 * Make a completion when the number of episode steps recorded exceeds some threshold.
	 *
	 * @param EpisodeSteps The number of steps recorded.
	 * @param MaxEpisodeSteps The step threshold above which to complete with the given CompletionType.
	 * @param CompletionType The type of completion to make
	 * @param Tag The tag for the completion. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this completion. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this completion.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting completion.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, DefaultToSelf = "VisualLoggerListener"))
	static UE_API ELearningAgentsCompletion MakeCompletionOnEpisodeStepsRecorded(
		const int32 EpisodeSteps, 
		const int32 MaxEpisodeSteps = 64, 
		const ELearningAgentsCompletion CompletionType = ELearningAgentsCompletion::Truncation,
		const FName Tag = TEXT("EpisodeStepsRecordedCompletion"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Yellow);

	/**
	 * Make a completion when the distance between two locations is below some threshold.
	 *
	 * @param LocationA The first location.
	 * @param LocationB The second location.
	 * @param DistanceThreshold The distance threshold.
	 * @param CompletionType The type of completion to make
	 * @param Tag The tag for the completion. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this completion. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this completion.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting completion.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4, DefaultToSelf = "VisualLoggerListener"))
	static UE_API ELearningAgentsCompletion MakeCompletionOnLocationDifferenceBelowThreshold(
		const FVector LocationA, 
		const FVector LocationB, 
		const float DistanceThreshold = 100.0f, 
		const ELearningAgentsCompletion CompletionType = ELearningAgentsCompletion::Termination,
		const FName Tag = TEXT("LocationDifferenceBelowThresholdCompletion"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Yellow);

	/**
	 * Make a completion when the distance between two locations is above some threshold.
	 *
	 * @param LocationA The first location.
	 * @param LocationB The second location.
	 * @param DistanceThreshold The distance threshold.
	 * @param CompletionType The type of completion to make
	 * @param Tag The tag for the completion. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this completion. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this completion.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting completion.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4, DefaultToSelf = "VisualLoggerListener"))
	static UE_API ELearningAgentsCompletion MakeCompletionOnLocationDifferenceAboveThreshold(
		const FVector LocationA, 
		const FVector LocationB, 
		const float DistanceThreshold = 100.0f, 
		const ELearningAgentsCompletion CompletionType = ELearningAgentsCompletion::Termination,
		const FName Tag = TEXT("LocationDifferenceAboveThresholdCompletion"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Yellow);

	/**
	 * Make a completion when a location moves outside of sound bounds.
	 *
	 * @param Location The location.
	 * @param BoundsTransform The transform of the bounds object.
	 * @param BoundsMins The minimums of the bounds object.
	 * @param BoundsMaxs The maximums of the bounds object.
	 * @param CompletionType The type of completion to make
	 * @param Tag The tag for the completion. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this completion. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this completion.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting completion.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 5, DefaultToSelf = "VisualLoggerListener"))
	static UE_API ELearningAgentsCompletion MakeCompletionOnLocationOutsideBounds(
		const FVector Location, 
		const FTransform BoundsTransform = FTransform(),
		const FVector BoundsMins = FVector(-100.0f, -100.0f, -100.0f),
		const FVector BoundsMaxs = FVector(+100.0f, +100.0f, +100.0f),
		const ELearningAgentsCompletion CompletionType = ELearningAgentsCompletion::Termination,
		const FName Tag = TEXT("LocationOutsideBoundsCompletion"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Yellow);
};

#undef UE_API
