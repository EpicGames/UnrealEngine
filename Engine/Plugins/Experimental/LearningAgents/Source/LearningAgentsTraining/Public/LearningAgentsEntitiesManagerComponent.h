// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <tuple>
#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "LearningAgentsLearningComponentInterface.h"
#include "LearningAgentsEntitiesManagerComponent.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

class ILearningAgentsEntityInterface;

/**
 * FLearningAgentsEntityInfo holds spawn information of a single entity type.
 */
USTRUCT(BlueprintType)
struct FLearningAgentsEntityInfo
{
	GENERATED_BODY()

	/** Specify the entity class to spawn. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents")
	TSubclassOf<AActor> EntityClass;

	/** Specify the Z offset to apply to spawn locations. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents")
	float EntitySpawnZOffset = 0.0f;

	/** The min number of entities to spawn at the start of an episode. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0"))
	int32 EpisodeEntitySpawnCountMin = 0;

	/** The max number of entities to spawn at the start of an episode. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0"))
	int32 EpisodeEntitySpawnCountMax = 0;
};

/**
 * FSpawnedEntitiesInfo holds references to spawned entities.
 */
USTRUCT(BlueprintType)
struct FSpawnedEntitiesInfo
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "LearningAgents")
	FLearningAgentsEntityInfo EntityInfo;

	UPROPERTY(VisibleAnywhere, Category = "LearningAgents")
	TArray<TScriptInterface<ILearningAgentsEntityInterface>> SpawnedEntities;
};

/**
 * ULearningAgentsEntitiesManagerComponent handles the spawn and reset of multiple entity types during training.
 */
UCLASS(MinimalAPI, ClassGroup=(LearningAgents), meta=(BlueprintSpawnableComponent))
class ULearningAgentsEntitiesManagerComponent : public UActorComponent, public ILearningAgentsLearningComponentInterface
{
	GENERATED_BODY()

public:
	/** Specify the entity types to spawn. */
	UPROPERTY(EditDefaultsOnly, Category = "LearningAgents")
	TArray<FLearningAgentsEntityInfo> Entities;

	/** Initializes the entities that the component manages. */
	UE_API virtual void InitializeLearningComponent();

	/** Resets entities that the component manages. */
	UE_API virtual void ResetLearningComponent();

	/** Spawns pooled entities at random locations. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API TScriptInterface<ILearningAgentsEntityInterface> SpawnEntitiesAtRandomLocations(TSubclassOf<AActor> EntityClass, float EntitySpawnZOffset, int32 SpawnCount);

	/** Spawns a single pooled entity with a specified transform projected in the gym. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API TScriptInterface<ILearningAgentsEntityInterface> SpawnEntityAtProjectedLocation(TSubclassOf<AActor> EntityClass, float EntitySpawnZOffset, const FTransform& Transform);

	/** Spawns multiple pooled entities. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API TArray<TScriptInterface<ILearningAgentsEntityInterface>> SpawnEntities(TSubclassOf<AActor> EntityClass, float EntitySpawnZOffset, int32 SpawnCount, const FTransform& Transform);

protected:
	UE_API virtual void BeginPlay() override;

private:
	UPROPERTY(VisibleAnywhere, Category = "LearningAgents")
	TMap<FName, FSpawnedEntitiesInfo> EntitiesPool;

	UE_API void ProjectTransform(FTransform& Transform) const;

	UE_API void RandomizeTransform(FTransform& OutTransform, float LocationZOffset) const;
	
	UE_API bool CheckEntityClasses() const;
};

#undef UE_API
