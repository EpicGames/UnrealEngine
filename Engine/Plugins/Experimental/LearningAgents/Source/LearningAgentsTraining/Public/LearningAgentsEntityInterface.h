// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "LearningAgentsGym.h"
#include "LearningAgentsEntityInterface.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

/**
* Classes that implement the Entity Training Interface need to provide definitions for how to initialize an object for training and resetting the 
* object after a training episode ends.
*/
UINTERFACE(MinimalAPI, Blueprintable, BlueprintType)
class ULearningAgentsEntityTrainingInterface : public UInterface
{
	GENERATED_BODY()
};

/** Completion modes for episodes. */
class ILearningAgentsEntityTrainingInterface
{
	GENERATED_BODY()

public:
	/** Initialize an entity at the start of training. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "LearningAgents")
	UE_API void InitializeEntity(ALearningAgentsGymBase* InGym);

	/** Resets an entity for a new training episode. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "LearningAgents")
	UE_API void ResetEntity(ALearningAgentsGymBase* InGym);

	/** Resets an entity for a new training episode. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "LearningAgents")
	UE_API ALearningAgentsGymBase* GetGym() const;
};

/**
* Classes that implement the Entity Interface need to provide definitions for enabling and disabling an entity during training.
*
* If using the LearningAgentsGym, this interface should be implemented by any actor that is spawned during training.
*/
UINTERFACE(MinimalAPI, Blueprintable, BlueprintType)
class ULearningAgentsEntityInterface : public ULearningAgentsEntityTrainingInterface
{
	GENERATED_BODY()
};

class ILearningAgentsEntityInterface: public ILearningAgentsEntityTrainingInterface
{
	GENERATED_BODY()

public:
	/** Enables an entity to be used for training. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "LearningAgents")
	UE_API void EnableEntity();

	/** Disables an entity used for training. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "LearningAgents")
	UE_API void DisableEntity();

	/** Checks whether an entity is eligible to be used for training. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "LearningAgents")
	UE_API bool IsEntityEnabled() const;
};

#undef UE_API
