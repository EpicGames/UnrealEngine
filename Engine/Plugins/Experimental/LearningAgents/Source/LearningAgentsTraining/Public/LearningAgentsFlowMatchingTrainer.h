// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsTrainer.h"

#include "GameplayTagContainer.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

#include "LearningAgentsFlowMatchingTrainer.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

namespace UE::Learning
{
	struct FFileTrainer;
	struct FReplayBuffer;
	struct IExternalTrainer;
}

class ULearningAgentsInteractor;
class ULearningAgentsFlowMatching;
class ULearningAgentsRecording;

/** The configurable settings for a ULearningAgentsFlowMatchingTrainer. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsFlowMatchingTrainerSettings
{
	GENERATED_BODY()

public:

	/** Time in seconds to wait for the training process before timing out. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float TrainerCommunicationTimeout = 10.0f;

	/** Tag used to filter records from the recording. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents")
	FGameplayTag RecordingFilterTag;
};

/** The hyperparameters for the flow matching training process. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsFlowMatchingHyperparameters
{
	GENERATED_BODY()

public:

	/** The number of iterations to run before ending training. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 NumberOfIterations = 1000000;

	/** Learning rate of the Denoiser network. Typical values are between 0.001 and 0.0001. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float LearningRate = 0.0001f;

	/**
	 * Amount of weight decay to apply to the network. Larger values encourage network weights to be smaller but too
	 * large a value can cause the network weights to collapse to all zeros.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float WeightDecay = 0.0001f;

	/**
	 * Batch size to use for training. Smaller values tend to produce better results at the cost of slowing down
	 * training. Large batch sizes are much more computationally efficient when training on the GPU.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1", UIMax = "4096"))
	uint32 BatchSize = 256;

	/**
	 * A multiplicative scaling factor that controls the observation noise that increases the perturbations added to observations.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float ObservationNoiseScale = 0.0f;

	/**
	 * Whether to perform latent normalization. If true, then we sample from a gaussian fitted to the latent distribution instead of N(0, 1).
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bNormalizeLatents = false;

	/** The seed used for any random sampling the trainer will perform, e.g. for weight initialization. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0"))
	int32 RandomSeed = 1234;
};

USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsFlowMatchingLoggingSettings
{
	GENERATED_BODY()

	/**
	 * If true, TensorBoard logs will be emitted to the intermediate directory.
	 *
	 * TensorBoard will only work if it is installed in Unreal Engine's python environment. This can be done by
	 * enabling the "Tensorboard" plugin in your project.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bUseTensorboard = false;

	/** If true, snapshots of the trained networks will be emitted to the intermediate directory. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bSaveSnapshots = true;

	/** If bSaveSnapshots is true, the snapshots will be saved at an interval defined by the specified number of iterations. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 IterationsPerSnapshot = 1000;

	/** If true, MLflow will be used for experiment tracking. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bUseMLflow = false;

	/** The URI of the MLflow Tracking Server to log to. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	FString MLflowTrackingUri = "";
};

/** Validation settings for the flow matching training process. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsFlowMatchingValidationSettings
{
	GENERATED_BODY()

	/** Whether to perform a blocking check to ensure that the interactor's observation schema is a valid subset of the recording asset's observation schema JSON description.
		For example, this option can be left off to allow different observation encoding options in the interactor than that specified at recording time.
		However, we recommend to keep this check unless you are confident of handling schema compatibility on your own.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	bool bCheckObservationSchemaCompatibility = true;

	/** Whether to perform a blocking check to ensure that the interactor's action schema is a valid subset of the recording asset's action schema JSON description.
		For example, this option can be left off to allow different action encoding options in the interactor than that specified at recording time.
		However, we recommend to keep this check unless you are confident of handling schema compatibility on your own.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	bool bCheckActionSchemaCompatibility = true;
};

/** The configurable settings for the flow matching training process. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsFlowMatchingTrainerTrainingSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	FLearningAgentsFlowMatchingHyperparameters Hyperparameters;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	FLearningAgentsFlowMatchingLoggingSettings LoggingSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	FLearningAgentsFlowMatchingValidationSettings ValidationSettings;

	/** The device to train on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	ELearningAgentsTrainingDevice Device = ELearningAgentsTrainingDevice::GPU;

	/**
	 * Whether to embed dataset normalizations into encoder and decoder networks. 
	 * This is recommended to set to true to stabilize trainings.
	 * <Important> The normalization option in action/observation schemas must be set to one of the "auto" options.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bEmbedNormalizationsInWeights = false;

	UE_API TSharedRef<FJsonObject> AsJsonConfig() const;
};

/**
 * The ULearningAgentsFlowMatchingTrainer enables imitation learning with flow matching.
 * Inference actions are generated by integrating the learned velocity field with ODE approximates.
 *
 * @see ULearningAgentsFlowMatching for the Denoiser class that performs ODE integration during inference.
 * @see ULearningAgentsRecorder to understand how to make new recordings.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class ULearningAgentsFlowMatchingTrainer : public ULearningAgentsManagerListener
{
	GENERATED_BODY()

// ----- Setup -----
public:

	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsFlowMatchingTrainer();
	UE_API ULearningAgentsFlowMatchingTrainer(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsFlowMatchingTrainer();

	/** Will automatically call EndTraining if training is still in-progress when the object is destroyed. */
	UE_API virtual void BeginDestroy() override;

	/**
	 * Constructs the flow matching trainer and runs the setup functions.
	 *
	 * @param InManager			The agent manager we are using.
	 * @param InInteractor		The agent interactor we are recording with.
	 * @param InFlowMatchingModel	The flow matching model we are using.
	 * @param Communicator		The communicator.
	 * @param Class				The trainer class.
	 * @param Name				The trainer name.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (Class = "/Script/LearningAgentsTraining.LearningAgentsFlowMatchingTrainer", DeterminesOutputType = "Class"))
	static UE_API ULearningAgentsFlowMatchingTrainer* MakeFlowMatchingTrainer(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		UPARAM(ref) ULearningAgentsFlowMatching*& InFlowMatchingModel,
		const FLearningAgentsCommunicator& Communicator,
		TSubclassOf<ULearningAgentsFlowMatchingTrainer> Class,
		const FName Name = TEXT("FlowMatchingTrainer"));

	/**
	 * Initializes the flow matching trainer and runs the setup functions.
	 *
	 * @param InManager			The agent manager we are using.
	 * @param InInteractor		The agent interactor we are recording with.
	 * @param InFlowMatchingModel The flow matching model we are using.
	 * @param InCommunicator	The communicator.
	 */
	UE_API void SetupFlowMatchingTrainer(
		ULearningAgentsManager* InManager,
		ULearningAgentsInteractor* InInteractor,
		ULearningAgentsFlowMatching* InFlowMatchingModel,
		const FLearningAgentsCommunicator& Communicator);

	/** Returns true if the trainer is currently training; Otherwise, false. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API bool IsTraining() const;

	/**
	 * Returns true if the trainer has failed to communicate with the external training process. This can be used in
	 * combination with RunTraining to avoid filling the logs with errors.
	 *
	 * @returns				True if the training has failed. Otherwise, false.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API bool HasTrainingFailed() const;

	/**
	 * Begins the training process with the provided settings.
	 *
	 * @param Recording								The data to train on.
	 * @param FlowMatchingTrainerSettings			The settings for this trainer.
	 * @param FlowMatchingTrainerTrainingSettings	The training settings for this network.
	 * @param FlowMatchingTrainerPathSettings		The path settings used by the flow matching trainer.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "FlowMatchingTrainerSettings,FlowMatchingTrainerTrainingSettings,FlowMatchingTrainerPathSettings"))
	UE_API void BeginTraining(
		const ULearningAgentsRecording* Recording,
		const FLearningAgentsFlowMatchingTrainerSettings& FlowMatchingTrainerSettings = FLearningAgentsFlowMatchingTrainerSettings(),
		const FLearningAgentsFlowMatchingTrainerTrainingSettings& FlowMatchingTrainerTrainingSettings = FLearningAgentsFlowMatchingTrainerTrainingSettings(),
		const FLearningAgentsTrainerProcessSettings& FlowMatchingTrainerPathSettings = FLearningAgentsTrainerProcessSettings());

	/** Stops the training process. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void EndTraining();

	/** Iterates the training process and gets the updated Denoiser network. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void IterateTraining();

	/**
	 * Convenience function that runs a basic training loop. If training has not been started, it will start it. On
	 * each following call to this function, it will call IterateTraining.
	 *
	 * @param Recording								The data to train on.
	 * @param FlowMatchingTrainerSettings			The settings for this trainer.
	 * @param FlowMatchingTrainerTrainingSettings	The training settings for this network.
	 * @param FlowMatchingTrainerPathSettings		The path settings used by the flow matching trainer.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "FlowMatchingTrainerSettings,FlowMatchingTrainerTrainingSettings,FlowMatchingTrainerPathSettings"))
	UE_API void RunTraining(
		const ULearningAgentsRecording* Recording,
		const FLearningAgentsFlowMatchingTrainerSettings& FlowMatchingTrainerSettings = FLearningAgentsFlowMatchingTrainerSettings(),
		const FLearningAgentsFlowMatchingTrainerTrainingSettings& FlowMatchingTrainerTrainingSettings = FLearningAgentsFlowMatchingTrainerTrainingSettings(),
		const FLearningAgentsTrainerProcessSettings& FlowMatchingTrainerPathSettings = FLearningAgentsTrainerProcessSettings());

	/**
	 * Exports training materials to file, only performing work needed for the requested export flags.
	 * Skips expensive replay buffer construction when not exporting replay buffers.
	 *
	 * @param Recording							The data to export.
	 * @param FlowMatchingTrainerSettings		The settings for this trainer.
	 * @param FlowMatchingTrainerTrainingSettings	The training settings for this network.
	 * @param ExportFlags						Which sections to export.
	 */
	UE_API void ExportToFile(
		UE::Learning::FFileTrainer& FileTrainer,
		const ULearningAgentsRecording* Recording,
		const FLearningAgentsFlowMatchingTrainerSettings& FlowMatchingTrainerSettings,
		const FLearningAgentsFlowMatchingTrainerTrainingSettings& FlowMatchingTrainerTrainingSettings,
		UE::Learning::ETrainerExportFlags ExportFlags);

// ----- Private Data -----
private:

	/** The interactor being trained. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsInteractor> Interactor;

	/** The flow matching model being trained. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsFlowMatching> Model;

	/** True if training is currently in-progress. Otherwise, false. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	bool bIsTraining = false;

	/**
	 * True if trainer encountered an unrecoverable error during training (e.g. the trainer process timed out). Otherwise, false.
	 * This exists mainly to keep the editor from locking up if something goes wrong during training.
	 */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	bool bHasTrainingFailed = false;

	UE_API TSharedRef<FJsonObject> CreateSchemaConfig(const int32 ObservationSchemaId = 0, const int32 ActionSchemaId = 0) const;
	UE_API TSharedRef<FJsonObject> CreateDataConfig() const;
	UE_API TSharedRef<FJsonObject> CreateModelConfig() const;
	UE_API TSharedRef<FJsonObject> CreateTrainerConfig(const FLearningAgentsFlowMatchingTrainerTrainingSettings& TrainingSettings) const;
	UE_API void SendConfigs(const TSharedRef<FJsonObject>& DataConfigObject, const TSharedRef<FJsonObject>& ModelConfigObject, const TSharedRef<FJsonObject>& TrainerConfigObject);

	UE_API void DoneTraining();

	TUniquePtr<UE::Learning::FReplayBuffer> ReplayBuffer;
	TSharedPtr<UE::Learning::IExternalTrainer> Trainer;

	int32 DenoiserNetworkId = INDEX_NONE;
	int32 EncoderNetworkId = INDEX_NONE;
	int32 DecoderNetworkId = INDEX_NONE;

	int32 ReplayBufferId = INDEX_NONE;

	int32 ObservationId = INDEX_NONE;
	int32 ActionId = INDEX_NONE;
	int32 MemoryStateId = INDEX_NONE;
};

#undef UE_API
