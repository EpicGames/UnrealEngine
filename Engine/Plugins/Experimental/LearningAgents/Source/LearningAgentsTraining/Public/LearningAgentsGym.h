// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "LearningAgentsEntitiesManagerComponent.h"
#include "LearningAgentsGym.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

#define INVALID_GYM_VECTOR (FVector::ZeroVector)
#define INVALID_GYM_ROTATOR (FRotator::ZeroRotator)

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnGymInitializedSignature);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnBeginGymResetSignature);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnPostGymResetSignature);

class ILearningAgentsLearningComponentInterface;

/**
 * The Gym Base abstract class handles the start and reset of entities training in a single gym.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class ALearningAgentsGymBase : public AActor
{
	GENERATED_BODY()
	
public:
	UE_API ALearningAgentsGymBase();

	/** Initializes the gym at the start of training. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void Initialize();

	/** Resets the gym for a new training episode. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void Reset();

public:
	/** Event called at a gym's initialization. */
	UPROPERTY(BlueprintAssignable, Category = "LearningAgents")
	FOnGymInitializedSignature OnGymInitialized;

	/** Event called at the start of a gym's reset. */
	UPROPERTY(BlueprintAssignable, Category = "LearningAgents")
	FOnBeginGymResetSignature OnBeginGymReset;

	/** Event called at the end of a gym's reset. */
	UPROPERTY(BlueprintAssignable, Category = "LearningAgents")
	FOnPostGymResetSignature OnPostGymReset;

public:
	/** Retrieves the current random stream used by the Gym.*/
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API void GetRandomStream(FRandomStream& OutRandomStream) const;
	
	/** Retrieves the current random stream used by the Gym.*/
	UE_API TSharedPtr<FRandomStream> GetRandomStream() const;

	/** Sets the random stream used by the Gym.*/
	UE_API void SetRandomStream(const TSharedPtr<FRandomStream>& InRandomStream);

	/** Checks if an actor is training in this gym.*/
	UE_API bool IsMemberOfGym(TObjectPtr<AActor> Actor) const;

	/** Generates a random rotator using the gym's random stream.*/
	virtual FRotator GenerateRandomRotationInGym() const PURE_VIRTUAL(ALearningAgentsGymBase::GenerateRandomRotationInGym, return INVALID_GYM_ROTATOR;);

	/** Generates a valid random point in the gym using the gym's random stream. Must be overridden in a derived class. */
	virtual FVector GenerateRandomLocationInGym() const PURE_VIRTUAL(ALearningAgentsGymBase::GenerateRandomLocationInGym, return INVALID_GYM_VECTOR;);

	/** Projects a point onto a valid location in the gym. Must be overridden in a derived class. */
	virtual FVector ProjectPointToGym(const FVector& InPoint) const PURE_VIRTUAL(ALearningAgentsGymBase::ProjectPointToGym, return INVALID_GYM_VECTOR;);

	/** Gets the gym max and min bounds. Must be overridden in a derived class. */
	virtual void GetGymExtents(FVector& OutMinExtents, FVector& OutMaxExtents) const PURE_VIRTUAL(ALearningAgentsGymBase::GetGymExtents, );

protected:
	/** The random seed used for spawn locations. Note: This is only used if a random stream is not setup by the GymsManager.*/
	UPROPERTY(EditDefaultsOnly, Category = "LearningAgents")
	int32 RandomSeed = 1234;

	TSharedPtr<FRandomStream> RandomStream; 

	UPROPERTY()
	TArray<TScriptInterface<ILearningAgentsLearningComponentInterface>> LearningComponents;

	UE_API void PopulateLearningComponents();
};

#undef UE_API
