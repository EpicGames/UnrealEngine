// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LearningAgentsGym.h"
#include "LearningAgentsGymSimple.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

class UStaticMeshComponent;

/**
 * A simple gym template class that uses a static mesh as its training floor.
 */
UCLASS(MinimalAPI, ClassGroup = (LearningAgents), BlueprintType, Blueprintable)
class ALearningAgentsGymSimple : public ALearningAgentsGymBase
{
	GENERATED_BODY()
	
public:
	UE_API ALearningAgentsGymSimple();

	UPROPERTY(EditDefaultsOnly, Category = "LearningAgents")
	TObjectPtr<UStaticMeshComponent> SimpleGymFloor;

	UE_API virtual void GetGymExtents(FVector& OutMinExtents, FVector& OutMaxExtents) const override;

	UE_API virtual FRotator GenerateRandomRotationInGym() const override;

	UE_API virtual FVector GenerateRandomLocationInGym() const override;

	UE_API virtual FVector ProjectPointToGym(const FVector& InPoint) const override;
};

#undef UE_API
