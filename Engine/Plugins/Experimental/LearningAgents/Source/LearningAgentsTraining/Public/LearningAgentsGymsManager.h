// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "LearningAgentsGymsManager.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

class ALearningAgentsGymBase;
class UNavigationSystemV1;

/**
 * This struct holds information (num to spawn etc) on one gym template to spawn.
 */
USTRUCT(BlueprintType)
struct FSpawnGymInfo
{
	GENERATED_BODY()

	/** The gym class used to construct gym instances. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents")
	TSubclassOf<ALearningAgentsGymBase> GymClass;

	/** The number of gym instances to spawn. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0"))
	int32 SpawnCount = 0;

	int32 GetCount() const
	{
		return (SpawnCount < 0) ? 0 : SpawnCount;
	}
};


/**
 * The Learning Agents GymsManager centralizes the start and reset of training for multiple gym templates.
 */
UCLASS(MinimalAPI, ClassGroup = (LearningAgents), Blueprintable)
class ALearningAgentsGymsManager : public AActor
{
	GENERATED_BODY()
	
	UE_API ALearningAgentsGymsManager();

public:		
	/** The random seed to initialize the random stream used by the GymsManager. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	int32 RandomSeed = 1234;

	/** The distance between each parallel gym when spawned. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	float GymsSpacing = 300.0f;

	/** Spawns and initializes gym instances. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void Start();

	/** Get the number of gyms managed by the GymsManager. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetGymsCount() const;

private:
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	TArray<FSpawnGymInfo> GymTemplates;

	TArray<TObjectPtr<ALearningAgentsGymBase>> SpawnedGyms;

	TSharedPtr<FRandomStream> RandomStream;

	UE_API void InitializeRandomStream();

	UE_API void SpawnGyms();
};

#undef UE_API
