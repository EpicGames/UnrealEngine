// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "LearningAgentsLearningComponentInterface.generated.h"

/**
 */
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint), BlueprintType)
class ULearningAgentsLearningComponentInterface : public UInterface
{
	GENERATED_BODY()
};

class ILearningAgentsLearningComponentInterface
{
	GENERATED_BODY()

public:
	/** Initializes the component at the start of training. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	virtual void InitializeLearningComponent() = 0;

	/** Resets the component for a new training episode. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	virtual void ResetLearningComponent() = 0;
};
