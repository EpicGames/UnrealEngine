// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsTrainer.h"

#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

#include "LearningAgentsPPOTrainer.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

class FJsonObject;

namespace UE::Learning
{
	struct FEpisodeBuffer;
	struct FReplayBuffer;
	struct IExternalTrainer;
	enum class ETrainerDevice : uint8;
}

struct FLearningAgentsCommunicator;
class ULearningAgentsCritic;
class ULearningAgentsInteractor;
class ULearningAgentsPolicy;
class ULearningAgentsTrainingEnvironment;

/** The configurable settings for a ULearningAgentsPPOTrainer. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsPPOTrainerSettings
{
	GENERATED_BODY()

public:

	/**
	 * Maximum number of steps recorded in an episode before it is added to the replay buffer. This can generally be left at the default value and
	 * does not have a large impact on training.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 MaxEpisodeStepNum = 512;

	/**
	 * Maximum number of episodes to record before running a training iteration. An iteration of training will be run when either this or
	 * MaximumRecordedEpisodesPerIteration is reached. Typical values for this should be around 1000. Setting this too small means there is not
	 * enough data each iteration for the system to train. Setting it too large means training will be very slow.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 MaximumRecordedEpisodesPerIteration = 1000;

	/**
	 * Maximum number of steps to record before running a training iteration. An iteration of training will be run when either this or
	 * MaximumRecordedEpisodesPerIteration is reached. Typical values for this should be around 10000. Setting this too small means there is not
	 * enough data each iteration for the system to train. Setting it too large means training will be very slow.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 MaximumRecordedStepsPerIteration = 10000;
};

/** The configurable settings for the PPO training process. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsPPOTrainingSettings
{
	GENERATED_BODY()

public:

	/** The number of iterations to run before ending training. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1"))
	int32 NumberOfIterations = 1000000;

	/** Learning rate of the policy network. Typical values are between 0.001 and 0.0001. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float LearningRatePolicy = 0.0001f;

	/**
	 * Learning rate of the critic network. To avoid instability generally the critic should have a larger learning
	 * rate than the policy. Typically this can be set to 10x the rate of the policy.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float LearningRateCritic = 0.001f;

	/** Amount by which to multiply the learning rate every 1000 iterations. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float LearningRateDecay = 1.0f;

	/**
	 * Amount of weight decay to apply to the network. Larger values encourage network weights to be smaller but too
	 * large a value can cause the network weights to collapse to all zeros.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float WeightDecay = 0.0001f;

	/**
	 * Batch size to use for training the policy. Large batch sizes are much more computationally efficient when training on the GPU.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1", UIMax = "4096"))
	int32 PolicyBatchSize = 1024;

	/**
	 * Batch size to use for training the critic. Large batch sizes are much more computationally efficient when training on the GPU.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1", UIMax = "4096"))
	int32 CriticBatchSize = 4096;

	/**
	 * The number of consecutive steps of observations and actions over which to train the policy. Increasing this value
	 * will encourage the policy to use its memory effectively. Too large and training can become slow and unstable.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1", UIMax = "128"))
	int32 PolicyWindowSize = 16;

	/**
	 * Number of training iterations to perform per buffer of experience gathered. This should be large enough for
	 * the critic and policy to be effectively updated, but too large and it will simply slow down training.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1", UIMax = "1024"))
	int32 IterationsPerGather = 32;

	/**
	 * Number of iterations of training to perform to warm - up the Critic. This helps speed up and stabilize training
	 * at the beginning when the Critic may be producing predictions at the wrong order of magnitude.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "1", UIMin = "1", UIMax = "128"))
	int32 CriticWarmupIterations = 8;

	/**
	 * Clipping ratio to apply to policy updates. Keeps the training "on-policy". Larger values may speed up training at
	 * the cost of stability. Conversely, too small values will keep the policy from being able to learn an
	 * optimal policy.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float EpsilonClip = 0.2f;

	/**
	 * Weight used to regularize returns. Encourages the critic not to over or under estimate returns.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float ReturnRegularizationWeight = 0.0001f;

	/**
	 * Weight for the loss used to train the policy via the PPO surrogate objective.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float ActionSurrogateWeight = 1.0f;

	/**
	 * Weight used to regularize actions. Larger values will encourage exploration and smaller actions, but too large will cause
	 * noisy actions centered around zero.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float ActionRegularizationWeight = 0.001f;

	/**
	 * Weighting used for the entropy bonus. Larger values encourage larger action noise and therefore greater
	 * exploration but can make actions very noisy.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float ActionEntropyWeight = 0.0f;

	/**
	 * This is used in the Generalized Advantage Estimation, where larger values will tend to assign more credit to recent actions. Typical
	 * values should be between 0.9 and 1.0.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float GaeLambda = 0.95f;

	/** When true, advantages are normalized. This tends to makes training more robust to adjustments of the scale of rewards. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bAdvantageNormalization = true;

	/**
	 * The minimum advantage to allow. Setting this below zero will encourage the policy to move away from bad actions,
	 * but can introduce instability.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (UIMin = "-10.0", UIMax = "0.0"))
	float MinimumAdvantage = 0.0f;

	/**
	 * The maximum advantage to allow. Making this smaller may increase training stability
	 * at the cost of some training speed.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (UIMin = "0.0", UIMax = "10.0"))
	float MaximumAdvantage = 10.0f;

	/**
	 * When true, gradient norm max clipping will be used on the policy, critic, encoder, and decoder. Set this as True if
	 * training is unstable (and adjust GradNormMax) or leave as False if unused.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bUseGradNormMaxClipping = false;

	/**
	 * The maximum gradient norm to clip updates to. Only used when bUseGradNormMaxClipping is set to true.
	 *
	 * This needs to be carefully chosen based on the size of your gradients during training. Setting too low can make it
	 * difficult to learn an optimal policy, and too high will have no impact.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (UIMin = "0.0", UIMax = "10.0"))
	float GradNormMax = 0.5f;

	/**
	 * The number of steps to trim from the start of the episode, e.g. can be useful if some things are still getting
	 * setup at the start of the episode and you don't want them used for training.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0"))
	int32 NumberOfStepsToTrimAtStartOfEpisode = 0;

	/**
	 * The number of steps to trim from the end of the episode. Can be useful if the end of the episode contains
	 * irrelevant data.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0"))
	int32 NumberOfStepsToTrimAtEndOfEpisode = 0;

	/** The seed used for any random sampling the trainer will perform, e.g. for weight initialization. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0"))
	int32 RandomSeed = 1234;

	/**
	 * The discount factor to use during training. This affects how much the agent cares about future rewards vs
	 * near-term rewards. Should typically be a value less than but near 1.0.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float DiscountFactor = 0.99f;

	/** The device to train on. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	ELearningAgentsTrainingDevice Device = ELearningAgentsTrainingDevice::GPU;

	/**
	 * If true, TensorBoard logs will be emitted to the intermediate directory.
	 * 
	 * TensorBoard will only work if it is installed in Unreal Engine's python environment. This can be done by
	 * enabling the "Tensorboard" plugin in your project.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bUseTensorboard = false;

	/** If true, snapshots of the trained networks will be emitted to the intermediate directory. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bSaveSnapshots = false;

	/** The iterations interval to save new networks snapshot. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (ClampMin = "0", UIMin = "0"))
	int32 IterationsPerSnapshot = 1000;

	/** If true, MLflow will be used for experiment tracking. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bUseMLflow = false;

	/** The URI of the MLflow Tracking Server to log to. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	FString MLflowTrackingUri = "";

	UE_API TSharedRef<FJsonObject> AsJsonConfig() const;
};

UCLASS(MinimalAPI, BlueprintType, Blueprintable, meta = (BlueprintSpawnableComponent))
class ULearningAgentsPPOTrainer : public ULearningAgentsManagerListener
{
	GENERATED_BODY()

// ----- Setup -----
public:

	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsPPOTrainer();
	UE_API ULearningAgentsPPOTrainer(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsPPOTrainer();

	/** Will automatically call EndTraining if training is still in-progress when the object is destroyed. */
	UE_API virtual void BeginDestroy() override;

	/**
	 * Constructs the trainer.
	 *
	 * @param InManager				The agent manager we are using.
	 * @param InInteractor			The agent interactor we are training with.
	 * @param InTrainingEnvironment	The training environment.
	 * @param InPolicy				The policy to be trained.
	 * @param InCritic				The critic to be trained.
	 * @param Communicator			The communicator.
	 * @param Class					The trainer class.
	 * @param Name					The trainer name.
	 * @param TrainerSettings		The trainer settings to use.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (Class = "/Script/LearningAgents.LearningAgentsPPOTrainer", DeterminesOutputType = "Class", AutoCreateRefTerm = "TrainerSettings"))
	static UE_API ULearningAgentsPPOTrainer* MakePPOTrainer(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		UPARAM(ref) ULearningAgentsTrainingEnvironment*& InTrainingEnvironment,
		UPARAM(ref) ULearningAgentsPolicy*& InPolicy,
		UPARAM(ref) ULearningAgentsCritic*& InCritic,
		const FLearningAgentsCommunicator& Communicator,
		TSubclassOf<ULearningAgentsPPOTrainer> Class,
		const FName Name = TEXT("PPOTrainer"),
		const FLearningAgentsPPOTrainerSettings& TrainerSettings = FLearningAgentsPPOTrainerSettings());

	/**
	 * Initializes the trainer.
	 *
	 * @param InManager				The agent manager we are using.
	 * @param InInteractor			The agent interactor we are training with.
	 * @param InTrainingEnvironment	The training environment.
	 * @param InPolicy				The policy to be trained.
	 * @param InCritic				The critic to be trained.
	 * @param InCommunicator		The communicator.
	 * @param TrainerSettings		The trainer settings to use.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "TrainerSettings"))
	UE_API void SetupPPOTrainer(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		UPARAM(ref) ULearningAgentsTrainingEnvironment*& InTrainingEnvironment,
		UPARAM(ref) ULearningAgentsPolicy*& InPolicy,
		UPARAM(ref) ULearningAgentsCritic*& InCritic,
		const FLearningAgentsCommunicator& Communicator,
		const FLearningAgentsPPOTrainerSettings& TrainerSettings = FLearningAgentsPPOTrainerSettings());

public:

	//~ Begin ULearningAgentsManagerListener Interface
	UE_API virtual void OnAgentsAdded_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsReset_Implementation(const TArray<int32>& AgentIds) override;
	//~ End ULearningAgentsManagerListener Interface

// ----- Training Process -----
public:

	/** Returns true if the trainer is currently training; Otherwise, false. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API const bool IsTraining() const;

	/**
	 * Begins the training process with the provided settings.
	 *
	 * @param TrainerTrainingSettings	The settings for this training run.
	 * @param TrainingGameSettings		The settings that will affect the game's simulation.
	 * @param bResetAgentsOnBegin		If true, reset all agents at the beginning of training.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "TrainerTrainingSettings,TrainingGameSettings"))
	UE_API void BeginTraining(
		const FLearningAgentsPPOTrainingSettings& TrainerTrainingSettings = FLearningAgentsPPOTrainingSettings(),
		const FLearningAgentsTrainingGameSettings& TrainingGameSettings = FLearningAgentsTrainingGameSettings(),
		const bool bResetAgentsOnBegin = true);

	/** Stops the training process. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void EndTraining();

	/**
	 * Call this function at the end of each step of your training loop. This takes the current observations/actions/
	 * rewards and moves them into the episode experience buffer. All agents with full episode buffers or those which
	 * have been signaled complete will be reset. If enough experience is gathered, it will be sent to the training
	 * process and an iteration of training will be run and the updated policy will be synced back.
	 *
	 * @param bResetAgentsOnUpdate				If true, reset all agents whenever an updated policy is received.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void ProcessExperience(const bool bResetAgentsOnUpdate = true);

	/**
	 * Convenience function that runs a basic training loop. If training has not been started, it will start it, and
	 * then call RunInference. On each following call to this function, it will call GatherRewards,
	 * GatherCompletions, and ProcessExperience, followed by RunInference.
	 *
	 * @param TrainerTrainingSettings			The settings for this training run.
	 * @param TrainingGameSettings				The settings that will affect the game's simulation.
	 * @param bResetAgentsOnBegin				If true, reset all agents at the beginning of training.
	 * @param bResetAgentsOnUpdate				If true, reset all agents whenever an updated policy is received.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "TrainerTrainingSettings,TrainingGameSettings"))
	UE_API void RunTraining(
		const FLearningAgentsPPOTrainingSettings& TrainerTrainingSettings = FLearningAgentsPPOTrainingSettings(),
		const FLearningAgentsTrainingGameSettings& TrainingGameSettings = FLearningAgentsTrainingGameSettings(),
		const bool bResetAgentsOnBegin = true,
		const bool bResetAgentsOnUpdate = true);

	/**
	 * Gets the number of step recorded in an episode for the given agent.
	 *
	 * @param AgentId	The AgentId to look-up the number of recorded episode steps for
	 * @returns			The number of recorded episode steps
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API int32 GetEpisodeStepNum(const int32 AgentId) const;

	/**
	 * Returns true if the trainer has failed to communicate with the external training process. This can be used in
	 * combination with RunTraining to avoid filling the logs with errors.
	 *
	 * @returns				True if the training has failed. Otherwise, false.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API bool HasTrainingFailed() const;

// ----- Private Data ----- 
private:

	/** The agent interactor associated with this component. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsInteractor> Interactor;

	/** The training environment associated with this component. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsTrainingEnvironment> TrainingEnvironment;

	/** The current policy for experience gathering. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsPolicy> Policy;

	/** The current critic. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsCritic> Critic;

	/** True if training is currently in-progress. Otherwise, false. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	bool bIsTraining = false;

	/**
	 * True if trainer encountered an unrecoverable error during training (e.g. the trainer process timed out). Otherwise, false.
	 * This exists mainly to keep the editor from locking up if something goes wrong during training.
	 */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	bool bHasTrainingFailed = false;

	TUniquePtr<UE::Learning::FEpisodeBuffer> EpisodeBuffer;
	TUniquePtr<UE::Learning::FReplayBuffer> ReplayBuffer;
	TSharedPtr<UE::Learning::IExternalTrainer> Trainer;

	/**
	 * The data config contains the info needed to create the neural network models and the supporting data buffers.
	 * These need to stay synchronized between UE and the trainer process, otherwise we will run into memory errors.
	 */
	UE_API TSharedRef<FJsonObject> CreateDataConfig() const;

	/**
	 * The trainer config contains the info needed run our specific training algorithm.
	 * In theory, most of these values can be easily overridden on the trainer process side without causing any errors.
	 */
	UE_API TSharedRef<FJsonObject> CreateTrainerConfig(const FLearningAgentsPPOTrainingSettings& TrainingSettings) const;

	UE_API void SendConfigs(const TSharedRef<FJsonObject>& DataConfigObject, const TSharedRef<FJsonObject>& TrainerConfigObject);

	UE_API void DoneTraining();

	UE::Learning::Agents::FGameSettingsState PreviousGameSettingsState;

	int32 PolicyNetworkId = INDEX_NONE;
	int32 CriticNetworkId = INDEX_NONE;
	int32 EncoderNetworkId = INDEX_NONE;
	int32 DecoderNetworkId = INDEX_NONE;

	int32 ReplayBufferId = INDEX_NONE;

	int32 ObservationId = INDEX_NONE;
	int32 ActionId = INDEX_NONE;
	int32 ActionModifierId = INDEX_NONE;
	int32 MemoryStateId = INDEX_NONE;
	int32 RewardId = INDEX_NONE;
};

#undef UE_API
