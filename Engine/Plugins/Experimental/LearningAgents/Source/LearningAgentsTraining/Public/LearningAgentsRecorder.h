// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsRecording.h"

#include "LearningAction.h"
#include "LearningArray.h"
#include "LearningObservation.h"

#include "Containers/Map.h"
#include "UObject/ObjectPtr.h"

#include "LearningAgentsRecorder.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnRecordingBeginSignature);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnRecordingEndSignature);

struct FLearningAgentsRecord;
class ULearningAgentsRecording;

/** The path settings for the recorder. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsRecorderPathSettings
{
	GENERATED_BODY()

public:

	UE_API FLearningAgentsRecorderPathSettings();

	/** The relative path to the Intermediate directory. Defaults to FPaths::ProjectIntermediateDir. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents", meta = (RelativePath))
	FDirectoryPath IntermediateRelativePath;

	/** The name of the sub-directory to use in the intermediate directory */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents", meta = (RelativePath))
	FString RecordingsSubdirectory = TEXT("Recordings");
};

/** A component that can be used to create recordings of training data for imitation learning. */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, meta = (BlueprintSpawnableComponent))
class ULearningAgentsRecorder : public ULearningAgentsManagerListener
{
	GENERATED_BODY()

// ----- Setup -----
public:

	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsRecorder();
	UE_API ULearningAgentsRecorder(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsRecorder();

	/** Will automatically call EndRecording if recording is still in-progress when the object is destroyed. */
	UE_API virtual void BeginDestroy() override;

	/**
	 * Constructs this object and runs the setup functions for the underlying data storage.
	 * 
	 * @param InManager					The agent manager we are using.
	 * @param InInteractor				The agent interactor we are recording with.
	 * @param Class						The recorder class
	 * @param Name						The recorder class
	 * @param RecorderPathSettings		The path settings used by the recorder.
	 * @param RecordingAsset			Optional recording asset to use. If not provided or bReinitializeRecording is 
	 *									set then a new recording object will be created.
	 * @pram bReinitializeRecording		If to reinitialize the recording asset
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (Class = "/Script/LearningAgentsTraining.LearningAgentsRecorder", DeterminesOutputType = "Class", AutoCreateRefTerm = "RecorderPathSettings"))
	static UE_API ULearningAgentsRecorder* MakeRecorder(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		TSubclassOf<ULearningAgentsRecorder> Class,
		const FName Name = TEXT("Recorder"),
		const FLearningAgentsRecorderPathSettings& RecorderPathSettings = FLearningAgentsRecorderPathSettings(),
		ULearningAgentsRecording* RecordingAsset = nullptr,
		bool bReinitializeRecording = true);

	/**
	 * Initializes this object and runs the setup functions for the underlying data storage.
	 * 
	 * @param InManager					The agent manager we are using.
	 * @param InInteractor				The agent interactor we are recording with.
	 * @param RecorderPathSettings		The path settings used by the recorder.
	 * @param RecordingAsset			Optional recording asset to use. If not provided or bReinitializeRecording is
	 *									set then a new recording object will be created.
	 * @pram bReinitializeRecording		If to reinitialize the recording asset
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "RecorderPathSettings"))
	UE_API void SetupRecorder(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		UPARAM(ref) ULearningAgentsInteractor*& InInteractor,
		const FLearningAgentsRecorderPathSettings& RecorderPathSettings = FLearningAgentsRecorderPathSettings(),
		ULearningAgentsRecording* RecordingAsset = nullptr,
		bool bReinitializeRecording = true);

public:

	//~ Begin ULearningAgentsManagerListener Interface
	UE_API virtual void OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsReset_Implementation(const TArray<int32>& AgentIds) override;
	//~ End ULearningAgentsManagerListener Interface

// ----- Recording Process -----
public:

	/** Begins the recording of the observations and actions of each added agent. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void BeginRecording();

	/** Ends the recording of the observations and actions of each agent and stores them in the current recording object. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void EndRecording();

	/** Ends the recording of the observations and actions of each agent and discards them. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void EndRecordingAndDiscard();

	/** Returns true if recording is active; Otherwise, false. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API bool IsRecording() const;

	/** Called after the recording has started */
	UPROPERTY(BlueprintAssignable, Category = "LearningAgents")
	FOnRecordingBeginSignature OnRecordingBegin;

	/** Called after the recording has stopped */
	UPROPERTY(BlueprintAssignable, Category = "LearningAgents")
	FOnRecordingEndSignature OnRecordingEnd;
	
	/**
	 * While recording, adds the current buffered observations and actions of the added agents to the internal buffer. Call this after 
	 * GatherObservations and either EvaluateAgentController (if recording a human/AI demonstration) or DecodeAndSampleActions (if recording from 
	 * another policy).
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void AddExperience();

	/** Gets the current recording object. Note: this may be empty until EndRecording has been called. */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API const ULearningAgentsRecording* GetRecordingAsset() const;

protected:
	UFUNCTION(BlueprintPure, Category="LearningAgents")
	UE_API ULearningAgentsInteractor* GetInteractor() const;

	/**
	 * Saves the schema config JSON to the recording directory with the given filename.
	 * Defaults to "schema-config.json".
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API FLearningAgentsSchema SaveSchemaConfig(const FString& FileName = TEXT("schema-config.json"));

// ----- Private -----
private:

	/** The agent interactor this recorder is associated with. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsInteractor> Interactor;

	/** The current recording object. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsRecording> Recording;

	/** True if recording is currently in-progress. Otherwise, false. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "LearningAgents")
	bool bIsRecording = false;

// ----- Private Data ----- 
private:

	/** Directory to store recordings in */
	FString RecordingDirectory;

	/** Basic structure used to buffer the observations and actions of each agent. */
	struct FAgentRecordBuffer
	{
		FAgentRecordBuffer();
		FAgentRecordBuffer(
			const int32 InObservationCompatibilityHash,
			const int32 InActionCompatibilityHash,
			const UE::Learning::Observation::FSchema InObservationSchema,
			const UE::Learning::Observation::FSchemaElement InObservationSchemaElement,
			const UE::Learning::Action::FSchema InActionSchema,
			const UE::Learning::Action::FSchemaElement InActionSchemaElement);

		const int32 ChunkSize = 1024;
		int32 StepNum = 0;
		int32 ObservationCompatibilityHash = 0;
		int32 ActionCompatibilityHash = 0;
		UE::Learning::Observation::FSchema ObservationSchema;
		UE::Learning::Observation::FSchemaElement ObservationSchemaElement;
		UE::Learning::Action::FSchema ActionSchema;
		UE::Learning::Action::FSchemaElement ActionSchemaElement;
		TArray<TLearningArray<2, float>, TInlineAllocator<16>> Observations;
		TArray<TLearningArray<2, float>, TInlineAllocator<16>> Actions;

		TLearningArrayView<1, float> GetObservation(const int32 SampleIdx);
		TLearningArrayView<1, float> GetAction(const int32 SampleIdx);
		TLearningArrayView<1, const float> GetObservation(const int32 SampleIdx) const;
		TLearningArrayView<1, const float> GetAction(const int32 SampleIdx) const;

		bool IsEmpty() const;

		void Empty();

		void Push(
			const TLearningArrayView<1, const float> Observation,
			const TLearningArrayView<1, const float> Action);

		void CopyToRecord(FLearningAgentsRecord& Record) const;
	};

	/** Array of recording buffers for each agent */
	TArray<FAgentRecordBuffer, TInlineAllocator<32>> RecordBuffers;
};

#undef UE_API
