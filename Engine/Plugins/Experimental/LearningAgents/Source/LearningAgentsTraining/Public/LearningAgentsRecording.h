// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Engine/DataAsset.h"
#include "GameplayTagContainer.h"
#include "UObject/NameTypes.h"

#include "Templates/UniquePtr.h"

#include "LearningAgentsRecording.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

namespace UE::Learning
{
	struct FReplayBuffer;
}

class ULearningAgentsInteractor;

namespace UE::Learning::Agents::Recording
{
	UE_API int32 GetMagicNumber();
	UE_API int32 GetVersionNumber();
}

/** A schema entry. */
USTRUCT(BlueprintType)
struct FLearningAgentsSchema
{
	GENERATED_BODY()

public:

	/** Observation schema in JSON format. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "LearningAgents")
	FString ObservationSchemaJson;

	/** Action schema in JSON format. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "LearningAgents")
	FString ActionSchemaJson;
};

/** Container for the record's binary data. Kept separate to keep Slate from loading it. */
UCLASS()
class ULearningAgentsRecordBlob : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TArray<float> ObservationData;

	UPROPERTY()
	TArray<float> ActionData;
};

/** A single recording of a series of observations and actions. */
USTRUCT(BlueprintType)
struct FLearningAgentsRecord
{
	GENERATED_BODY()

public:

	/** Record's name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	FString Name;

	/** Schema's name. Must correspond with a schema in the recording. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	FName SchemaName;

	/** The number of observations and actions recorded. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "LearningAgents")
	int32 StepNum = 0;

	/** The number of dimensions in the observation vector for this record */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "LearningAgents")
	int32 ObservationDimNum = 0;

	/** The number of dimensions in the action vector for this record */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "LearningAgents")
	int32 ActionDimNum = 0;

	/** The compatibility hash for the recorded observation vectors */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "LearningAgents")
	int32 ObservationCompatibilityHash = 0;

	/** The compatibility hash for the recorded action vectors */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "LearningAgents")
	int32 ActionCompatibilityHash = 0;

	/** The tags to be referenced in the imitation trainer. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	FGameplayTagContainer Tags;

	/** The binary data. */
	UPROPERTY()
	TObjectPtr<ULearningAgentsRecordBlob> Blob;
};

/** Data asset representing an array of records. */
UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class ULearningAgentsRecording : public UDataAsset
{
	GENERATED_BODY()

public:
	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsRecording();
	UE_API ULearningAgentsRecording(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsRecording();

public:

	/** Resets this recording asset to be empty. */
	UFUNCTION(CallInEditor, Category = "LearningAgents")
	UE_API virtual void ResetRecording();

	/** Appends the recording from "RecordingFile" to this recording.  */
	UFUNCTION(CallInEditor, Category = "LearningAgents")
	UE_API void AppendRecording();

	/** Appends all .record files from the specified folder to this recording asset */
	UFUNCTION(CallInEditor, Category = "LearningAgents", meta = (DisplayName = "Append All Recordings from Folder"))
	UE_API void AppendAllRecordingsFromFolder();

	/** Appends the schema from "SchemaFile" to this recording.  */
	UFUNCTION(CallInEditor, Category = "LearningAgents")
	UE_API void AppendSchema();

	/** Append schema to this recording from a file. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (RelativePath))
	UE_API void AppendSchemaFromFile(const FName& SchemaName, const FFilePath& Schema);

	/** Load this recording from a file. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (RelativePath))
	UE_API virtual void LoadRecordingFromFile(const FFilePath& File, const FName& SchemaName, const FGameplayTag& Tag, TArray<FLearningAgentsRecord>& OutRecords);

	/** Save this recording to a file. */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "LearningAgents", meta = (RelativePath))
	UE_API void SaveRecordingToFile(const FFilePath& File) const;

	/** Append to this recording from a file. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (RelativePath))
	UE_API void AppendRecordingFromFile(const FFilePath& File, const FName& SchemaName, const FGameplayTag& Tag);

	/** Loads this recording from the given recording asset */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void LoadRecordingFromAsset(ULearningAgentsRecording* RecordingAsset);

	/** Saves this recording to the given recording asset */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void SaveRecordingToAsset(ULearningAgentsRecording* RecordingAsset);

	/** Appends this recording to the given recording asset */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void AppendRecordingToAsset(ULearningAgentsRecording* RecordingAsset);

	/** Get the number of records */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetRecordNum() const;

	/** Get the number of steps in a given record */
	UFUNCTION(BlueprintPure, Category = "LearningAgents")
	UE_API int32 GetRecordStepNum(const int32 Record) const;

	/**
	 * Get the Observation Vector associated with a particular step of a given recording
	 * 
	 * @param OutObservationVector				Output Observation Vector
	 * @param OutObservationCompatibilityHash	Output Compatibility Hash for the given Observation Vector
	 * @param Record							Index of the record in the array of records.
	 * @param Step								Step of the recording
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void GetObservationVector(TArray<float>& OutObservationVector, int32& OutObservationCompatibilityHash, const int32 Record, const int32 Step);

	/**
	 * Get the Action Vector associated with a particular step of a given recording
	 *
	 * @param OutActionVector					Output Action Vector
	 * @param OutActionCompatibilityHash		Output Compatibility Hash for the given Action Vector
	 * @param Record							Index of the record in the array of records.
	 * @param Step								Step of the recording
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void GetActionVector(TArray<float>& OutActionVector, int32& OutActionCompatibilityHash, const int32 Record, const int32 Step);

public:

	/**
	 * Builds a replay buffer from this recording, filtering by tag and validating schema compatibility.
	 * Returns nullptr if no valid data is found.
	 *
	 * @param Interactor							The interactor whose observation/action schemas define the target layout.
	 * @param RecordingFilterTag					Tag used to filter records. If invalid, no filtering is applied.
	 * @param bCheckObservationSchemaCompatibility	Whether to validate observation schema subset compatibility.
	 * @param bCheckActionSchemaCompatibility		Whether to validate action schema subset compatibility.
	 * @return A replay buffer containing the copied data, or nullptr if no valid records were found.
	 */
	UE_API TUniquePtr<UE::Learning::FReplayBuffer> BuildReplayBuffer(
		const ULearningAgentsInteractor* Interactor,
		const FGameplayTag& RecordingFilterTag,
		bool bCheckObservationSchemaCompatibility,
		bool bCheckActionSchemaCompatibility) const;

	/** Marks this asset as modified even during PIE */
	UE_API void ForceMarkDirty();

	UE_API void AddSchema(const FName& SchemaName, FLearningAgentsSchema NewSchema);

public:

	/** A recording file. Used in combination with the "AppendRecording" button in the editor. */
	UPROPERTY(EditInstanceOnly, Category = "LearningAgents")
	FFilePath RecordingFile;

	/** A folder containing .record files to load. Used in combination with the "Append All Recordings From Folder" button in the editor.*/
	UPROPERTY(EditInstanceOnly, Category = "LearningAgents")
	FDirectoryPath RecordingFolder;

	/** The tag to apply to new records. Used in combination with the "AppendRecording" and "Append All Recordings from Folder" buttons in the editor. */
	UPROPERTY(EditInstanceOnly, Category = "LearningAgents")
	FGameplayTag NewTag;

	/** The schema name. Used in combination with the "AppendSchema" and "AppendRecording" buttons in the editor. */
	UPROPERTY(EditInstanceOnly, Category = "LearningAgents")
	FName NewSchemaName;

	/** A schema file. Used in combination with the "AppendSchema" button in the editor. */
	UPROPERTY(EditInstanceOnly, Category = "LearningAgents")
	FFilePath SchemaFile;

	/** Map of schemas. */
	UPROPERTY(EditInstanceOnly, Category = "LearningAgents")
	TMap<FName, FLearningAgentsSchema> Schemas;

	/** Set of records. */
	UPROPERTY(EditInstanceOnly, Category = "LearningAgents")
	TArray<FLearningAgentsRecord> Records;
};

#undef UE_API
