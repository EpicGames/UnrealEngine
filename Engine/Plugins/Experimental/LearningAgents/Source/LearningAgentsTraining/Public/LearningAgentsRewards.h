// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "LearningAgentsRewards.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

class ULearningAgentsManagerListener;
class USplineComponent;

UCLASS(MinimalAPI, BlueprintType)
class ULearningAgentsRewards : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Make a reward from a float value.
	 *
	 * @param RewardValue The float value used to create the reward.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, DefaultToSelf = "VisualLoggerListener"))
	static UE_API float MakeReward(
		const float RewardValue, 
		const float RewardScale = 1.0f, 
		const FName Tag = TEXT("Reward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);

	/**
	 * Make a reward which is equal to RewardScale when bCondition is true, otherwise returns zero.
	 *
	 * @param bCondition The condition under which to create a reward.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 2, DefaultToSelf = "VisualLoggerListener"))
	static UE_API float MakeRewardOnCondition(
		const bool bCondition, 
		const float RewardScale = 1.0f,
		const FName Tag = TEXT("ConditionReward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);

	/**
	 * Make a reward based on the distance between two locations. Gives a larger reward the further the distance between the locations.
	 *
	 * @param LocationA The first location.
	 * @param LocationB The second location.
	 * @param LocationScale The expected scale for the distance between locations.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4, DefaultToSelf = "VisualLoggerListener"))
	static UE_API float MakeRewardFromLocationDifference(
		const FVector LocationA,
		const FVector LocationB,
		const float LocationScale = 100.0f,
		const float RewardScale = 1.0f,
		const FName Tag = TEXT("LocationDifferenceReward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);

	/**
	 * Make a reward when the distance between two locations is below a threshold, otherwise returns zero.
	 *
	 * @param LocationA The first location.
	 * @param LocationB The second location.
	 * @param DistanceThreshold The distance threshold.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4, DefaultToSelf = "VisualLoggerListener"))
	static UE_API float MakeRewardOnLocationDifferenceBelowThreshold(
		const FVector LocationA, 
		const FVector LocationB, 
		const float DistanceThreshold = 100.0f, 
		const float RewardScale = 1.0f,
		const FName Tag = TEXT("LocationDifferenceBelowThresholdReward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);

	/**
	 * Make a reward when the distance between two locations is above a threshold, otherwise returns zero.
	 *
	 * @param LocationA The first location.
	 * @param LocationB The second location.
	 * @param DistanceThreshold The distance threshold.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4, DefaultToSelf = "VisualLoggerListener"))
	static UE_API float MakeRewardOnLocationDifferenceAboveThreshold(
		const FVector LocationA,
		const FVector LocationB,
		const float DistanceThreshold = 100.0f,
		const float RewardScale = 1.0f,
		const FName Tag = TEXT("LocationDifferenceAboveThresholdReward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);

	/**
	 * Make a reward based on how similar two locations are.
	 *
	 * @param LocationA The first location.
	 * @param LocationB The second location.
	 * @param LocationScale The expected scale for the distance between locations.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 4, DefaultToSelf = "VisualLoggerListener"))
	static UE_API float MakeRewardFromLocationSimilarity(
		const FVector LocationA, 
		const FVector LocationB, 
		const float LocationScale = 100.0f, 
		const float RewardScale = 1.0f,
		const FName Tag = TEXT("LocationSimilarityReward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);

	/**
	 * Make a reward based on how similar two rotations are.
	 *
	 * @param RotationA The first rotation.
	 * @param RotationB The second rotation.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerRotationLocationA A location for the visual logger to display the first rotation in the world.
	 * @param VisualLoggerRotationLocationB A location for the visual logger to display the second rotation in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, DefaultToSelf = "VisualLoggerListener"))
	static UE_API float MakeRewardFromRotationSimilarity(
		const FRotator RotationA, 
		const FRotator RotationB, 
		const float RewardScale = 1.0f,
		const FName Tag = TEXT("RotationSimilarityReward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerRotationLocationA = FVector::ZeroVector,
		const FVector VisualLoggerRotationLocationB = FVector::ZeroVector,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);

	/**
	 * Make a reward based on how similar two rotations are represented as quaternions.
	 *
	 * @param RotationA The first rotation.
	 * @param RotationB The second rotation.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerRotationLocationA A location for the visual logger to display the first rotation in the world.
	 * @param VisualLoggerRotationLocationB A location for the visual logger to display the second rotation in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, DefaultToSelf = "VisualLoggerListener"))
	static UE_API float MakeRewardFromRotationSimilarityAsQuats(
		const FQuat RotationA, 
		const FQuat RotationB, 
		const float RewardScale = 1.0f,
		const FName Tag = TEXT("RotationSimilarityReward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerRotationLocationA = FVector::ZeroVector,
		const FVector VisualLoggerRotationLocationB = FVector::ZeroVector,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);

	/**
	 * Make a reward based on how similar two angles are. Angles should be given in degrees.
	 *
	 * @param AngleA The first angle.
	 * @param AngleB The second angle.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerAngleLocationA A location for the visual logger to display the first angle in the world.
	 * @param VisualLoggerAngleLocationB A location for the visual logger to display the second angle in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, DefaultToSelf = "VisualLoggerListener"))
	static UE_API float MakeRewardFromAngleSimilarity(
		const float AngleA, 
		const float AngleB, 
		const float RewardScale = 1.0f,
		const FName Tag = TEXT("AngleSimilarityReward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerAngleLocationA = FVector::ZeroVector,
		const FVector VisualLoggerAngleLocationB = FVector::ZeroVector,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);

	/**
	 * Make a reward based on how similar two directions are.
	 *
	 * @param DirectionA The first direction.
	 * @param DirectionB The second direction.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerDirectionLocationA A location for the visual logger to display the first direction in the world.
	 * @param VisualLoggerDirectionLocationB A location for the visual logger to display the second direction in the world.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerArrowLength The length of the arrow to display for the directions.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 3, DefaultToSelf = "VisualLoggerListener", DirectionA="1.0,0.0,0.0", DirectionB = "1.0,0.0,0.0"))
	static UE_API float MakeRewardFromDirectionSimilarity(
		const FVector DirectionA, 
		const FVector DirectionB, 
		const float RewardScale = 1.0f,
		const FName Tag = TEXT("DirectionSimilarityReward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerDirectionLocationA = FVector::ZeroVector,
		const FVector VisualLoggerDirectionLocationB = FVector::ZeroVector,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const float VisualLoggerArrowLength = 100.0f,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);

	/**
	 * Make a reward based on the velocity an object is moving along a spline.
	 *
	 * @param SplineComponent The spline.
	 * @param Location The location of the object.
	 * @param Velocity The velocity of the object.
	 * @param VelocityScale The expected scale for the velocity.
	 * @param RewardScale The scale of the reward. Use a negative scale to create a penalty.
	 * @param FiniteDifferenceDelta The finite difference to use when computing the velocity along the spline.
	 * @param Tag The tag for the reward. Used for debugging.
	 * @param bVisualLoggerEnabled When true, debug data will be sent to the visual logger.
	 * @param VisualLoggerListener The listener object which is making this reward. This must be set to use logging.
	 * @param VisualLoggerAgentId The agent id associated with this reward.
	 * @param VisualLoggerLocation A location for the visual logger information in the world.
	 * @param VisualLoggerColor The color for the visual logger display.
	 * @return The resulting reward value.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AdvancedDisplay = 6, DefaultToSelf = "VisualLoggerListener"))
	static UE_API float MakeRewardFromVelocityAlongSpline(
		const USplineComponent* SplineComponent, 
		const FVector Location, 
		const FVector Velocity, 
		const float VelocityScale = 200.0f, 
		const float RewardScale = 1.0f, 
		const float FiniteDifferenceDelta = 10.0f,
		const FName Tag = TEXT("VelocityAlongSplineReward"),
		const bool bVisualLoggerEnabled = false,
		ULearningAgentsManagerListener* VisualLoggerListener = nullptr,
		const int32 VisualLoggerAgentId = -1,
		const FVector VisualLoggerLocation = FVector::ZeroVector,
		const FLinearColor VisualLoggerColor = FLinearColor::Green);
};

#undef UE_API
