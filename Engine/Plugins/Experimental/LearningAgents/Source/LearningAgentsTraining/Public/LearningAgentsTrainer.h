// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningTrainer.h"

#include "LearningAgentsTrainer.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

/** Enumeration of the training devices. */
UENUM(BlueprintType, Category = "LearningAgents")
enum class ELearningAgentsTrainingDevice : uint8
{
	CPU,
	GPU,
};

/**
 * The configurable game settings for a ULearningAgentsTrainer. These allow the timestep and physics tick to be fixed
 * during training, which can enable ticking faster than real-time.
 */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsTrainingGameSettings
{
	GENERATED_BODY()

public:

	/**
	 * If true, the game will run in fixed time step mode (i.e the frame's delta times will always be the same
	 * regardless of how much wall time has passed). This can enable faster than real-time training if your game runs
	 * quickly. If false, the time steps will match real wall time.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bUseFixedTimeStep = true;

	/**
	 * Determines the amount of time for each frame when bUseFixedTimeStep is true; Ignored if false. You want this
	 * time step to match as closely as possible to the expected inference time steps, otherwise your training results
	 * may not generalize to your game.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (DisplayName = "Fixed Time Step Frequency (Hz)"), meta = (ClampMin = "0.0", UIMin = "0.0"))
	float FixedTimeStepFrequency = 60.0f;

	/** If true, set the physics delta time to match the fixed time step. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bSetMaxPhysicsStepToFixedTimeStep = true;

	/** If true, the MaxFPS console variable will be set to a negative number during training; Otherwise, it will not. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bDisableMaxFPS = true;

	/** If true, VSync will be disabled; Otherwise, it will not. Disabling VSync can speed up the game simulation. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bDisableVSync = true;

	/** If true, the viewport rendering will be unlit; Otherwise, it will not. Disabling lighting can speed up the game simulation. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bUseUnlitViewportRendering = false;

#if WITH_EDITORONLY_DATA

	/** If true, the Use Less CPU In The Background editor setting will be disabled. This prevents the editor from running slowly when minimized. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bDisableUseLessCPUInTheBackground = true;

	/** If true, Editor VSync will be disabled; Otherwise, it will not. Disabling Editor VSync can speed up the game simulation. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	bool bDisableEditorVSync = true;

#endif
};

namespace UE::Learning::Agents
{
	// ----- Private Recording of GameSettings ----- 
	struct FGameSettingsState
	{
		bool bFixedTimestepUsed = false;
		float FixedTimeStepDeltaTime = -1.0f;
		float MaxPhysicsStep = -1.0f;
		int32 MaxFPS = 120;
		bool bVSyncEnabled = true;
		int32 ViewModeIndex = -1;

		bool bUseLessCPUInTheBackground = true;
		bool bEditorVSyncEnabled = true;
	};

	/** Get the learning agents trainer device from the UE::Learning trainer device. */
	LEARNINGAGENTSTRAINING_API ELearningAgentsTrainingDevice GetLearningAgentsTrainingDevice(const ETrainerDevice Device);

	/** Get the UE::Learning trainer device from the learning agents trainer device. */
	LEARNINGAGENTSTRAINING_API ETrainerDevice GetTrainingDevice(const ELearningAgentsTrainingDevice Device);

	LEARNINGAGENTSTRAINING_API void ApplyGameSettings(const FLearningAgentsTrainingGameSettings& Settings, const UWorld* World, FGameSettingsState& OutGameSettingsState);

	LEARNINGAGENTSTRAINING_API void RevertGameSettings(const FGameSettingsState& Settings, const UWorld* World);
}

/** The path settings for the trainer. */
USTRUCT(BlueprintType, Category = "LearningAgents")
struct FLearningAgentsTrainerProcessSettings
{
	GENERATED_BODY()

public:

	UE_API FLearningAgentsTrainerProcessSettings();

	/** Training task name. Used to avoid filename collisions with other training processes running on the same machine. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	FString TaskName = TEXT("Training");

	/** The relative path to the engine for editor builds. Defaults to FPaths::EngineDir. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (RelativePath))
	FDirectoryPath EditorEngineRelativePath;

	/**
	 * The relative path to the editor engine folder for non-editor builds.
	 *
	 * If we want to run training in cooked, non-editor builds, then by default we wont have access to python and the
	 * LearningAgents training scripts - these are editor-only things and are stripped during the cooking process.
	 *
	 * However, running training in non-editor builds can be very important - we probably want to disable rendering
	 * and sound while we are training to make experience gathering as fast as possible - and for any non-trivial game
	 * is simply may not be realistic to run it for a long time in play-in-editor.
	 *
	 * For this reason even in non-editor builds we let you provide the path where all of these editor-only things can
	 * be found. This allows you to run training when these things actually exist somewhere accessible to the executable,
	 * which will usually be the case on a normal development machine or cloud machine if it is set up that way.
	 *
	 * Since non-editor builds can be produced in a number of different ways, this is not set by default and cannot
	 * use a directory picker since it is relative to the final location of where your cooked, non-editor executable
	 * will exist rather than the current with-editor executable.
	 */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	FString NonEditorEngineRelativePath;

	/** The relative path to the Intermediate directory. Defaults to FPaths::ProjectIntermediateDir. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents", meta = (RelativePath))
	FDirectoryPath EditorIntermediateRelativePath;

	/** The relative path to the intermediate folder for non-editor builds. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	FString NonEditorIntermediateRelativePath;

	/** The complete path to a custom trainer module when training with a custom trainer. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	FDirectoryPath CustomTrainerModulePath;

	/** The complete path to a custom trainer module when training with a custom trainer for non-editor builds. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	FString NonEditorCustomTrainerModulePath;

	/** Trainer file name. The name of the python file to use for training. Do NOT include the '.py' file extension. */
	UPROPERTY(EditAnywhere, Category = "LearningAgents")
	FString TrainerFileName = TEXT("learning_core.train_behavior_cloning");

public:

	/** Gets the Relative Editor Engine Path accounting for if this is an editor build or not  */
	UE_API FString GetEditorEnginePath() const;

	/** Gets the Relative Intermediate Path  */
	UE_API FString GetIntermediatePath() const;

	/** Gets the Custom Trainer Module Path  */
	UE_API FString GetCustomTrainerModulePath() const;

	/** Gets the NNE Runtime Basic CPU Python content path. Auto-discovers from the plugin if not overridden. */
	UE_API FString GetNneCpuContentPath() const;
};

#undef UE_API
