// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningAgentsManagerListener.h"
#include "LearningAgentsCompletions.h"

#include "LearningAgentsTrainingEnvironment.generated.h"

#define UE_API LEARNINGAGENTSTRAINING_API

namespace UE::Learning
{
	struct FResetInstanceBuffer;
}

UCLASS(MinimalAPI, Abstract, HideDropdown, BlueprintType, Blueprintable)
class ULearningAgentsTrainingEnvironment : public ULearningAgentsManagerListener
{
	GENERATED_BODY()

// ----- Setup -----
public:

	// These constructors/destructors are needed to make forward declarations happy
	UE_API ULearningAgentsTrainingEnvironment();
	UE_API ULearningAgentsTrainingEnvironment(FVTableHelper& Helper);
	UE_API virtual ~ULearningAgentsTrainingEnvironment();

	/**
	 * Constructs the training environment and runs the setup functions for rewards and completions.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (DeterminesOutputType = "Class", AutoCreateRefTerm = "TrainerSettings"))
	static UE_API ULearningAgentsTrainingEnvironment* MakeTrainingEnvironment(
		UPARAM(ref) ULearningAgentsManager*& InManager,
		TSubclassOf<ULearningAgentsTrainingEnvironment> Class,
		const FName Name = TEXT("TrainingEnvironment"));

	/**
	 * Initializes the training environment and runs the setup functions for rewards and completions.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents", meta = (AutoCreateRefTerm = "TrainerSettings"))
	UE_API void SetupTrainingEnvironment(UPARAM(ref) ULearningAgentsManager*& InManager);

public:

	//~ Begin ULearningAgentsManagerListener Interface
	UE_API virtual void OnAgentsAdded_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsRemoved_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsReset_Implementation(const TArray<int32>& AgentIds) override;
	UE_API virtual void OnAgentsManagerTick_Implementation(const TArray<int32>& AgentIds, const float DeltaTime) override;
	//~ End ULearningAgentsManagerListener Interface

// ----- Rewards -----
public:

	/**
	 * This callback should be overridden by the Trainer and gathers the reward value for the given agent.
	 *
	 * @param OutReward			Output reward for the given agent.
	 * @param AgentId			Agent id to gather reward for.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void GatherAgentReward(float& OutReward, const int32 AgentId);


	/**
	 * This callback can be overridden by the Trainer and gathers all the reward values for the given set of agents. By default this will call
	 * GatherAgentReward on each agent.
	 *
	 * @param OutRewards		Output rewards for each agent in AgentIds
	 * @param AgentIds			Agents to gather rewards for.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void GatherAgentRewards(TArray<float>& OutRewards, const TArray<int32>& AgentIds);

// ----- Completions ----- 
public:

	/**
	 * This callback should be overridden by the Trainer and gathers the completion for a given agent.
	 *
	 * @param OutCompletion		Output completion for the given agent.
	 * @param AgentId			Agent id to gather completion for.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void GatherAgentCompletion(ELearningAgentsCompletion& OutCompletion, const int32 AgentId);

	/**
	 * This callback can be overridden by the Trainer and gathers all the completions for the given set of agents. By default this will call
	 * GatherAgentCompletion on each agent.
	 *
	 * @param OutCompletions	Output completions for each agent in AgentIds
	 * @param AgentIds			Agents to gather completions for.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void GatherAgentCompletions(TArray<ELearningAgentsCompletion>& OutCompletions, const TArray<int32>& AgentIds);

// ----- Resets ----- 
public:

	/**
	 * This callback should be overridden by the Trainer and resets the episode for the given agent.
	 *
	 * @param AgentId			The id of the agent that need resetting.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void ResetAgentEpisode(const int32 AgentId);

	/**
	 * This callback can be overridden by the Trainer and resets all episodes for each agent in the given set. By default this will call
	 * ResetAgentEpisode on each agent.
	 *
	 * @param AgentIds			The ids of the agents that need resetting.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "LearningAgents", Meta = (ForceAsFunction))
	UE_API void ResetAgentEpisodes(const TArray<int32>& AgentIds);

	// ----- Training Process -----
public:

	/**
	 * Call this function when it is time to evaluate the rewards for your agents. This should be done at the beginning
	 * of each iteration of your training loop after the initial step, i.e. after taking an action, you want to get into
	 * the next state before evaluating the rewards.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void GatherRewards();

	/**
	 * Call this function when it is time to evaluate the completions for your agents. This should be done at the beginning
	 * of each iteration of your training loop after the initial step, i.e. after taking an action, you want to get into
	 * the next state before evaluating the completions.
	 */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	UE_API void GatherCompletions();

	/**
	 * Returns true if GatherRewards has been called and the reward already set for the given agent.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API bool HasReward(const int32 AgentId) const;

	/**
	 * Returns true if GatherCompletions has been called and the completion already set for the given agent.
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API bool HasCompletion(const int32 AgentId) const;

	/**
	 * Gets the current reward for an agent. Should be called only after GatherRewards.
	 *
	 * @param AgentId	The AgentId to look-up the reward for
	 * @returns			The reward
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API float GetReward(const int32 AgentId) const;

	/**
	 * Gets the current completion for an agent. Should be called only after GatherCompletions.
	 *
	 * @param AgentId	The AgentId to look-up the completion for
	 * @returns			The completion type
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API ELearningAgentsCompletion GetCompletion(const int32 AgentId) const;

	/**
	 * Gets the current elapsed episode time for the given agent.
	 *
	 * @param AgentId	The AgentId to look-up the episode time for
	 * @returns			The elapsed episode time
	 */
	UFUNCTION(BlueprintPure, Category = "LearningAgents", meta = (AgentId = "-1"))
	UE_API float GetEpisodeTime(const int32 AgentId) const;

// ----- Non-blueprint public interface -----
public:

	/** Gets the rewards as a const array view. */
	UE_API const TLearningArrayView<1, const float> GetRewardArrayView() const;

	/** Gets the reward iteration value for the given agent id. */
	UE_API uint64 GetRewardIteration(const int32 AgentId) const;

	/** Gets the agent completion mode for the given agent id. */
	UE_API UE::Learning::ECompletionMode GetAgentCompletion(const int32 AgentId) const;

	/** Gets the agent completions as a const array view. */
	UE_API const TLearningArrayView<1, const UE::Learning::ECompletionMode> GetAgentCompletions() const;

	/** Gets the all completions as a const array view. */
	UE_API const TLearningArrayView<1, const UE::Learning::ECompletionMode> GetAllCompletions() const;

	/** Computes a combined completion buffer for agents that have been completed manually and those which have reached the maximum episode length. */
	UE_API void SetAllCompletions(UE::Learning::FIndexSet AgentSet);

	/** Gets the episode completions as a mutable array view. */
	UE_API TLearningArrayView<1, UE::Learning::ECompletionMode> GetEpisodeCompletions();

	/** Gets the completion iteration value for the given agent id. */
	UE_API uint64 GetCompletionIteration(const int32 AgentId) const;

	UE_API UE::Learning::FResetInstanceBuffer& GetResetBuffer() const;

// ----- Private Data ----- 
private:

	/** Callback Reward Output */
	TArray<float> RewardBuffer;

	/** Callback Completion Output */
	TArray<ELearningAgentsCompletion> CompletionBuffer;

	/** Reward Buffer */
	TLearningArray<1, float> Rewards;

	/** Agent Completions Buffer */
	TLearningArray<1, UE::Learning::ECompletionMode> AgentCompletions;

	/** Episode Completions Buffer */
	TLearningArray<1, UE::Learning::ECompletionMode> EpisodeCompletions;

	/** All Completions Buffer */
	TLearningArray<1, UE::Learning::ECompletionMode> AllCompletions;

	/** Agent episode times */
	TLearningArray<1, float> EpisodeTimes;

	TUniquePtr<UE::Learning::FResetInstanceBuffer> ResetBuffer;

	/** Number of times rewards have been evaluated for all agents */
	TLearningArray<1, uint64, TInlineAllocator<32>> RewardIteration;

	/** Number of times completions have been evaluated for all agents */
	TLearningArray<1, uint64, TInlineAllocator<32>> CompletionIteration;
};

#undef UE_API
