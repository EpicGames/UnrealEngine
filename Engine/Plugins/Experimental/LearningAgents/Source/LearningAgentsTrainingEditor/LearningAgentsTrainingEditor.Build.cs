// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LearningAgentsTrainingEditor : ModuleRules
{
	public LearningAgentsTrainingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[] {
			});

		PrivateIncludePaths.AddRange(
			new string[] {
			});

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Learning",
				"LearningAgents",
				"LearningTraining",
				"LearningAgentsTraining",
				"Slate",
				"SlateCore",
				"EditorStyle",
				"PropertyEditor",
				"UnrealEd",
				"UMG"
			});

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			});
	}
}
