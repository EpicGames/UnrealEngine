// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsDepthMapVisualizer.h"
#include "Engine/Canvas.h"
#include "Blueprint/WidgetTree.h"
#include "Components/CanvasPanel.h"
#include "Components/Image.h"
#include "Components/CanvasPanelSlot.h"
#include "LearningLog.h"

ULearningAgentsDepthMapVisualizerComponent::ULearningAgentsDepthMapVisualizerComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
	MarkAsEditorOnlySubobject();
}

void ULearningAgentsDepthMapVisualizerComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
	if (!DepthMapComp)
	{
		UE_LOGF(LogLearning, Error, "Unable to find a corresponding LearningAgentsDepthMapComponent! Failed to render texture.");
		return;
	}

	Widget->SetDepthValues(DepthMapComp->GetDepthMapFlatArray());
}

void ULearningAgentsDepthMapVisualizerComponent::BeginPlay()
{
	Super::BeginPlay();

	DepthMapComp = GetOwner()->FindComponentByClass<ULearningAgentsDepthMapComponent>();
	if (!DepthMapComp)
	{
		UE_LOGF(LogLearning, Error, "Unable to find a corresponding LearningAgentsDepthMapComponent! Failed to render texture.");
		return;
	}

	SetComponentTickInterval(DepthMapComp->PrimaryComponentTick.TickInterval);

	Widget = CreateWidget<ULearningAgentsDepthMapWidget>(GetWorld(), ULearningAgentsDepthMapWidget::StaticClass());
	Widget->InitializeView(RenderSize, RenderPosition, DepthMapComp->DepthMapConfig.Height, DepthMapComp->DepthMapConfig.Width);
	Widget->AddToViewport();
}

void ULearningAgentsDepthMapWidget::SetDepthValues(const TArray<float>& InDepthValues)
{
	DepthValues = InDepthValues;

	if (DepthMapRenderer)
	{
		DepthMapRenderer->UpdateResource();
	}
}

void ULearningAgentsDepthMapWidget::InitializeView(FVector2D RenderSize, FVector2D RenderPosition, int32 DMapHeight, int32 DMapWidth)
{
	DepthValues.SetNumUninitialized(DMapWidth * DMapHeight);

	DepthMapRenderer = UCanvasRenderTarget2D::CreateCanvasRenderTarget2D(GetWorld(), UCanvasRenderTarget2D::StaticClass(), DMapWidth, DMapHeight);
	DepthMapRenderer->OnCanvasRenderTargetUpdate.AddDynamic(this, &ULearningAgentsDepthMapWidget::OnCanvasRenderTargetUpdate);
	DepthMapRenderer->UpdateResource();

	UCanvasPanel* Root = WidgetTree->ConstructWidget<UCanvasPanel>(UCanvasPanel::StaticClass());
	WidgetTree->RootWidget = Root;

	UImage* Image = WidgetTree->ConstructWidget<UImage>(UImage::StaticClass());
	Root->AddChildToCanvas(Image);

	UCanvasPanelSlot* CanvasSlot = Cast<UCanvasPanelSlot>(Image->Slot);
	CanvasSlot->SetSize(RenderSize);
	CanvasSlot->SetPosition(RenderPosition);

	FSlateBrush Brush;
	Brush.SetResourceObject(DepthMapRenderer);
	Image->SetBrush(Brush);
	Image->InvalidateLayoutAndVolatility();
}

void ULearningAgentsDepthMapWidget::OnCanvasRenderTargetUpdate(UCanvas* Canvas, int32 Width, int32 Height)
{
	for (int32 H = 0; H < Height; H++)
	{
		for (int32 W = 0; W < Width; W++)
		{
			float DepthValue = DepthValues[H * Width + W];
			FLinearColor DepthColor(DepthValue, DepthValue, DepthValue, 1.0f);
			Canvas->K2_DrawBox(FVector2D(W, H), FVector2D(1.0f, 1.0f), 1.0f, DepthColor);
		}
	}
}
