// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsFlowMatchingTrainerEditor.h"
#include "LearningExternalTrainer.h"
#include "LearningLog.h"
#include "Containers/Ticker.h"
#include "Logging/StructuredLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsFlowMatchingTrainerEditor)

ALearningAgentsFlowMatchingTrainerEditor::ALearningAgentsFlowMatchingTrainerEditor() {}

void ALearningAgentsFlowMatchingTrainerEditor::StartTraining()
{
	if (ULearningAgentsFlowMatchingTrainer* Trainer = Cast<ULearningAgentsFlowMatchingTrainer>(SetupTraining()))
	{
		LearningAgentsFlowMatchingTrainer = Trainer;
	}
	else
	{
		LearningAgentsFlowMatchingTrainer = nullptr;
	}

	if (!LearningAgentsFlowMatchingTrainer)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Training has failed. Invalid flow matching trainer. Was the flow matching trainer set with SetupTraining?", ("Name", GetName()));
		return;
	}

	if (LearningAgentsFlowMatchingTrainer->IsTraining())
	{
		UE_LOGFMT(LogLearning, Warning, "{Name}: Cannot start training when an existing training process is active!", ("Name", GetName()));
		return;
	}

	UE_LOGFMT(LogLearning, Display, "{Name}: Starting training...", ("Name", GetName()));

	LearningAgentsFlowMatchingTrainer->BeginTraining(Recording, FlowMatchingTrainerSettings, FlowMatchingTrainerTrainingSettings, FlowMatchingTrainerPathSettings);

	FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateLambda([this](float DeltaTime)->bool {
			if (LearningAgentsFlowMatchingTrainer->IsTraining())
			{
				LearningAgentsFlowMatchingTrainer->IterateTraining();
				return true;
			}
			return false;
			}),
		TrainingTickInterval
	);
}

void ALearningAgentsFlowMatchingTrainerEditor::StartExport(UE::Learning::ETrainerExportFlags Flags)
{
	ExportFlags = Flags;

	ULearningAgentsFlowMatchingTrainer* Trainer = Cast<ULearningAgentsFlowMatchingTrainer>(SetupTraining());
	if (!Trainer)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Export failed. Invalid flow matching trainer. Was the flow matching trainer set with SetupTraining?", ("Name", GetName()));
		ExportFlags = UE::Learning::ETrainerExportFlags::All;
		return;
	}

	if (!FileTrainer)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Export failed. No file communicator. Use MakeFileCommunicator in your SetupTraining Blueprint.", ("Name", GetName()));
		ExportFlags = UE::Learning::ETrainerExportFlags::All;
		return;
	}

	UE_LOGFMT(LogLearning, Display, "{Name}: Exporting training materials...", ("Name", GetName()));

	Trainer->ExportToFile(*FileTrainer, Recording, FlowMatchingTrainerSettings, FlowMatchingTrainerTrainingSettings, Flags);

	ExportFlags = UE::Learning::ETrainerExportFlags::All;
}

void ALearningAgentsFlowMatchingTrainerEditor::StopTraining()
{
	if (!LearningAgentsFlowMatchingTrainer)
	{
		UE_LOGFMT(LogLearning, Error, "{Name}: Cannot stop training with an invalid flow matching trainer.", ("Name", GetName()));
		return;
	}
	if (!LearningAgentsFlowMatchingTrainer->IsTraining())
	{
		UE_LOGFMT(LogLearning, Warning, "{Name}: There is no training active. Cannot stop training when training has not started.", ("Name", GetName()));
		return;
	}

	UE_LOGFMT(LogLearning, Display, "{Name}: Ending training...", ("Name", GetName()));
	LearningAgentsFlowMatchingTrainer->EndTraining();
}

bool ALearningAgentsFlowMatchingTrainerEditor::IsTraining() const
{
	if (LearningAgentsFlowMatchingTrainer)
	{
		return LearningAgentsFlowMatchingTrainer->IsTraining();
	}
	return false;
}
