// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsImitationTrainerEditor.h"
#include "LearningExternalTrainer.h"
#include "LearningLog.h"
#include "Containers/Ticker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsImitationTrainerEditor)

ALearningAgentsImitationTrainerEditor::ALearningAgentsImitationTrainerEditor() {}

void ALearningAgentsImitationTrainerEditor::StartTraining()
{
	if (ULearningAgentsImitationTrainer* Trainer = Cast<ULearningAgentsImitationTrainer>(SetupTraining()))
	{
		LearningAgentsImitationTrainer = Trainer;
	}
	else
	{
		LearningAgentsImitationTrainer = nullptr;
	}

	if (!LearningAgentsImitationTrainer)
	{
		UE_LOGF(LogLearning, Error, "%ls: Training has failed. Invalid imitation trainer. Was the imitation trainer set with SetupTraining?", *GetName());
		return;
	}

	if (LearningAgentsImitationTrainer->IsTraining())
	{
		UE_LOGF(LogLearning, Warning, "%ls: Cannot start training when an existing training process is active!", *GetName());
		return;
	}

	UE_LOGF(LogLearning, Display, "%ls: Starting training...", *GetName());

	LearningAgentsImitationTrainer->BeginTraining(Recording, ImitationTrainerSettings, ImitationTrainerTrainingSettings, ImitationTrainerPathSettings);

	FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateLambda([this](float DeltaTime)->bool {
			if (LearningAgentsImitationTrainer->IsTraining())
			{
				LearningAgentsImitationTrainer->IterateTraining();
				return true;
			}
			return false;
			}),
		TrainingTickInterval
	);
}

void ALearningAgentsImitationTrainerEditor::StartExport(UE::Learning::ETrainerExportFlags Flags)
{
	ExportFlags = Flags;

	ULearningAgentsImitationTrainer* Trainer = Cast<ULearningAgentsImitationTrainer>(SetupTraining());
	if (!Trainer)
	{
		UE_LOGF(LogLearning, Error, "%ls: Export failed. Invalid imitation trainer. Was the imitation trainer set with SetupTraining?", *GetName());
		ExportFlags = UE::Learning::ETrainerExportFlags::All;
		return;
	}

	if (!FileTrainer)
	{
		UE_LOGF(LogLearning, Error, "%ls: Export failed. No file communicator. Use MakeFileCommunicator in your SetupTraining Blueprint.", *GetName());
		ExportFlags = UE::Learning::ETrainerExportFlags::All;
		return;
	}

	UE_LOGF(LogLearning, Display, "%ls: Exporting training materials...", *GetName());

	Trainer->ExportToFile(*FileTrainer, Recording, ImitationTrainerSettings, ImitationTrainerTrainingSettings, Flags);

	ExportFlags = UE::Learning::ETrainerExportFlags::All;
}

void ALearningAgentsImitationTrainerEditor::StopTraining()
{
	if (!LearningAgentsImitationTrainer)
	{
		UE_LOGF(LogLearning, Error, "%ls: Cannot stop training with an invalid imitation trainer.", *GetName());
		return;
	}
	if (!LearningAgentsImitationTrainer->IsTraining())
	{
		UE_LOGF(LogLearning, Warning, "%ls: There is no training active. Cannot stop training when training has not started.", *GetName());
		return;
	}

	UE_LOGF(LogLearning, Display, "%ls: Ending training...", *GetName());
	LearningAgentsImitationTrainer->EndTraining();
}

bool ALearningAgentsImitationTrainerEditor::IsTraining() const
{
	if (LearningAgentsImitationTrainer)
	{
		return LearningAgentsImitationTrainer->IsTraining();
	}
	return false;
}
