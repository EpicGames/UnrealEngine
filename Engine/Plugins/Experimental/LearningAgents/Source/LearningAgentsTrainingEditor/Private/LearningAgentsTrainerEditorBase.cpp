// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsTrainerEditorBase.h"
#include "LearningExternalTrainer.h"
#include "LearningLog.h"
#include "Containers/Ticker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LearningAgentsTrainerEditorBase)

ALearningAgentsTrainerEditorBase::ALearningAgentsTrainerEditorBase()
{
	LearningAgentsManager = CreateDefaultSubobject<ULearningAgentsManager>(TEXT("LearningAgentsManager"));
}

FLearningAgentsCommunicator ALearningAgentsTrainerEditorBase::MakeFileCommunicator(FDirectoryPath EditorIntermediateRelativePath)
{
	FileTrainer = MakeShared<UE::Learning::FFileTrainer>(UE::Learning::Trainer::GetIntermediatePath(EditorIntermediateRelativePath.Path), ExportFlags);
	FLearningAgentsCommunicator Communicator;
	Communicator.Trainer = FileTrainer;
	return Communicator;
}
