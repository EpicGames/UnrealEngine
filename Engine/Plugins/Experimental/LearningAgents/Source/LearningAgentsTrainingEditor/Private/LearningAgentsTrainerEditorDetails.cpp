// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsTrainerEditorDetails.h"
#include "LearningAgentsTrainerEditorBase.h"
#include "LearningExternalTrainer.h"
#include "LearningLog.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "Misc/Attribute.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "LearningAgentsTrainerEditorDetails"

TSharedRef<IDetailCustomization> FLearningAgentsTrainerEditorDetails::MakeInstance()
{
	return MakeShareable(new FLearningAgentsTrainerEditorDetails);
}

void FLearningAgentsTrainerEditorDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.GetObjectsBeingCustomized(EditedObjects);
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("LearningAgents");

	Category.AddCustomRow(LOCTEXT("RunTrainingCategory", "Run Training"))
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("RunTrainingLabel", "Run Training"))
				.Font(DetailBuilder.GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("RunTrainingButton", "Run"))
						.OnClicked(FOnClicked::CreateSP(this, &FLearningAgentsTrainerEditorDetails::OnRunClicked))
						.IsEnabled_Lambda([this]()
						{
							if (EditedObjects.Num() > 0)
							{
								if (ALearningAgentsTrainerEditorBase* TrainerEditor = Cast<ALearningAgentsTrainerEditorBase>(EditedObjects[0].Get()))
								{
									return !TrainerEditor->IsTraining();
								}
							}
							return false;
						})
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("StopTrainingButton", "Stop"))
						.OnClicked(FOnClicked::CreateSP(this, &FLearningAgentsTrainerEditorDetails::OnStopClicked))
						.IsEnabled_Lambda([this]()
						{
							if (EditedObjects.Num() > 0)
							{
								if (ALearningAgentsTrainerEditorBase* TrainerEditor = Cast<ALearningAgentsTrainerEditorBase>(EditedObjects[0].Get()))
								{
									return TrainerEditor->IsTraining();
								}
							}
							return false;
						})
				]
		];

	auto IsNotTrainingLambda = [this]()
	{
		if (EditedObjects.Num() > 0)
		{
			if (ALearningAgentsTrainerEditorBase* TrainerEditor = Cast<ALearningAgentsTrainerEditorBase>(EditedObjects[0].Get()))
			{
				return !TrainerEditor->IsTraining();
			}
		}
		return false;
	};

	Category.AddCustomRow(LOCTEXT("ExportCategory", "Export"))
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("ExportLabel", "Export"))
				.Font(DetailBuilder.GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("ExportAllButton", "All"))
						.OnClicked(FOnClicked::CreateSP(this, &FLearningAgentsTrainerEditorDetails::OnExportAllClicked))
						.IsEnabled_Lambda(IsNotTrainingLambda)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("ExportNetworksButton", "Networks"))
						.OnClicked(FOnClicked::CreateSP(this, &FLearningAgentsTrainerEditorDetails::OnExportNetworksClicked))
						.IsEnabled_Lambda(IsNotTrainingLambda)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("ExportReplayBuffersButton", "Replay Buffers"))
						.OnClicked(FOnClicked::CreateSP(this, &FLearningAgentsTrainerEditorDetails::OnExportReplayBuffersClicked))
						.IsEnabled_Lambda(IsNotTrainingLambda)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SButton)
						.Text(LOCTEXT("ExportTrainerConfigButton", "Trainer Config"))
						.OnClicked(FOnClicked::CreateSP(this, &FLearningAgentsTrainerEditorDetails::OnExportTrainerConfigClicked))
						.IsEnabled_Lambda(IsNotTrainingLambda)
				]
		];
}

FReply FLearningAgentsTrainerEditorDetails::OnRunClicked()
{
	if (EditedObjects.Num() > 0)
	{
		if (ALearningAgentsTrainerEditorBase* TrainerEditor = Cast<ALearningAgentsTrainerEditorBase>(EditedObjects[0].Get()))
		{
			TrainerEditor->StartTraining();
		}
	}
	return FReply::Handled();
}

FReply FLearningAgentsTrainerEditorDetails::OnStopClicked()
{
	if (EditedObjects.Num() > 0)
	{
		if (ALearningAgentsTrainerEditorBase* TrainerEditor = Cast<ALearningAgentsTrainerEditorBase>(EditedObjects[0].Get()))
		{
			TrainerEditor->StopTraining();
		}
	}
	return FReply::Handled();
}

FReply FLearningAgentsTrainerEditorDetails::OnExportAllClicked()
{
	if (EditedObjects.Num() > 0)
	{
		if (ALearningAgentsTrainerEditorBase* TrainerEditor = Cast<ALearningAgentsTrainerEditorBase>(EditedObjects[0].Get()))
		{
			TrainerEditor->StartExport(UE::Learning::ETrainerExportFlags::All);
		}
	}
	return FReply::Handled();
}

FReply FLearningAgentsTrainerEditorDetails::OnExportNetworksClicked()
{
	if (EditedObjects.Num() > 0)
	{
		if (ALearningAgentsTrainerEditorBase* TrainerEditor = Cast<ALearningAgentsTrainerEditorBase>(EditedObjects[0].Get()))
		{
			TrainerEditor->StartExport(UE::Learning::ETrainerExportFlags::Networks);
		}
	}
	return FReply::Handled();
}

FReply FLearningAgentsTrainerEditorDetails::OnExportReplayBuffersClicked()
{
	if (EditedObjects.Num() > 0)
	{
		if (ALearningAgentsTrainerEditorBase* TrainerEditor = Cast<ALearningAgentsTrainerEditorBase>(EditedObjects[0].Get()))
		{
			TrainerEditor->StartExport(UE::Learning::ETrainerExportFlags::ReplayBuffers);
		}
	}
	return FReply::Handled();
}

FReply FLearningAgentsTrainerEditorDetails::OnExportTrainerConfigClicked()
{
	if (EditedObjects.Num() > 0)
	{
		if (ALearningAgentsTrainerEditorBase* TrainerEditor = Cast<ALearningAgentsTrainerEditorBase>(EditedObjects[0].Get()))
		{
			TrainerEditor->StartExport(UE::Learning::ETrainerExportFlags::TrainerConfig);
		}
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
