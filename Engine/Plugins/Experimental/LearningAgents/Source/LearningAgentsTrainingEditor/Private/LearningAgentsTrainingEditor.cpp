// Copyright Epic Games, Inc. All Rights Reserved.

#include "LearningAgentsTrainingEditor.h"
#include "LearningAgentsTrainerEditorDetails.h"
#include "LearningAgentsImitationTrainerEditor.h"
#include "LearningAgentsFlowMatchingTrainerEditor.h"
#include "PropertyEditorModule.h"
#include "PropertyEditorDelegates.h"
#include "Misc/CoreDelegates.h"

void FLearningAgentsTrainingEditorModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(
		ALearningAgentsImitationTrainerEditor::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FLearningAgentsTrainerEditorDetails::MakeInstance)
	);
	PropertyModule.RegisterCustomClassLayout(
		ALearningAgentsFlowMatchingTrainerEditor::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FLearningAgentsTrainerEditorDetails::MakeInstance)
	);
}

void FLearningAgentsTrainingEditorModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout(ALearningAgentsImitationTrainerEditor::StaticClass()->GetFName());
		PropertyModule.UnregisterCustomClassLayout(ALearningAgentsFlowMatchingTrainerEditor::StaticClass()->GetFName());
	}
}
	
IMPLEMENT_MODULE(FLearningAgentsTrainingEditorModule, LearningAgentsTrainingEditor)
