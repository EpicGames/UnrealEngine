// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Blueprint/UserWidget.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "LearningAgentsDepthMapComponent.h"
#include "LearningAgentsDepthMapVisualizer.generated.h"

#define UE_API LEARNINGAGENTSTRAININGEDITOR_API

UCLASS()
class ULearningAgentsDepthMapWidget : public UUserWidget
{
    GENERATED_BODY()

public:
	UE_API void SetDepthValues(const TArray<float>& InDepthValues);
	UE_API void InitializeView(FVector2D RenderSize, FVector2D RenderPosition, int32 DMapHeight, int32 DMapWidth);

private:
	TArray<float> DepthValues;

	TObjectPtr<UCanvasRenderTarget2D> DepthMapRenderer;

	UFUNCTION()
	void OnCanvasRenderTargetUpdate(UCanvas* Canvas, int32 Width, int32 Height);
};

UCLASS(Blueprintable, ClassGroup = (LearningAgents), meta = (BlueprintSpawnableComponent))
class ULearningAgentsDepthMapVisualizerComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	ULearningAgentsDepthMapVisualizerComponent();

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	FVector2D RenderSize = FVector2D(512, 512);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "LearningAgents")
	FVector2D RenderPosition = FVector2D(100, 100);

	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

protected:
	virtual void BeginPlay() override;

private:
	TObjectPtr<ULearningAgentsDepthMapComponent> DepthMapComp;
	TObjectPtr<ULearningAgentsDepthMapWidget> Widget;
};

#undef UE_API
