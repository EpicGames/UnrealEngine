// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "LearningAgentsManager.h"
#include "LearningAgentsCommunicator.h"
#include "LearningAgentsTrainerEditorBase.h"
#include "LearningAgentsFlowMatchingTrainer.h"
#include "LearningAgentsFlowMatching.h"
#include "LearningAgentsFlowMatchingTrainerEditor.generated.h"

#define UE_API LEARNINGAGENTSTRAININGEDITOR_API

/** Editor callable flow matching learning trainer. */
UCLASS(BlueprintType, Blueprintable)
class ALearningAgentsFlowMatchingTrainerEditor : public ALearningAgentsTrainerEditorBase
{
	GENERATED_BODY()

public:
	UE_API ALearningAgentsFlowMatchingTrainerEditor();

	UE_API virtual void StartTraining() override;

	UE_API virtual void StopTraining() override;

	UE_API virtual bool IsTraining() const override;

	UE_API virtual void StartExport(UE::Learning::ETrainerExportFlags Flags) override;

	UPROPERTY(BlueprintReadWrite, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsFlowMatchingTrainer> LearningAgentsFlowMatchingTrainer;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents")
	FLearningAgentsFlowMatchingTrainerSettings FlowMatchingTrainerSettings;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents")
	FLearningAgentsFlowMatchingTrainerTrainingSettings FlowMatchingTrainerTrainingSettings;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents")
	FLearningAgentsTrainerProcessSettings FlowMatchingTrainerPathSettings;
	
};

#undef UE_API
