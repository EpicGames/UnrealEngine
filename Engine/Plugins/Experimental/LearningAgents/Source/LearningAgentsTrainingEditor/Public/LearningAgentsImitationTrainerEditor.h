// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "LearningAgentsManager.h"
#include "LearningAgentsCommunicator.h"
#include "LearningAgentsTrainerEditorBase.h"
#include "LearningAgentsImitationTrainer.h"
#include "LearningAgentsImitationTrainerEditor.generated.h"

#define UE_API LEARNINGAGENTSTRAININGEDITOR_API

/** Editor callable imitation learning trainer. */
UCLASS(BlueprintType, Blueprintable)
class ALearningAgentsImitationTrainerEditor : public ALearningAgentsTrainerEditorBase
{
	GENERATED_BODY()

public:
	UE_API ALearningAgentsImitationTrainerEditor();

	/** Start training. */
	UE_API virtual void StartTraining() override;

	/** Stop training. */
	UE_API virtual void StopTraining() override;

	UE_API virtual bool IsTraining() const override;

	UE_API virtual void StartExport(UE::Learning::ETrainerExportFlags Flags) override;

	UPROPERTY(BlueprintReadWrite, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsImitationTrainer> LearningAgentsImitationTrainer;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents")
	FLearningAgentsImitationTrainerSettings ImitationTrainerSettings;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents")
	FLearningAgentsImitationTrainerTrainingSettings ImitationTrainerTrainingSettings;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents")
	FLearningAgentsTrainerProcessSettings ImitationTrainerPathSettings;
};

#undef UE_API
