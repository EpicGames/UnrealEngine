// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "LearningAgentsManager.h"
#include "LearningAgentsCommunicator.h"
#include "LearningAgentsRecording.h"
#include "LearningAgentsTrainer.h"
#include "LearningExternalTrainer.h"
#include "LearningAgentsTrainerEditorBase.generated.h"

#define UE_API LEARNINGAGENTSTRAININGEDITOR_API

/** Editor callable imitation learning trainer. */
UCLASS(Abstract)
class ALearningAgentsTrainerEditorBase : public AActor
{
	GENERATED_BODY()

public:
	ALearningAgentsTrainerEditorBase();

	/** Setup the imitation trainer with necessary components in blueprints. */
	UFUNCTION(BlueprintImplementableEvent, Category = "LearningAgents", meta = (ReturnDisplayName = "Trainer"))
	ULearningAgentsManagerListener* SetupTraining();

	/** Start training. */
	virtual void StartTraining() PURE_VIRTUAL(ALearningAgentsTrainerEditorBase::StartTraining);

	/** Stop training. */
	virtual void StopTraining() PURE_VIRTUAL(ALearningAgentsTrainerEditorBase::StopTraining);

	/** Is training. */
	virtual bool IsTraining() const PURE_VIRTUAL(ALearningAgentsTrainerEditorBase::IsTraining, return false;);

	/** Make File communicator to file training materials for external training. */
	UFUNCTION(BlueprintCallable, Category = "LearningAgents")
	FLearningAgentsCommunicator MakeFileCommunicator(FDirectoryPath EditorIntermediateRelativePath);

	/** Export training materials to file with the given export flags. No training loop is started. */
	virtual void StartExport(UE::Learning::ETrainerExportFlags Flags) PURE_VIRTUAL(ALearningAgentsTrainerEditorBase::StartExport);

	/** The export flags used by MakeFileCommunicator. */
	UE::Learning::ETrainerExportFlags ExportFlags = UE::Learning::ETrainerExportFlags::All;

	/** The file trainer created by MakeFileCommunicator, used by StartExport. */
	TSharedPtr<UE::Learning::FFileTrainer> FileTrainer;

	float TrainingTickInterval = 0.0f;
	
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "LearningAgents")
    TObjectPtr<ULearningAgentsManager> LearningAgentsManager;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "LearningAgents")
	TObjectPtr<ULearningAgentsRecording> Recording;
};

#undef UE_API
