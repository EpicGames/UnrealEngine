// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"

#define UE_API LEARNINGAGENTSTRAININGEDITOR_API

/** Details customization for the flow matching learning trainer */
class FLearningAgentsTrainerEditorDetails : public IDetailCustomization
{
public:
	UE_API static TSharedRef<IDetailCustomization> MakeInstance();
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	TArray<TWeakObjectPtr<UObject>> EditedObjects;
	FReply OnRunClicked();
	FReply OnStopClicked();
	FReply OnExportAllClicked();
	FReply OnExportNetworksClicked();
	FReply OnExportReplayBuffersClicked();
	FReply OnExportTrainerConfigClicked();
};

#undef UE_API
